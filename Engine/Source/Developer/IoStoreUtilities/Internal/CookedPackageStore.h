// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "IO/IoBuffer.h"
#include "IO/IoChunkId.h"
#include "IO/IoHash.h"
#include "IO/IoStatus.h"
#include "Serialization/PackageStore.h"
#include "Serialization/FileRegions.h"
#include "Tasks/Task.h"
#include "UObject/NameTypes.h"

namespace UE { class FZenStoreHttpClient; }

class FCookedPackageStore
{
public:
	struct FChunkInfo
	{
		FIoChunkId ChunkId;
		FIoHash ChunkHash;
		uint64 ChunkSize = 0;
		FName PackageName;
		FString RelativeFileName;
		TArray<FFileRegion> FileRegions;
	};

	IOSTOREUTILITIES_API FCookedPackageStore(FStringView InCookedDir);

	FIoStatus LoadManifest(const TCHAR* ManifestFilename);
	IOSTOREUTILITIES_API FIoStatus LoadProjectStore(const TCHAR* ProjectStoreFilename);

	IOSTOREUTILITIES_API FIoChunkId GetChunkIdFromFileName(const FString& Filename) const;
	IOSTOREUTILITIES_API const FChunkInfo* GetChunkInfoFromChunkId(const FIoChunkId& ChunkId) const;
	IOSTOREUTILITIES_API const FChunkInfo* GetChunkInfoFromFileName(const FString& Filename) const;
	IOSTOREUTILITIES_API FString GetRelativeFilenameFromChunkId(const FIoChunkId& ChunkId) const;
	IOSTOREUTILITIES_API FName GetPackageNameFromChunkId(const FIoChunkId& ChunkId) const;
	IOSTOREUTILITIES_API FName GetPackageNameFromFileName(const FString& Filename) const;
	IOSTOREUTILITIES_API const FPackageStoreEntryResource* GetPackageStoreEntry(FPackageId PackageId) const;

	IOSTOREUTILITIES_API bool HasZenStoreClient() const;
	IOSTOREUTILITIES_API UE::FZenStoreHttpClient* GetZenStoreClient();

	IOSTOREUTILITIES_API TIoStatusOr<FIoBuffer> ReadChunk(const FIoChunkId& ChunkId);
	IOSTOREUTILITIES_API UE::Tasks::FTask ReadChunkAsync(const FIoChunkId& ChunkId, TFunction<void(TIoStatusOr<FIoBuffer>)>&& Callback);

private:
	void ParseOplog(FCbObject& OplogObject);
	FIoStatus LoadChunkHashes();

	TUniquePtr<UE::FZenStoreHttpClient> ZenStoreClient;
	FString CookedDir;
	TMap<FPackageId, FPackageStoreEntryResource> PackageIdToEntry;
	TMap<FString, FIoChunkId> FilenameToChunkIdMap;
	TMap<FIoChunkId, FChunkInfo> ChunkInfoMap;
};
