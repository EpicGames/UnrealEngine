// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Serialization/AsyncLoading2.h"
#include "Serialization/PackageStore.h"
#include "IO/IoDispatcher.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/UObjectMarks.h"
#include "IO/IoContainerHeader.h"

namespace Verse { struct VUniqueString; }
class FBufferWriter;

class FPackageStoreNameMapBuilder
{
public:
	void SetNameMapType(FMappedName::EType InNameMapType)
	{
		NameMapType = InNameMapType;
	}

	void AddName(FName Name)
	{
		AddName(FDisplayNameEntryId(Name));
	}

	void AddName(FDisplayNameEntryId DisplayId)
	{
		NameMap.Add(DisplayId);
		int32 Index = NameMap.Num();
		NameIndices.Add(DisplayId, Index);
	}

	void MarkNamesAsReferenced(const TArray<FName>& Names, TArray<int32>& OutNameIndices)
	{
		for (FName Name : Names)
		{
			FDisplayNameEntryId DisplayId(Name);
			int32& Index = NameIndices.FindOrAdd(DisplayId);
			if (Index == 0)
			{
				NameMap.Add(DisplayId);
				Index = NameMap.Num();
			}

			OutNameIndices.Add(Index - 1);
		}
	}

	void MarkNameAsReferenced(FName Name)
	{
		FDisplayNameEntryId DisplayId(Name);
		int32& Index = NameIndices.FindOrAdd(DisplayId);
		if (Index == 0)
		{
			NameMap.Add(DisplayId);
			Index = NameMap.Num();
		}
	}

	FMappedName MapName(FName Name) const
	{
		int32 Index = NameIndices.FindChecked(FDisplayNameEntryId(Name));
		return FMappedName::Create(Index - 1, Name.GetNumber(), NameMapType);
	}

	TConstArrayView<FDisplayNameEntryId> GetNameMap() const
	{
		return NameMap;
	}

	void Empty()
	{
		NameIndices.Empty();
		NameMap.Empty();
	}

private:
	TMap<FDisplayNameEntryId, int32> NameIndices;
	TArray<FDisplayNameEntryId> NameMap;
	FMappedName::EType NameMapType = FMappedName::EType::Package;
};

class FPackageStorePackage
{
public:
	FPackageId GetId() const
	{
		return Id;
	}

	uint64 GetHeaderSize() const
	{
		return HeaderBuffer.DataSize();
	}

	uint64 GetNameCount() const
	{
		return NameMapBuilder.GetNameMap().Num();
	}

	const bool HasEditorData() const
	{
		return (PackageFlags & PKG_FilterEditorOnly) == 0;
	}

	const TArray<FBulkDataMapEntry>& GetBulkDataEntries() const
	{
		return BulkDataEntries;
	}

private:
	struct FExportGraphNode
	{
		FExportBundleEntry BundleEntry;
		TArray<FExportGraphNode*> InternalDependencies;
		int32 IncomingEdgeCount = 0;
		bool bIsPublic = false;
	};

	struct FExport
	{
		FName ObjectName;
		uint64 PublicExportHash = 0;
		FPackageObjectIndex OuterIndex;
		FPackageObjectIndex ClassIndex;
		FPackageObjectIndex SuperIndex;
		FPackageObjectIndex TemplateIndex;
		EObjectFlags ObjectFlags = RF_NoFlags;
		uint64 SerialOffset = uint64(-1);
		uint64 SerialSize = uint64(-1);
		bool bNotForClient = false;
		bool bNotForServer = false;
		bool bIsPublic = false;
		FExportGraphNode* Nodes[FExportBundleEntry::ExportCommandType_Count] = { nullptr };
	};
	struct FCellExport
	{
		uint64 PublicExportHash = 0;
		FName CppClassInfo;
		uint64 SerialOffset = uint64(-1);
		uint64 SerialLayoutSize = uint64(-1);
		uint64 SerialSize = uint64(-1);
		FExportGraphNode* Nodes[FExportBundleEntry::ExportCommandType_Count] = { nullptr };
	};

	struct FUnresolvedExport
	{
		FString FullName;
		FString FullPath;
	};

	struct FUnresolvedImport
	{
		FString FullName;
		FString FullPath;
		FName FromPackageName;
		int32 FromPackageNameLen = 0;
		bool bIsScriptImport = false;
		bool bIsVerseVNIImport = false;
		bool bIsImportOfPackage = false;
	};

	struct FGraphData
	{
		TArray<FExportBundleEntry> ExportBundleEntries;
		TArray<FDependencyBundleHeader> DependencyBundleHeaders;
		TArray<FDependencyBundleEntry> DependencyBundleEntries;
	};

	struct FExportBundleGraphNode
	{
		FPackageStorePackage* Package = nullptr;
		TArray<FExportGraphNode*> ExportGraphNodes;
		int32 Index = -1;
		int32 IncomingEdgeCount = 0;
	};

	struct FImportedPackageRef
	{
		FImportedPackageRef(FName InName)
			: Id(FPackageId::FromName(InName))
			, Name(InName)
		{
		}

		bool operator<(const FImportedPackageRef& Other) const
		{
			return Id < Other.Id;
		}

		const FPackageId Id;
		const FName Name;
	};

	FPackageId Id;
	FName Name;

	TOptional<FZenPackageVersioningInfo> VersioningInfo;

	FPackageStoreNameMapBuilder NameMapBuilder;
	TArray<FPackageObjectIndex> Imports;
	TArray<FExport> Exports;
	TArray<FPackageObjectIndex> CellImports;
	TArray<FCellExport> CellExports;
	TArray<FExportGraphNode> ExportGraphNodes;
	FGraphData GraphData;

	TArray<FImportedPackageRef> ImportedPackages;
	TArray<uint64> ImportedPublicExportHashes;
	TArray<FBulkDataMapEntry> BulkDataEntries;
	TArray<FName> SoftPackageReferences;

	FIoBuffer HeaderBuffer;

	uint32 PackageFlags = 0;
	uint32 CookedHeaderSize = 0;
	
	friend class FPackageStoreOptimizer;
};

class FPackageStoreOptimizer
{
public:
	struct FScriptObjectData
	{
		FName ObjectName;
		FString FullName;
		FPackageObjectIndex GlobalIndex;
		FPackageObjectIndex OuterIndex;
		FPackageObjectIndex CDOClassIndex;
	};
	struct FScriptCellData
	{
		FUtf8String VersePath;
		FPackageObjectIndex GlobalIndex;
	};

	uint64 GetTotalScriptObjectCount() const
	{
		return TotalScriptObjectCount;
	}

	IOSTOREUTILITIES_API void Initialize();
	void Initialize(const FIoBuffer& ScriptObjectsBuffer);

	FPackageStorePackage* CreateMissingPackage(const FName& Name) const;
	FPackageStorePackage* CreatePackageFromCookedHeader(const FName& Name, const FIoBuffer& CookedHeaderBuffer) const;
	FIoBuffer CreatePackageBuffer(const FPackageStorePackage* Package, const FIoBuffer& CookedExportsBuffer) const;
	FPackageStoreEntryResource CreatePackageStoreEntry(const FPackageStorePackage* Package, const FPackageStorePackage* OptionalSegmentPackage) const;
	IOSTOREUTILITIES_API FIoBuffer CreateScriptObjectsBuffer() const;
	void LoadScriptObjectsBuffer(const FIoBuffer& ScriptObjectsBuffer);

	IOSTOREUTILITIES_API static void FindScriptObjects(TMap<FPackageObjectIndex, FScriptObjectData>& OutScriptObjectsMap);
	IOSTOREUTILITIES_API static void AppendPathForPublicExportHash(UObject* Object, FStringBuilderBase& OutPath);
	IOSTOREUTILITIES_API static bool TryGetPublicExportHash(FStringView PackageRelativeExportPath, uint64& OutPublicExportHash);

private:

	struct FCookedHeaderData
	{
		FPackageFileSummary Summary;
		TArray<FName> SummaryNames;
		TArray<FObjectImport> ObjectImports;
		TArray<FObjectExport> ObjectExports;
		TArray<FCellImport> CellImports;
		TArray<FCellExport> CellExports;
		TArray<FPackageIndex> PreloadDependencies;
		TArray<FObjectDataResource> DataResources;
		TArray<FName> SoftPackageReferences;
	};

	using FExportGraphEdges = TMultiMap<FPackageStorePackage::FExportGraphNode*, FPackageStorePackage::FExportGraphNode*>;

	static uint64 GetPublicExportHash(FStringView PackageRelativeExportPath);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	static uint64 GetCellExportHash(Verse::VUniqueString* VersePath);
#endif
	static uint64 GetCellExportHash(FUtf8StringView VersePath);
	FCookedHeaderData LoadCookedHeader(const FIoBuffer& CookedHeaderBuffer) const;
	void ResolveImport(FPackageStorePackage::FUnresolvedImport* Imports, FPackageStorePackage::FUnresolvedExport* Exports, const FObjectImport* ObjectImports, const FObjectExport* ObjectExports, const FName& PackageName, int32 LocalImportIndex) const;
	void ProcessImports(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package, TArray<FPackageStorePackage::FUnresolvedImport>& UnresolvedImports, TArray<FPackageStorePackage::FUnresolvedExport>& UnresolvedExports) const;
	void ResolveExport(FPackageStorePackage::FUnresolvedImport* Imports, FPackageStorePackage::FUnresolvedExport* Exports, const FObjectImport* ObjectImports, const FObjectExport* ObjectExports, const FName& PackageName, int32 LocalExportIndex) const;
	void ProcessExports(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package, TArray<FPackageStorePackage::FUnresolvedImport>& UnresolvedImports, TArray<FPackageStorePackage::FUnresolvedExport>& UnresolvedExports) const;
	void ProcessPreloadDependencies(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package) const;
	void ProcessDataResources(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package) const;
	TArray<FPackageStorePackage::FExportGraphNode*> SortExportGraphNodesInLoadOrder(FPackageStorePackage* Package, FExportGraphEdges& Edges) const;
	void CreateExportBundle(FPackageStorePackage* Package) const;
	void FinalizePackageHeader(FPackageStorePackage* Package) const;
	static void FindScriptObjectsRecursive(TMap<FPackageObjectIndex, FScriptObjectData>& OutScriptObjectsMap,
		FPackageObjectIndex OuterIndex, bool bOuterIsVerseVNI, UObject* Object);
	void FindScriptObjects();

	TMap<FPackageObjectIndex, FScriptObjectData> ScriptObjectsMap;
	TMap<FPackageObjectIndex, FScriptCellData> ScriptCellsMap;
	uint64 TotalScriptObjectCount = 0;
};
