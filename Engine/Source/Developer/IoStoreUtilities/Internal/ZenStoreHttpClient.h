// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/AnsiString.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Experimental/ZenServerInterface.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "IO/IoDispatcher.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

#define UE_API IOSTOREUTILITIES_API

class FCbPackage;
class FCbObject;

namespace UE {
	class IHttpRequest;
namespace Zen {
	struct FZenHttpRequestPool;
	class FZenHttpRequest;
	enum class EContentType;

	namespace Internal {
		struct FZenHttpClientState;
	}
}

/**
 * HTTP protocol implementation of Zen Store client interface
 */
class FZenStoreHttpClient
{
public:
	UE_API FZenStoreHttpClient();
	UE_API FZenStoreHttpClient(FStringView HostName, uint16 Port);
	UE_API FZenStoreHttpClient(UE::Zen::FServiceSettings&& InSettings);
	UE_API ~FZenStoreHttpClient();

	UE_API bool TryCreateProject(FStringView InProjectId, FStringView InParentProjectId, FStringView InOplogId, FStringView ServerRoot, 
					FStringView EngineRoot, FStringView ProjectRoot,
					FStringView ProjectFilePath);
	UE_API bool TryCreateOplog(FStringView InProjectId, FStringView InOplogId, FStringView InOplogLifetimeMarkerPath);
	UE_API bool TryDeleteOplog(FStringView InProjectId, FStringView InOplogId);

	UE_API void InitializeReadOnly(FStringView InProjectId, FStringView InOplogId);

	UE_API bool IsConnected() const;

	UE_API void StartBuildPass();
	UE_API TIoStatusOr<uint64> EndBuildPass(FCbPackage OpEntry);

	UE_API TIoStatusOr<uint64> AppendOp(FCbPackage OpEntry);

	UE_API TIoStatusOr<uint64> GetChunkSize(const FIoChunkId& Id);
	UE_API TIoStatusOr<FIoBuffer> ReadChunk(const FIoChunkId& Id, uint64 Offset = 0, uint64 Size = ~0ull);
	UE_API TIoStatusOr<FIoBuffer> ReadChunk(const FIoHash& RawHash, uint64 Offset = 0, uint64 Size = ~0ull);
	// TODO: Expand API to add partial chunk requests and ModTag field
	UE_INTERNAL UE_API void ReadChunks(TConstArrayView<FIoChunkId> Ids, TUniqueFunction<void(const FIoChunkId& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback);
	UE_INTERNAL UE_API void ReadChunks(TConstArrayView<FIoHash> RawHashes, TUniqueFunction<void(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback);
	UE_INTERNAL UE_API void ReadChunksAsync(TConstArrayView<FIoChunkId> Ids, TUniqueFunction<void(const FIoChunkId& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback);
	UE_INTERNAL UE_API void ReadChunksAsync(TConstArrayView<FIoHash> RawHashes, TUniqueFunction<void(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback);

	UE_DEPRECATED(5.6, "Use ReadChunk(FIoHash) instead")
	UE_API TIoStatusOr<FIoBuffer> ReadOpLogAttachment(FStringView Id);

	const UE::Zen::FZenServiceInstance& GetZenServiceInstance() const { return ZenService.GetInstance(); }
	UE::Zen::FZenServiceInstance& GetZenServiceInstance() { return ZenService.GetInstance(); }

	UE_API TFuture<TIoStatusOr<FCbObject>> GetProjectInfo();
	UE_API TFuture<TIoStatusOr<FCbObject>> GetOplog(bool bTrimByReferencedset = true);
	UE_API TFuture<TArray<TIoStatusOr<FCbObject>>> GetOplogEntries(FStringView TargetProject, FStringView TargetOplog, TArray<FString> EntryKeys);
	UE_API TFuture<TIoStatusOr<FCbObject>> GetFiles();
	UE_API TFuture<TIoStatusOr<FCbObject>> GetChunkInfos();

	static UE_API const UTF8CHAR* FindOrAddAttachmentId(FUtf8StringView AttachmentText);
	static UE_API const UTF8CHAR* FindAttachmentId(FUtf8StringView AttachmentText);

private:
	TIoStatusOr<FIoBuffer> ReadOpLogUri(FStringBuilderBase& ChunkUri, uint64 Offset = 0, uint64 Size = ~0ull);
	bool Download(Zen::FZenHttpRequest& Request, FStringView Uri, TArray64<uint8>* Buffer, Zen::EContentType AcceptType);
	bool Post(Zen::FZenHttpRequest& Request, FStringView Uri, FCbObjectView Obj);
	bool Post(Zen::FZenHttpRequest& Request, FStringView Uri, FMemoryView Payload);
	void Rpc(FAnsiStringView Uri, FCbObject RpcRequestObject, FCbPackage& OutResponse);
	void RpcAsync(FAnsiStringView Uri, FCbObject RpcRequestObject, TUniqueFunction<void(const FCbPackage& Response)>&& ResponseCallback);
	bool Delete(Zen::FZenHttpRequest& Request, FStringView Uri);
	bool ShouldRetry(Zen::FZenHttpRequest& Request);
	bool TryRecover();
	void SetRequestUri(class IHttpRequest& Request, FAnsiStringView Path);

	template<bool bAsync, typename T>
	void ReadChunks_Internal(TConstArrayView<T> Requests, TUniqueFunction<void(const T& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback);

	UE_API void InitCommon();

	static UE_API const uint32 PoolEntryCount;
	struct FSaltGenerator
	{
		FSaltGenerator();
		inline int32_t Next()
		{
			const uint32_t A = ++GOpCounter;
			return static_cast<int32_t>((A ^ (SaltBase + (A << 6) + (A >> 2))) & 0x7fffffffu);
		}
	private:
		static std::atomic<uint32> GOpCounter;
		const uint32_t SaltBase;
	};

	UE::Zen::FScopeZenService ZenService;

	// Legacy request pool
	TUniquePtr<Zen::FZenHttpRequestPool> RequestPool;

	// Newer connection pool and request queue
	TUniquePtr<Zen::Internal::FZenHttpClientState> ClientState;

	FSaltGenerator SaltGen;
	FString ProjectPath;
	FString OplogPath;
	FAnsiString OplogRpcPath;
	FString OplogNewEntryPath;
	FString OplogPrepNewEntryPath;
	FString TempDirPath;
	const uint64 StandaloneThresholdBytes = 1 * 1024 * 1024;
	bool bAllowRead = false;
	bool bAllowEdit = false;
	bool bConnectionSucceeded = false;
};

}

#undef UE_API
