// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatformGroups("Desktop")]
public class IoStoreUtilities : ModuleRules
{
	public IoStoreUtilities (ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[] {
			"StorageServerClient",
			"TargetPlatform",
		});

		PrivateDependencyModuleNames.AddRange(new string[] {
			"AssetRegistry",
			"CookMetadata",
			"Core",
			"CoreUObject",
			"DerivedDataCache",
			"DeveloperToolSettings",
			"DevHttp",
			"Json",
			"PakFile",
			"Projects",
			"RenderCore",
			"RSA",
			"SandboxFile",
			"Sockets",
			"StudioTelemetry",
			"Zen",
		});

		PublicIncludePathModuleNames.AddRange(new string[] {
			"Zen",
			"CoreUObject",
		});
	}
}
