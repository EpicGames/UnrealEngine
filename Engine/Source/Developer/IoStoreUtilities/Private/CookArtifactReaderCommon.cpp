// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookArtifactReaderCommon.h"

#include "Algo/Accumulate.h"
#include "Async/ParallelFor.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/Archive.h"
#include "Templates/UniquePtr.h"

FArchive* FCookArtifactReaderCommon::CreateFileReader(const TCHAR* Filename)
{
	if (IFileHandle* File = OpenRead(Filename))
	{
		if (TUniquePtr<FArchive> Reader = MakeUnique<FArchiveFileReaderGeneric>(File, Filename, File->Size()))
		{
			return Reader.Release();
		}
	}
	return nullptr;
}

bool FCookArtifactReaderCommon::IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	class FRecurse : public IPlatformFile::FDirectoryVisitor
	{
	public:
		FDirectoryVisitor&  Visitor;
		TArray<FString>&    Directories;
		FRecurse(FDirectoryVisitor& InVisitor, TArray<FString>& InDirectories)
			: FDirectoryVisitor(InVisitor.DirectoryVisitorFlags)
			, Visitor(InVisitor)
			, Directories(InDirectories)
		{
		}
		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			bool bResult = Visitor.CallShouldVisitAndVisit(FilenameOrDirectory, bIsDirectory);
			if (bResult && bIsDirectory)
			{
				Directories.Emplace(FilenameOrDirectory);
			}
			return bResult;
		}
	};

	TArray<FString> DirectoriesToVisit;
	DirectoriesToVisit.Add(Directory);

	constexpr int32 MinBatchSize = 1;
	const EParallelForFlags ParallelForFlags = FTaskGraphInterface::IsRunning() && Visitor.IsThreadSafe()
		? EParallelForFlags::Unbalanced : EParallelForFlags::ForceSingleThread;
	std::atomic<bool> bResult{true};
	TArray<TArray<FString>> DirectoriesToVisitNext;
	while (bResult && DirectoriesToVisit.Num() > 0)
	{
		ParallelForWithTaskContext(TEXT("IterateDirectoryRecursively.PF"),
			DirectoriesToVisitNext,
			DirectoriesToVisit.Num(),
			MinBatchSize,
			[this, &Visitor, &DirectoriesToVisit, &bResult](TArray<FString>& Directories, int32 Index)
			{
				FRecurse Recurse(Visitor, Directories);
				if (bResult.load(std::memory_order_relaxed) && !IterateDirectory(*DirectoriesToVisit[Index], Recurse))
				{
					bResult.store(false, std::memory_order_relaxed);
				}
			},
			ParallelForFlags);
		DirectoriesToVisit.Reset(Algo::TransformAccumulate(DirectoriesToVisitNext, &TArray<FString>::Num, 0));
		for (TArray<FString>& Directories : DirectoriesToVisitNext)
		{
			DirectoriesToVisit.Append(MoveTemp(Directories));
		}
	}

	return bResult;
}

namespace CookArtifactReaderImpl
{
	class FFileMatch : public IPlatformFile::FDirectoryVisitor
	{
	public:
		TArray<FString>& Result;
		FRWLock ResultLock;
		FString WildCard;
		bool bFiles;
		bool bDirectories;
		bool bStoreFullPath;
		FFileMatch(TArray<FString>& InResult, const FString& InWildCard, bool bInFiles, bool bInDirectories, bool bInStoreFullPath = false)
			: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
			, Result(InResult)
			, WildCard(InWildCard)
			, bFiles(bInFiles)
			, bDirectories(bInDirectories)
			, bStoreFullPath(bInStoreFullPath)
		{
		}

		virtual bool ShouldVisitLeafPathname(FStringView LeafFilename) override
		{
			return FString(LeafFilename).MatchesWildcard(WildCard);
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if ((bIsDirectory && bDirectories) || (!bIsDirectory && bFiles))
			{
				FString Filename = FPaths::GetCleanFilename(FilenameOrDirectory);
				if (ensureMsgf(ShouldVisitLeafPathname(Filename),
					TEXT("PlatformFile.IterateDirectory needs to call ShouldVisitLeafFilename before calling Visit.")))
				{
					FString FullPath = bStoreFullPath ? FString(FilenameOrDirectory) : MoveTemp(Filename);
					FWriteScopeLock ScopeLock(ResultLock);
					Result.Add(MoveTemp(FullPath));
				}
			}
			return true;
		}
	};
}

void FCookArtifactReaderCommon::FindFiles( TArray<FString>& Result, const TCHAR* InFilename, bool Files, bool Directories )
{
	FString Filename( InFilename );
	FPaths::NormalizeFilename( Filename );
	const FString CleanFilename = FPaths::GetCleanFilename(Filename);
	const bool bFindAllFiles = CleanFilename == TEXT("*") || CleanFilename == TEXT("*.*");
	CookArtifactReaderImpl::FFileMatch FileMatch( Result, bFindAllFiles ? TEXT("*") : CleanFilename, Files, Directories );
	IterateDirectory( *FPaths::GetPath(Filename), FileMatch );
}

void FCookArtifactReaderCommon::FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension)
{
	if (!Directory)
	{
		return;
	}

	FString RootDir(Directory);
	FString ExtStr = (FileExtension != nullptr) ? FString(FileExtension) : "";

	// No Directory?
	if (RootDir.Len() < 1)
	{
		return;
	}

	FPaths::NormalizeDirectoryName(RootDir);

	// Don't modify the ExtStr if the user supplied the form "*.EXT" or "*" or "*.*" or "Name.*"
	if (!ExtStr.Contains(TEXT("*")))
	{
		if (ExtStr == "")
		{
			ExtStr = "*.*";
		}
		else
		{
			//Complete the supplied extension with * or *. to yield "*.EXT"
			ExtStr = (ExtStr.Left(1) == ".") ? "*" + ExtStr : "*." + ExtStr;
		}
	}

	// Create the full filter, which is "Directory/*.EXT".
	FString FinalPath = RootDir + "/" + ExtStr;
	FindFiles(FoundFiles, *FinalPath, true, false);
}
