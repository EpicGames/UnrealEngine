// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookMetadataFiles.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CookMetadata.h"
#include "CookedPackageStore.h"
#include "HAL/FileManager.h"
#include "IO/IoBuffer.h"
#include "IO/IoStore.h"
#include "Memory/MemoryView.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/LargeMemoryReader.h"
#include "ZenStoreHttpClient.h"

// Returns the hash of the development asset registry or 0 on failure.
static uint64 LoadAssetRegistry(FCookedPackageStore* InPackageStore, const FString& InAssetRegistryFileName, FIoChunkId InAssetRegistryChunkId, FAssetRegistryState& OutAssetRegistry)
{
	FAssetRegistryVersion::Type Version;
	FAssetRegistryLoadOptions Options(UE::AssetRegistry::ESerializationTarget::ForDevelopment);

	if (InPackageStore && InPackageStore->HasZenStoreClient() && InAssetRegistryChunkId.IsValid())
	{
		FIoBuffer Buffer;
		Buffer = InPackageStore->ReadChunk(InAssetRegistryChunkId).ConsumeValueOrDie();
		uint64 DevArHash = UE::Cook::FCookMetadataState::ComputeHashOfDevelopmentAssetRegistry(Buffer.GetView());

		FLargeMemoryReader MemoryReader(Buffer.GetData(), Buffer.GetSize());
		if (OutAssetRegistry.Load(MemoryReader, Options, &Version))
		{
			return DevArHash;
		}
	}
	else
	{
		TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InAssetRegistryFileName));
		if (FileReader)
		{
			TArray64<uint8> Data;
			Data.SetNumUninitialized(FileReader->TotalSize());
			FileReader->Serialize(Data.GetData(), Data.Num());
			check(!FileReader->IsError());

			uint64 DevArHash = UE::Cook::FCookMetadataState::ComputeHashOfDevelopmentAssetRegistry(MakeMemoryView(Data));

			FLargeMemoryReader MemoryReader(Data.GetData(), Data.Num());
			if (OutAssetRegistry.Load(MemoryReader, Options, &Version))
			{
				return DevArHash;
			}
		}
	}

	return 0;
}

ECookMetadataFiles FindAndLoadMetadataFiles(
	FCookedPackageStore* InPackageStore,
	const FString& InCookedDir, ECookMetadataFiles InRequiredFiles, 
	FAssetRegistryState& OutAssetRegistry, FString* OutAssetRegistryFileName /*optional, set on success*/,
	UE::Cook::FCookMetadataState* OutCookMetadata, FString* OutCookMetadataFileName /*optional, set on success or need*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LoadingAssetRegistry);

	// Look for the development registry. Should be in \\GameName\\Metadata\\DevelopmentAssetRegistry.bin, but we don't know what "GameName" is.
	TArray<FString> PossibleAssetRegistryFiles;
	IFileManager::Get().FindFilesRecursive(PossibleAssetRegistryFiles, *InCookedDir, GetDevelopmentAssetRegistryFilename(), true, false);

	if (PossibleAssetRegistryFiles.Num() > 1)
	{
		UE_LOGF(LogIoStore, Warning, "Found multiple possible development asset registries:");
		for (FString& Filename : PossibleAssetRegistryFiles)
		{
			UE_LOGF(LogIoStore, Warning, "    %ls", *Filename);
		}
	}

	FIoChunkId AssetRegistryChunkId = FIoChunkId::InvalidChunkId;
	if ((PossibleAssetRegistryFiles.Num() == 0) && InPackageStore)
	{
		if (UE::FZenStoreHttpClient* ZenStoreClient = InPackageStore->GetZenStoreClient())
		{
			TIoStatusOr<FCbObject> ProjectInfoStatus = ZenStoreClient->GetProjectInfo().Get();
			if (ProjectInfoStatus.IsOk())
			{
				FCbObject ProjectInfo = ProjectInfoStatus.ConsumeValueOrDie();
				FString ProjectFile(ProjectInfo["projectfile"].AsString());
				FString ProjectName = FPaths::GetBaseFilename(ProjectFile);
				if (!ProjectName.IsEmpty())
				{
					FString CandidateAssetRegistryFilename = FPaths::Combine(InCookedDir, ProjectName, "Metadata", GetDevelopmentAssetRegistryFilename());
					FPaths::NormalizeFilename(CandidateAssetRegistryFilename);
					AssetRegistryChunkId = InPackageStore->GetChunkIdFromFileName(CandidateAssetRegistryFilename);
					if (AssetRegistryChunkId.IsValid())
					{
						PossibleAssetRegistryFiles.Add(MoveTemp(CandidateAssetRegistryFilename));
					}
				}
			}
		}
	}

	if (PossibleAssetRegistryFiles.Num() == 0)
	{
		if (EnumHasAnyFlags(InRequiredFiles, ECookMetadataFiles::AssetRegistry))
		{
			UE_LOGF(LogIoStore, Error, "No development asset registry file found!");
		}
		else
		{
			UE_LOGF(LogIoStore, Display, "No development asset registry file found!");
		}
		return ECookMetadataFiles::None;
	}

	FPaths::NormalizeFilename(PossibleAssetRegistryFiles[0]);

	UE_LOGF(LogIoStore, Display, "Using input asset registry: %ls", *PossibleAssetRegistryFiles[0]);
	uint64 LoadedDevArHash = LoadAssetRegistry(InPackageStore, PossibleAssetRegistryFiles[0], AssetRegistryChunkId, OutAssetRegistry);

	if (LoadedDevArHash == 0)
	{
		return ECookMetadataFiles::None; // already logged
	}

	// If we found the asset registry, try and find the cook metadata that should be next to it.
	ECookMetadataFiles ResultFiles = ECookMetadataFiles::AssetRegistry;

	if (OutCookMetadata)
	{
		// The cook metadata file should be adjacent to the development asset registry.
		FString CookMetadataFileName = FPaths::GetPath(PossibleAssetRegistryFiles[0]) / UE::Cook::GetCookMetadataFilename();

		auto ValidateAndOutputCookMetadata = [InRequiredFiles, LoadedDevArHash, &CookMetadataFileName, &OutCookMetadata, &OutCookMetadataFileName, &ResultFiles]()
		{

			if (OutCookMetadata->GetAssociatedDevelopmentAssetRegistryHash() != LoadedDevArHash &&
				OutCookMetadata->GetAssociatedDevelopmentAssetRegistryHashPostWriteback() != LoadedDevArHash) // during testing we can repeat stage after cook so we might have already edited it.
			{
				if (EnumHasAnyFlags(InRequiredFiles, ECookMetadataFiles::CookMetadata))
				{
					UE_LOGF(LogIoStore, Error,
						"Cook metadata file mismatch: Hash of associated development asset registry does not match. [%ls] %llx vs %llx (%llx post writeback)",
						*CookMetadataFileName, LoadedDevArHash, OutCookMetadata->GetAssociatedDevelopmentAssetRegistryHash(), OutCookMetadata->GetAssociatedDevelopmentAssetRegistryHashPostWriteback());
					return false;
				}
				else
				{
					UE_LOGF(LogIoStore, Display,
						"Cook metadata file mismatch: Hash of associated development asset registry does not match. [%ls] %llx vs %llx (%llx post writeback)",
						*CookMetadataFileName, LoadedDevArHash, OutCookMetadata->GetAssociatedDevelopmentAssetRegistryHash(), OutCookMetadata->GetAssociatedDevelopmentAssetRegistryHashPostWriteback());
					OutCookMetadata->Reset();
				}
			}
			else
			{
				EnumAddFlags(ResultFiles, ECookMetadataFiles::CookMetadata);
				if (OutCookMetadataFileName)
				{
					*OutCookMetadataFileName = MoveTemp(CookMetadataFileName);
				}
			}
			return true;
		};

		if (InPackageStore && InPackageStore->HasZenStoreClient() && AssetRegistryChunkId.IsValid())
		{
			FIoChunkId CookMetadataChunkId = InPackageStore->GetChunkIdFromFileName(CookMetadataFileName);
			if (!CookMetadataChunkId.IsValid())
			{
				UE_LOGF(LogIoStore, Error, "Failed to find cook metadata file - chunk missing from package store. [%ls]", *CookMetadataFileName);
				if (EnumHasAnyFlags(InRequiredFiles, ECookMetadataFiles::CookMetadata))
				{
					return ECookMetadataFiles::None;
				}
			}

			FIoBuffer Buffer;
			Buffer = InPackageStore->ReadChunk(CookMetadataChunkId).ConsumeValueOrDie();

			FLargeMemoryReader MemoryReader(Buffer.GetData(), Buffer.GetSize());
			if (!OutCookMetadata->Serialize(MemoryReader))
			{
				UE_LOGF(LogIoStore, Error, "Failed to deserialize cook metadata from package store (%ls)", *CookMetadataFileName);
				if (EnumHasAnyFlags(InRequiredFiles, ECookMetadataFiles::CookMetadata))
				{
					return ECookMetadataFiles::None;
				}
			}
			else if (!ValidateAndOutputCookMetadata())
			{
				return ECookMetadataFiles::None;
			}
		}
		else if (IFileManager::Get().FileExists(*CookMetadataFileName))
		{
			if (!OutCookMetadata->ReadFromFile(CookMetadataFileName))
			{
				UE_LOGF(LogIoStore, Error, "Failed to deserialize cook metadata file - invalid data. [%ls]", *CookMetadataFileName);
				if (EnumHasAnyFlags(InRequiredFiles, ECookMetadataFiles::CookMetadata))
				{
					return ECookMetadataFiles::None;
				}
			}
			else if (!ValidateAndOutputCookMetadata())
			{
				return ECookMetadataFiles::None;
			}
		}
		else
		{
			if (EnumHasAnyFlags(InRequiredFiles, ECookMetadataFiles::CookMetadata))
			{
				UE_LOGF(LogIoStore, Error, "Failed to open and read cook metadata file %ls", *CookMetadataFileName);
				return ECookMetadataFiles::None;
			}

			UE_LOGF(LogIoStore, Display, "No cook metadata file found, checked %ls", *CookMetadataFileName);
			if (OutCookMetadataFileName)
			{
				*OutCookMetadataFileName = FString("");
			}
		}
	}


	if (OutAssetRegistryFileName)
	{
		*OutAssetRegistryFileName = MoveTemp(PossibleAssetRegistryFiles[0]);
	}
	return ResultFiles;
}