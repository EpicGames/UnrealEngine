// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"

class FAssetRegistryState;
class FCookedPackageStore;
class FString;
namespace UE::Cook { class FCookMetadataState; }

enum class ECookMetadataFiles
{
	None = 0,
	AssetRegistry = 1,
	CookMetadata = 2,
	All = 4
};
ENUM_CLASS_FLAGS(ECookMetadataFiles);

ECookMetadataFiles FindAndLoadMetadataFiles(
	FCookedPackageStore* InPackageStore,
	const FString& InCookedDir, ECookMetadataFiles InRequiredFiles, 
	FAssetRegistryState& OutAssetRegistry, FString* OutAssetRegistryFileName /*optional, set on success*/,
	UE::Cook::FCookMetadataState* OutCookMetadata, FString* OutCookMetadataFileName /*optional, set on success or need*/);
