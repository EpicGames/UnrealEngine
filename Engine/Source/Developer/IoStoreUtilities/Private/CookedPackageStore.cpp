// Copyright Epic Games, Inc. All Rights Reserved.

#include "CookedPackageStore.h"

#include "AssetRegistry/AssetRegistryState.h"
#include "CookMetadataFiles.h"
#include "HAL/FileManager.h"
#include "IO/IoStore.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "ZenStoreHttpClient.h"

#define COOKEDPACKAGESTORE_CPU_SCOPE(NAME) TRACE_CPUPROFILER_EVENT_SCOPE(CookedPackageStore##NAME);

FCookedPackageStore::FCookedPackageStore(FStringView InCookedDir)
	: CookedDir(InCookedDir)
{
}

FIoStatus FCookedPackageStore::LoadManifest(const TCHAR* ManifestFilename)
{
	COOKEDPACKAGESTORE_CPU_SCOPE(LoadCookedPackageStoreManifest);
	double StartTime = FPlatformTime::Seconds();

	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(ManifestFilename));
	if (!Ar)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}
	FCbObject ManifestObject = LoadCompactBinary(*Ar).AsObject();
	FCbObject OplogObject;
	if (FCbFieldView ZenServerField = ManifestObject["zenserver"])
	{
		FString ProjectId = FString(ZenServerField["projectid"].AsString());
		FString OplogId = FString(ZenServerField["oplogid"].AsString());

		UE::Zen::FServiceSettings ZenServiceSettings;
		if (ZenServiceSettings.ReadFromCompactBinary(ZenServerField["settings"]))
		{
			ZenStoreClient = MakeUnique<UE::FZenStoreHttpClient>(MoveTemp(ZenServiceSettings));
		}
		else
		{
			ZenStoreClient = MakeUnique<UE::FZenStoreHttpClient>();
		}
		ZenStoreClient->InitializeReadOnly(ProjectId, OplogId);

		COOKEDPACKAGESTORE_CPU_SCOPE(FetchOplog);
		TIoStatusOr<FCbObject> OplogStatus = ZenStoreClient->GetOplog().Get();
		if (!OplogStatus.IsOk())
		{
			return OplogStatus.Status();
		}
		OplogObject = OplogStatus.ConsumeValueOrDie();
		
	}
	else
	{
		OplogObject = ManifestObject["oplog"].AsObject();
	}
	UE_LOGF(LogIoStore, Display, "Fetched %lld oplog items from %ls in %.2lf seconds",
		OplogObject["entries"].AsArrayView().Num(),
		HasZenStoreClient() ? TEXT("Zen") : TEXT("Manifest"),
		FPlatformTime::Seconds() - StartTime);

	ParseOplog(OplogObject);
	LoadChunkHashes();

	return FIoStatus::Ok;
}

FIoStatus FCookedPackageStore::LoadProjectStore(const TCHAR* ProjectStoreFilename)
{
	COOKEDPACKAGESTORE_CPU_SCOPE(LoadCookedProjectStore);
	double StartTime = FPlatformTime::Seconds();

	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(ProjectStoreFilename));
	if (!Ar)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	TSharedPtr<FJsonObject> ProjectStoreObject;
	TSharedRef<TJsonReader<UTF8CHAR>> Reader = TJsonReaderFactory<UTF8CHAR>::Create(Ar.Get());
	if (FJsonSerializer::Deserialize(Reader, ProjectStoreObject) && ProjectStoreObject.IsValid())
	{
		const TSharedPtr<FJsonObject>* ZenServerObjectPtr = nullptr;
		if (ProjectStoreObject->TryGetObjectField(TEXT("zenserver"), ZenServerObjectPtr) && (ZenServerObjectPtr != nullptr))
		{
			FString ProjectId;
			FString OplogId;

			const TSharedPtr<FJsonObject>& ZenServerObject = *ZenServerObjectPtr;
			if (ZenServerObject->TryGetStringField(TEXT("projectid"), ProjectId) && !ProjectId.IsEmpty() &&
				ZenServerObject->TryGetStringField(TEXT("oplogid"), OplogId) && !OplogId.IsEmpty())
			{
				ZenStoreClient = MakeUnique<UE::FZenStoreHttpClient>();
				ZenStoreClient->InitializeReadOnly(ProjectId, OplogId);

				COOKEDPACKAGESTORE_CPU_SCOPE(FetchOplog);
				TIoStatusOr<FCbObject> OplogStatus = ZenStoreClient->GetOplog().Get();
				if (!OplogStatus.IsOk())
				{
					return OplogStatus.Status();
				}
				FCbObject OplogObject = OplogStatus.ConsumeValueOrDie();

				UE_LOGF(LogIoStore, Display, "Fetched %lld oplog items from Zen in %.2lf seconds",
					OplogObject["entries"].AsArrayView().Num(),
					FPlatformTime::Seconds() - StartTime);

				ParseOplog(OplogObject);
				LoadChunkHashes();

				return FIoStatus::Ok;
			}
		}
	}

	return FIoStatus(EIoErrorCode::NotFound);
}

void FCookedPackageStore::ParseOplog(FCbObject& OplogObject)
{
	COOKEDPACKAGESTORE_CPU_SCOPE(ParseOplog);
	double StartTime = FPlatformTime::Seconds();

	const FCbArrayView EntriesArray = OplogObject["entries"].AsArrayView();
	const int32 EstimatedChunksCount = 3 * EntriesArray.Num(); //PackageData+BulkData+OptionalBulkData
	ChunkInfoMap.Reserve(EstimatedChunksCount);
	FilenameToChunkIdMap.Reserve(EstimatedChunksCount);
	PackageIdToEntry.Reserve(EntriesArray.Num());
	for (FCbFieldView& OplogEntry : EntriesArray)
	{
		FCbObjectView OplogObj = OplogEntry.AsObjectView();
		FPackageStoreEntryResource PackageStoreEntry = FPackageStoreEntryResource::FromCbObject(OplogObj["packagestoreentry"].AsObjectView());

		auto AddChunksFromOplog = [this, &OplogEntry, &PackageStoreEntry](const char* Field)
		{
			for (FCbFieldView& ChunkEntry : OplogEntry[Field].AsArrayView())
			{
				FCbObjectView ChunkObj = ChunkEntry.AsObjectView();
				FIoChunkId ChunkId;
				ChunkId.Set(ChunkObj["id"].AsObjectId().GetView());
				if (ChunkId.IsValid())
				{
					FChunkInfo& ChunkInfo = ChunkInfoMap.Add(ChunkId);
					ChunkInfo.ChunkId = ChunkId;
					ChunkInfo.PackageName = PackageStoreEntry.PackageName;
					if (ChunkObj["filename"])
					{
						TStringBuilder<1024> RelativeFilename;
						RelativeFilename.Append(ChunkObj["filename"].AsString());
						ChunkInfo.RelativeFileName = RelativeFilename;
						TStringBuilder<1024> PathBuilder;
						FPathViews::AppendPath(PathBuilder, CookedDir);
						FPathViews::AppendPath(PathBuilder, RelativeFilename);
						FPathViews::NormalizeFilename(PathBuilder);
						FilenameToChunkIdMap.Add(*PathBuilder, ChunkId);
					}
					else if (ChunkObj["clientpath"])
					{
						TStringBuilder<1024> RelativeFilename;
						FUtf8StringView ClientPathView = ChunkObj["clientpath"].AsString();
						if (ClientPathView.StartsWith("/{project}/"))
						{
							if (!FPaths::IsProjectFilePathSet())
							{
								RelativeFilename.Append(ClientPathView);
								UE_LOGF(LogIoStore, Warning, "Project relative path could not be remapped because project file path is unset (possibly due to not specifying uproject path as first argument). %ls", RelativeFilename.ToString());
							}
							else
							{
								RelativeFilename.Append(FPathViews::GetBaseFilename(FPaths::GetProjectFilePath()));
								RelativeFilename.AppendChar('/');
								RelativeFilename.Append(ClientPathView.RightChop(11));
							}
						}
						else if (ClientPathView.StartsWith("/{engine}/"))
						{
							RelativeFilename.Append(TEXT("Engine/"));
							RelativeFilename.Append(ClientPathView.RightChop(10));
						}
						else
						{
							RelativeFilename.Append(ClientPathView);
						}
						ChunkInfo.RelativeFileName = RelativeFilename;
						TStringBuilder<1024> PathBuilder;
						FPathViews::AppendPath(PathBuilder, CookedDir);
						FPathViews::AppendPath(PathBuilder, RelativeFilename);
						FPathViews::NormalizeFilename(PathBuilder);
						FilenameToChunkIdMap.Add(*PathBuilder, ChunkId);
					}

					const FCbArrayView RegionsArray = ChunkObj["fileregions"].AsArrayView();
					ChunkInfo.FileRegions.Reserve(RegionsArray.Num());
					for (FCbFieldView RegionObj : RegionsArray)
					{
						FFileRegion& Region = ChunkInfo.FileRegions.AddDefaulted_GetRef();
						FFileRegion::LoadFromCompactBinary(RegionObj, Region);
					}
				}
			}
		};

		AddChunksFromOplog("packagedata");
		AddChunksFromOplog("bulkdata");
		AddChunksFromOplog("files");

		PackageIdToEntry.Add(PackageStoreEntry.GetPackageId(), MoveTemp(PackageStoreEntry));
	}

	UE_LOGF(LogIoStore, Display, "Parsed %lld oplog items %.2lf seconds, %d chunks",
		EntriesArray.Num(),
		FPlatformTime::Seconds() - StartTime,
		ChunkInfoMap.Num());
}

FIoStatus FCookedPackageStore::LoadChunkHashes()
{
	COOKEDPACKAGESTORE_CPU_SCOPE(LoadChunkHashes);
	double StartLoadTime = FPlatformTime::Seconds();
	double StartUpdateTime = StartLoadTime;

	uint32 LoadedHashCount = 0;
	uint32 UpdatedHashCount = 0;

	if (HasZenStoreClient())
	{
		FCbObject ChunksObj;
		{
			COOKEDPACKAGESTORE_CPU_SCOPE(GetChunkInfos);
			TIoStatusOr<FCbObject> Chunks = ZenStoreClient->GetChunkInfos().Get();
			if (!Chunks.IsOk())
			{
				return Chunks.Status();
			}
			ChunksObj = Chunks.ConsumeValueOrDie();
		}

		StartUpdateTime = FPlatformTime::Seconds();

		COOKEDPACKAGESTORE_CPU_SCOPE(ParseChunkInfos);
		for (FCbField& ChunkEntry : ChunksObj["chunkinfos"])
		{
			FCbObject ChunkObj = ChunkEntry.AsObject();
			FIoChunkId ChunkId;
			if (!LoadFromCompactBinary(ChunkObj["id"], ChunkId))
			{
				UE_LOGF(LogIoStore, Warning, "Received invalid chunk id, skipping.");
				continue;
			}
			if (FChunkInfo* FindChunkInfo = ChunkInfoMap.Find(ChunkId); FindChunkInfo != nullptr)
			{
				FindChunkInfo->ChunkHash = ChunkObj["rawhash"].AsHash();
				FindChunkInfo->ChunkSize = ChunkObj["rawsize"].AsUInt64();
				++UpdatedHashCount;
			}
			++LoadedHashCount;
		}
	}
	else
	{
		FAssetRegistryState AssetRegistry;
		if (FindAndLoadMetadataFiles(this, CookedDir, ECookMetadataFiles::None, AssetRegistry, nullptr, nullptr, nullptr) == ECookMetadataFiles::None)
		{
			return FIoStatus(EIoErrorCode::NotFound);
		}

		StartUpdateTime = FPlatformTime::Seconds();

		const TMap<FName, const FAssetPackageData*>& Packages = AssetRegistry.GetAssetPackageDataMap();
		for (auto PackageIter : Packages)
		{
			for (const TPair<FIoChunkId, FIoHash>& HashIter : PackageIter.Value->ChunkHashes)
			{
				// For the moment, only bulk data types are added to teh asset registry - gate here so that
				// we remember to verify all the hashes match when they eventually get added during cook.
				if (HashIter.Key.GetChunkType() == EIoChunkType::BulkData ||
					HashIter.Key.GetChunkType() == EIoChunkType::OptionalBulkData)
				{
					if (FChunkInfo* FindChunkInfo = ChunkInfoMap.Find(HashIter.Key); FindChunkInfo != nullptr)
					{
						FindChunkInfo->ChunkHash = HashIter.Value;
						++UpdatedHashCount;
					}
					++LoadedHashCount;
				}
			}
		}
	}

	UE_LOGF(LogIoStore, Display, "Loaded %u chunk hashes from %ls in %.2lf seconds, %d hashes updated in %.2lf seconds",
		LoadedHashCount,
		HasZenStoreClient() ? TEXT("Zen") : TEXT("AssetRegistry"),
		StartUpdateTime - StartLoadTime,
		UpdatedHashCount,
		FPlatformTime::Seconds() - StartUpdateTime);

	return FIoStatus::Ok;
}

FIoChunkId FCookedPackageStore::GetChunkIdFromFileName(const FString& Filename) const
{
	return FilenameToChunkIdMap.FindRef(Filename);
}

const FCookedPackageStore::FChunkInfo* FCookedPackageStore::GetChunkInfoFromChunkId(const FIoChunkId& ChunkId) const
{
	return ChunkInfoMap.Find(ChunkId);
}

const FCookedPackageStore::FChunkInfo* FCookedPackageStore::GetChunkInfoFromFileName(const FString& Filename) const
{
	FIoChunkId ChunkId = GetChunkIdFromFileName(Filename);
	return ChunkInfoMap.Find(ChunkId);
}

FString FCookedPackageStore::GetRelativeFilenameFromChunkId(const FIoChunkId& ChunkId) const
{
	const FChunkInfo* FindChunkInfo = ChunkInfoMap.Find(ChunkId);
	if (!FindChunkInfo)
	{
		return FString();
	}
	return FindChunkInfo->RelativeFileName;
}

FName FCookedPackageStore::GetPackageNameFromChunkId(const FIoChunkId& ChunkId) const
{
	const FChunkInfo* FindChunkInfo = ChunkInfoMap.Find(ChunkId);
	if (!FindChunkInfo)
	{
		return NAME_None;
	}
	return FindChunkInfo->PackageName;
}

FName FCookedPackageStore::GetPackageNameFromFileName(const FString& Filename) const
{
	FIoChunkId ChunkId = GetChunkIdFromFileName(Filename);
	return GetPackageNameFromChunkId(ChunkId);
}

const FPackageStoreEntryResource* FCookedPackageStore::GetPackageStoreEntry(FPackageId PackageId) const
{
	return PackageIdToEntry.Find(PackageId);
}

bool FCookedPackageStore::HasZenStoreClient() const
{
	return ZenStoreClient.IsValid();
}

UE::FZenStoreHttpClient* FCookedPackageStore::GetZenStoreClient()
{
	return ZenStoreClient.Get();
}

TIoStatusOr<FIoBuffer> FCookedPackageStore::ReadChunk(const FIoChunkId& ChunkId)
{
	FIoReadOptions ReadOptions;
	return ZenStoreClient->ReadChunk(ChunkId, ReadOptions.GetOffset(), ReadOptions.GetSize());
}

UE::Tasks::FTask FCookedPackageStore::ReadChunkAsync(const FIoChunkId& ChunkId, TFunction<void(TIoStatusOr<FIoBuffer>)>&& Callback)
{
	return UE::Tasks::Launch(TEXT("ReadChunkAsync"), [this, ChunkId, Callback = MoveTemp(Callback)]()
	{
		FIoReadOptions ReadOptions;
		Callback(ZenStoreClient->ReadChunk(ChunkId, ReadOptions.GetOffset(), ReadOptions.GetSize()));
	}, UE::Tasks::ETaskPriority::Normal);
}
