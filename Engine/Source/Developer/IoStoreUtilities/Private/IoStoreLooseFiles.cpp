// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreLooseFiles.h"
#include "IoStoreWriter.h"

#include "HAL/FileManager.h"
#include "IO/IoStore.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Tasks/Task.h"
#include "Tasks/Pipe.h"

namespace
{

using FJsonWriter = TSharedPtr<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>>; 
using FJsonWriterFactory = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>;

class FLooseFilesIoStoreWriter final
	: public IIoStoreWriter
{
	struct FWriteStats
	{
		std::atomic<uint32> PendingCount{0};
		std::atomic<uint32> TotalCount{0};
		std::atomic<uint64> TotalByteCount{0};
	};

	struct FPendingWrite
	{
		FPendingWrite(const FIoChunkId& Id, IIoStoreWriteRequest* Request, const FIoWriteOptions& Options)
			: ChunkId(Id)
			, WriteRequest(Request)
			, WriteOptions(Options)
			, TaskPipe(UE_SOURCE_LOCATION)
		{ }
		
		FIoChunkId ChunkId;
		TUniquePtr<IIoStoreWriteRequest> WriteRequest;
		FIoWriteOptions WriteOptions;
		UE::Tasks::FPipe TaskPipe;
		FString ErrorText;
		uint64 UncompressedSize{0};
		uint64 CompressedSize{0};
	};

	const uint32 MaxConcurrentWrites;

public:
	FLooseFilesIoStoreWriter(const FLooseFilesWriterSettings& WriterSettings,
		uint32 MaxConcurrentWrites)
		: MaxConcurrentWrites(MaxConcurrentWrites)
		, Settings(WriterSettings)
	{
		Settings.TocFilePath = FPaths::SetExtension(Settings.TocFilePath, TEXT("utocmanifest"));
	}

	virtual ~FLooseFilesIoStoreWriter()
	{ }

	virtual void SetReferenceChunkDatabase(TSharedPtr<IIoStoreWriterReferenceChunkDatabase> ReferenceChunkDatabase, TSharedPtr<IIoStoreWriterReferenceChunkDatabase> PatchInPlaceReferenceChunkDatabase) override
	{ }

	virtual void EnableDiskLayoutOrdering(const TArray<TUniquePtr<FIoStoreReader>>& PatchSourceReaders = TArray<TUniquePtr<FIoStoreReader>>())
	{ }

	virtual void Append(const FIoChunkId& ChunkId, FIoBuffer Chunk, const FIoWriteOptions& WriteOptions, uint64 OrderHint = MAX_uint64) override
	{
		struct FWriteRequest
			: IIoStoreWriteRequest
		{
			FWriteRequest(FIoBuffer InSourceBuffer, uint64 InOrderHint)
				: OrderHint(InOrderHint)
			{
				SourceBuffer = InSourceBuffer;
				SourceBuffer.MakeOwned();
			}

			virtual ~FWriteRequest() = default;

			void PrepareSourceBufferAsync(UE::Tasks::FTaskEvent& CompletionEvent) override
			{
				CompletionEvent.Trigger();
			}

			const FIoBuffer* GetSourceBuffer() override
			{
				return &SourceBuffer;
			}

			void FreeSourceBuffer() override
			{
			}

			uint64 GetOrderHint() override
			{
				return OrderHint;
			}

			TArrayView<const FFileRegion> GetRegions()
			{
				return TArrayView<const FFileRegion>();
			}

			virtual const FIoHash* GetChunkHash() override
			{
				return nullptr;
			}

			virtual uint64 GetSourceBufferSizeEstimate() override
			{
				return SourceBuffer.DataSize();
			}

			virtual const TCHAR* DebugNameOfRepository() const override
			{
				return TEXT("FLooseFilesIoStoreWriter::Append");
			}

			FIoBuffer SourceBuffer;
			uint64 OrderHint;
		};

		Append(ChunkId, new FWriteRequest(Chunk, OrderHint), WriteOptions);
	}

	virtual void Append(const FIoChunkId& ChunkId, IIoStoreWriteRequest* Request, const FIoWriteOptions& WriteOptions) override
	{
		check(Request);

		for(;;)
		{
			{
				FScopeLock _(&PendingLock);
				if (WriteStats.PendingCount.load() < MaxConcurrentWrites)
				{
					WriteStats.PendingCount++;
					FPendingWrite* PendingWrite = PendingWrites.Add_GetRef(MakeUnique<FPendingWrite>(ChunkId, Request, WriteOptions)).Get();
					
					// Setup the task pipe when holding the lock to make it easy to flush all pending task pipe(s)

					UE::Tasks::FTaskEvent Event { UE_SOURCE_LOCATION };
					PendingWrite->WriteRequest->PrepareSourceBufferAsync(Event);
					UE::Tasks::FTask ReadChunkTask = PendingWrite->TaskPipe.Launch(TEXT("ReadChunk"), [PendingWrite, Event]() mutable
					{
						Event.Wait();
					});

					UE::Tasks::FTask WriteChunkTask = PendingWrite->TaskPipe.Launch(TEXT("WriteChunk"), [this, PendingWrite]() mutable
					{
						if (const FIoBuffer* SourceBuffer = PendingWrite->WriteRequest->GetSourceBuffer())
						{
							FString FileName = PendingWrite->WriteOptions.FileName;
							FString FilePath = FPaths::ConvertRelativePathToFull(Settings.TargetRootPath, FileName);
							FString FileDirectory = FPaths::GetPath(FilePath);
							if (IFileManager::Get().MakeDirectory(*FileDirectory, true))
							{
								if (TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*FilePath)); Ar.IsValid())
								{
									const uint64 ChunkSize = SourceBuffer->GetSize();
									const uint32 CurrentCount = WriteStats.TotalCount.fetch_add(1) + 1;
									UE_CLOGF((CurrentCount % 128 == 0), LogIoStore, Display,
										"Writing loose file chunk #%u '%ls' -> '%ls' (%llu Bytes)",
										CurrentCount, *PendingWrite->WriteOptions.FileName, *FileName, ChunkSize);

									Ar->Serialize((void*)SourceBuffer->GetData(), ChunkSize);
									PendingWrite->UncompressedSize = PendingWrite->CompressedSize = ChunkSize;
									PendingWrite->WriteRequest->FreeSourceBuffer();
									WriteStats.TotalByteCount += ChunkSize;
								}
								else
								{
									PendingWrite->ErrorText = FString::Printf(TEXT("Failed to create directory '%s'"), *FileDirectory);
								}
							}
							else
							{
								PendingWrite->ErrorText = FString::Printf(TEXT("Failed to create parent folder for file '%s'"), *FileName);
							}
						}
						else
						{
							PendingWrite->ErrorText = TEXT("Invalid source buffer");
						}

						PendingWrite->WriteRequest.Reset();

						WriteStats.PendingCount--;
						WriteCompletedEvent->Trigger();
					}, ReadChunkTask);
					
					break;
				}
			}

			WriteCompletedEvent->Wait();
		}
	}

	virtual TIoStatusOr<FIoStoreWriterResult> GetResult() override
	{
		FScopeLock _(&PendingLock);

		for (TUniquePtr<FPendingWrite>& PendingWrite : PendingWrites)
		{
			PendingWrite->TaskPipe.WaitUntilEmpty();
		}
		
		FIoStoreWriterResult WriteResult;
		WriteResult.ContainerId = FIoContainerId::FromName(Settings.ContainerName);
		WriteResult.ContainerName = Settings.ContainerName.ToString();
		
		FString Json;
		FJsonWriter JsonWriter = FJsonWriterFactory::Create(&Json);
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("ContainerName"), Settings.ContainerName.ToString());
		JsonWriter->WriteArrayStart(TEXT("Chunks"));
		
		for (TUniquePtr<FPendingWrite>& PendingWrite : PendingWrites)
		{
			if (PendingWrite->ErrorText.IsEmpty() == false)
			{
				UE_LOGF(LogIoStore, Display, "Failed to write on demand chunk '%ls', reason '%ls'", *LexToString(PendingWrite->ChunkId), *PendingWrite->ErrorText);
				return FIoStatus(EIoErrorCode::WriteError, PendingWrite->ErrorText);
			}
			JsonWriter->WriteObjectStart();
			JsonWriter->WriteValue(TEXT("IoChunkId"), LexToString(PendingWrite->ChunkId));
			JsonWriter->WriteValue(TEXT("FileName"), *PendingWrite->WriteOptions.FileName);
			JsonWriter->WriteObjectEnd();
			
			WriteResult.UncompressedContainerSize += PendingWrite->UncompressedSize;
			WriteResult.CompressedContainerSize += PendingWrite->CompressedSize;
			WriteResult.TocEntryCount++;
		}

		JsonWriter->WriteArrayEnd();
		JsonWriter->WriteObjectEnd();
		JsonWriter->Close();

		UE_LOGF(LogIoStore, Display, "Saving loose file JSON manifest '%ls'", *Settings.TocFilePath);
		if (!FFileHelper::SaveStringToFile(Json, *Settings.TocFilePath))
		{
			return FIoStatus(EIoErrorCode::CorruptToc, TEXTVIEW("Failed writing loose file JSON manifest")); 
		}
		
		WriteResult.TocSize = sizeof(TCHAR) + Json.Len();

		return WriteResult;
	}

	virtual void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const override
	{
	}

private:
	FLooseFilesWriterSettings Settings;
	FWriteStats WriteStats;
	FCriticalSection PendingLock;
	TArray<TUniquePtr<FPendingWrite>> PendingWrites;
	FEventRef WriteCompletedEvent;
};

} // namespace

TSharedPtr<IIoStoreWriter> MakeLooseFilesIoStoreWriter(const FLooseFilesWriterSettings& WriterSettings,
	uint32 MaxConcurrentWrites)
{
	return MakeShared<FLooseFilesIoStoreWriter>(WriterSettings, MaxConcurrentWrites);
}
