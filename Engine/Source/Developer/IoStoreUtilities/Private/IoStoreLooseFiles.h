// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class IIoStoreWriter;

struct FLooseFilesWriterSettings
{
	FString TocFilePath;
	FString TargetRootPath;
	FName ContainerName;
};

TSharedPtr<IIoStoreWriter> MakeLooseFilesIoStoreWriter(const FLooseFilesWriterSettings& WriterSettings,
	uint32 MaxConcurrentWrites = 64);
