// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreShaderLibraryProcessor.h"

#include "Async/ParallelFor.h"
#include "IO/IoStore.h"
#include "IoStoreUtilitiesCore.h"
#include "Misc/FileHelper.h"
#include "Misc/ChunkDependencyInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "ShaderCodeArchive.h"
#include "ShaderLibrary/EditorShaderCodeArchive.h"
#include "ShaderLibrary/ShaderCodeArchiveInternal.h"

#define IOSTORE_CPU_SCOPE(NAME) TRACE_CPUPROFILER_EVENT_SCOPE(IoStore##NAME);
#define IOSTORE_CPU_SCOPE_DATA(NAME, DATA) TRACE_CPUPROFILER_EVENT_SCOPE(IoStore##NAME);

namespace UE::IoStore::Private
{

static TAutoConsoleVariable<bool> CVarShaderLibraryPakChunkOverride(
	TEXT("r.ShaderCodeLibrary.PakChunkOverride"),
	false,
	TEXT("If true, shader library chunks are re-assigned with their asset-associations during packaging. Disabled by default.")
);

static TAutoConsoleVariable<bool> CVarShaderLibraryPakChunkOverrideDumpDebugInfo(
	TEXT("r.ShaderCodeLibrary.PakChunkOverride.DumpDebugInfo"),
	true, // @lh-todo - Keep this debug information on initially until the chunk override has been tested enough
	TEXT("If true, dumps a debug file for each pakchunk with information what shaders have been moved to a different pakchunk.\n")
	TEXT("Also requires `r.ShaderCodeLibrary.PakChunkOverride=true` or command line argument `-OverrideShaderChunks`.")
);

static const TCHAR* GProjectPackagingSettingConfigSection = TEXT("/Script/UnrealEd.ProjectPackagingSettings");

// Returns true if the project being packaged is meant to generate chunks for the specified platform.
// Without chunks being generated, there is nothing to re-assign shader chunks to.
// This might even fail if attempting to re-assign shaders to a single large (> 4 GB) consecutive buffer.
static bool IsGenerateChunksEnabled(const FString& ProjectDir)
{
	FString PlatformIniName;
	if (!ProjectDir.IsEmpty() && FParse::Value(FCommandLine::Get(), TEXT("platform="), PlatformIniName, false))
	{
		FConfigFile PlatformIniFile;
		FConfigCacheIni::LoadExternalIniFile(PlatformIniFile, TEXT("Game"),
			*FPaths::EngineConfigDir(),
			*FPaths::Combine(ProjectDir, TEXT("Config/")),
			true, *PlatformIniName);
		bool bGenerateChunks = false;
		if (PlatformIniFile.GetBool(GProjectPackagingSettingConfigSection, TEXT("bGenerateChunks"), bGenerateChunks))
		{
			return bGenerateChunks;
		}
	}
	return false;
}

// Returns the configured container name to be used for on-demand shader library TOCs.
static FString GetPrimaryContainerForOnDemandShaderLibTOC()
{
	FString PrimaryContainerName;
	GConfig->GetString(TEXT("ShaderCodeLibrary.PakChunkOverride"), TEXT("PrimaryContainerForOnDemandShaderLibTOC"), PrimaryContainerName, GEngineIni);
	return PrimaryContainerName;
}

// Returns true if shader code chunk override is enabled, via CVar `r.ShaderCodeLibrary.PakChunkOverride` or commandline `-OverrideShaderChunks`.
// This also requires that the project being packaged is configured to generate chunks, i.e. `[/Script/UnrealEd.ProjectPackagingSettings]:bGenerateChunks=True`.
static bool IsChunkOverrideEnabled(const FString& ProjectDir)
{
	if (IsGenerateChunksEnabled(ProjectDir))
	{
		const bool bIsChunkOverrideEnabledByCmdLine = FParse::Param(FCommandLine::Get(), TEXT("OverrideShaderChunks"));
		return bIsChunkOverrideEnabledByCmdLine || CVarShaderLibraryPakChunkOverride.GetValueOnAnyThread();
	}
	return false;
}

static bool IsChunkOverrideDebugEnabled()
{
	return CVarShaderLibraryPakChunkOverrideDumpDebugInfo.GetValueOnAnyThread();
}

// Returns whether shader group compression is enabled for the specified container target chunk.
// Defaults to true, but some platforms only allow it for selected chunks.
static bool ShouldCompressShaderGroups(FStringView ContainerTargetChunkName, const FString& InProjectDir)
{
	struct FShaderGroupCompressionConfig
	{
		FShaderGroupCompressionConfig(const FString& ProjectDir)
		{
			FString PlatformIniName;
			if (!ProjectDir.IsEmpty() && FParse::Value(FCommandLine::Get(), TEXT("platform="), PlatformIniName, false))
			{
				FConfigFile PlatformIniFile;
				FConfigCacheIni::LoadExternalIniFile(PlatformIniFile, TEXT("Game"),
					*FPaths::EngineConfigDir(),
					*FPaths::Combine(ProjectDir, TEXT("Config/")),
					true, *PlatformIniName);
				if (PlatformIniFile.GetBool(GProjectPackagingSettingConfigSection, TEXT("bFilterShaderGroupCompression"), bIsFiltered) && bIsFiltered)
				{
					PlatformIniFile.GetArray(GProjectPackagingSettingConfigSection, TEXT("FilteredShaderGroupCompressionContainers"), FilteredContainerNames);
				}
			}
		}

		bool bIsFiltered = false;
		TArray<FString> FilteredContainerNames;
	};

	// Compress shader groups if filtering is disabled or if the container chunk name is included in the list of filtered containers
	static FShaderGroupCompressionConfig ShaderGroupCompressionConfig(InProjectDir);
	return !ShaderGroupCompressionConfig.bIsFiltered || ShaderGroupCompressionConfig.FilteredContainerNames.Contains(ContainerTargetChunkName);
}

// Loads an *.assetinfo.json text file either from local file system or Zen store.
static bool LoadShaderAssetInfo(const FString& Filename, FCookedPackageStore* PackageStore, FShaderMapAssetAssociations& OutShaderCodeToAssets)
{
	FString JsonText;
	if (PackageStore && PackageStore->HasZenStoreClient())
	{
		FIoChunkId ChunkId = PackageStore->GetChunkIdFromFileName(Filename);
		if (!ChunkId.IsValid())
		{
			return false;
		}
		TIoStatusOr<FIoBuffer> Buffer = PackageStore->ReadChunk(ChunkId);
		if (!Buffer.IsOk())
		{
			return false;
		}
		FFileHelper::BufferToString(JsonText, Buffer.ValueOrDie().GetData(), (int32)Buffer.ValueOrDie().GetSize());
	}
	else
	{
		if (!FFileHelper::LoadFileToString(JsonText, *Filename))
		{
			return false;
		}
	}

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<TCHAR>> Reader = TJsonReaderFactory<TCHAR>::Create(JsonText);

	if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
	{
		return false;
	}

	TSharedPtr<FJsonValue> AssetInfoArrayValue = JsonObject->GetFieldUntyped(FStringView(TEXT("ShaderCodeToAssets")));
	if (!AssetInfoArrayValue.IsValid())
	{
		return false;
	}

	TArray<TSharedPtr<FJsonValue>> AssetInfoArray = AssetInfoArrayValue->AsArray();
	for (int32 IdxPair = 0, NumPairs = AssetInfoArray.Num(); IdxPair < NumPairs; ++IdxPair)
	{
		TSharedPtr<FJsonObject> Pair = AssetInfoArray[IdxPair]->AsObject();
		if (Pair.IsValid())
		{
			TSharedPtr<FJsonValue> ShaderMapHashJson = Pair->GetFieldUntyped(FStringView(TEXT("ShaderMapHash")));
			if (!ShaderMapHashJson.IsValid())
			{
				continue;
			}
			FShaderHash ShaderMapHash;
			ShaderMapHash.FromString(ShaderMapHashJson->AsString());

			TSharedPtr<FJsonValue> AssetPathsArrayValue = Pair->GetFieldUntyped(FStringView(TEXT("Assets")));
			if (!AssetPathsArrayValue.IsValid())
			{
				continue;
			}

			TArray<TSharedPtr<FJsonValue>> AssetPathsArray = AssetPathsArrayValue->AsArray();
			for (int32 IdxAsset = 0, NumAssets = AssetPathsArray.Num(); IdxAsset < NumAssets; ++IdxAsset)
			{
				const FName AssetName = FName(*AssetPathsArray[IdxAsset]->AsString());
				OutShaderCodeToAssets.FindOrAddAsset(AssetName, ShaderMapHash);
			}
		}
	}
	return true;
}

#if WITH_EDITORONLY_DATA

// Loads a *.stinfo binary file either from local file system or Zen store.
static bool LoadShaderTypeInfo(const FString& Filename, FCookedPackageStore* PackageStore, TArray<FSerializedShaderArchive::FShaderTypeHashes>& OutShaderTypes)
{
	TArray<uint8> FileData;
	if (PackageStore && PackageStore->HasZenStoreClient())
	{
		FIoChunkId ChunkId = PackageStore->GetChunkIdFromFileName(Filename);
		if (!ChunkId.IsValid())
		{
			return false;
		}
		TIoStatusOr<FIoBuffer> Buffer = PackageStore->ReadChunk(ChunkId);
		if (!Buffer.IsOk())
		{
			return false;
		}
		FileData.Append(Buffer.ValueOrDie().GetData(), (int32)Buffer.ValueOrDie().GetSize());
	}
	else
	{
		if (!FFileHelper::LoadFileToArray(FileData, *Filename))
		{
			return false;
		}
	}

	FMemoryReaderView MemoryReader(FMemoryView(FileData.GetData(), FileData.NumBytes()));
	MemoryReader << OutShaderTypes;

	return true;
}

#endif // WITH_EDITORONLY_DATA

// Stores the shader code of all serialized shaders as a single flat buffer.
// This is the final format of serialized shaders when they are written to or read from a *.ushaderbytecode file.
class FSerializedShaderArchiveBuffer : public FSerializedShaderArchive
{
	// Pointer to the original target file. This can be re-assigned when the IoStore shader library is generated during packaging.
	const FContainerTargetFile* TargetFile = nullptr;
	FIoBuffer LibraryBuffer;
	int64 OffsetToShaderCode = 0;

public:
	bool LoadFromIoBuffer(FIoBuffer&& InBuffer)
	{
		LibraryBuffer = MoveTemp(InBuffer);

		// Serialize shader archive header and store offset to start of shader code payload
		FLargeMemoryReader LibraryAr(LibraryBuffer.GetData(), LibraryBuffer.GetSize());
		if (!FSerializedShaderArchive::SerializeHeaderVersion(LibraryAr))
		{
			UE_LOGF(LogIoStore, Error, "Failed to serialize header version from ShaderCodeArchive");
			return false;
		}
		LibraryAr << *this;
		OffsetToShaderCode = LibraryAr.Tell();

		return true;
	}

	// Loads the shader library buffer either from package store or from local file.
	bool LoadFromTargetFile(const TCHAR* InFilename, const FContainerTargetFile* InTargetFile, FCookedPackageStore* PackageStore)
	{
		check(InTargetFile);
		TargetFile = InTargetFile;

		if (PackageStore && PackageStore->HasZenStoreClient() && InTargetFile->ChunkId.IsValid())
		{
			// Load serialized shaders from IoChunk with Id that was generated from an incremental cook.
			// We don't have to deal with generating this ChunkId here, it's just an opaque token that was assigned to the target file.
			TIoStatusOr<FIoBuffer> ReadChunkResult = PackageStore->ReadChunk(InTargetFile->ChunkId);
			if (!ReadChunkResult.IsOk())
			{
				UE_LOGF(LogIoStore, Error,
					"Missing shader code libary ChunkId. ChunkId = '%ls'. InFilename = '%ls'. IOStatus = '%ls'.",
					*LexToString(InTargetFile->ChunkId), InFilename ? InFilename : TEXT("<nullptr>"),
					*ReadChunkResult.Status().ToString());
				return false;
			}
			return LoadFromIoBuffer(ReadChunkResult.ConsumeValueOrDie());
		}
		else
		{
			// Load serialized shaders from local file.
			TUniquePtr<FArchive> LibraryAr(IFileManager::Get().CreateFileReader(InFilename));
			if (!LibraryAr)
			{
				UE_LOGF(LogIoStore, Error, "Missing shader code library file '%ls'.", InFilename);
				return false;
			}

			FIoBuffer IntermediateLibraryBuffer(LibraryAr->TotalSize());
			LibraryAr->Serialize(IntermediateLibraryBuffer.GetData(), IntermediateLibraryBuffer.GetSize());
			return LoadFromIoBuffer(MoveTemp(IntermediateLibraryBuffer));
		}
	}

#if WITH_EDITORONLY_DATA
	bool LoadFromEditorArchive(FEditorShaderCodeArchive& ShaderCodeArchive)
	{
		// Copy editor archive to this archive
		TArray<uint8> FlatShaderCode;
		bool bRanOutOfRoom = false;
		constexpr int64 kShaderSizeLimit = MAX_int32; // Limited by max capacity of TArray<>::Num()
		constexpr int64 kNoShaderCountLimit = -1;
		ShaderCodeArchive.CopyToArchiveAndClear(*this, FlatShaderCode, bRanOutOfRoom, kShaderSizeLimit, kNoShaderCountLimit, ECookShaderLibrarySource::CurrentCook);
		if (bRanOutOfRoom)
		{
			return false;
		}

		// Write header
		FLargeMemoryWriter HeaderWriter;
		const bool bSerializeVersionSucceeded = FSerializedShaderArchive::SerializeHeaderVersion(HeaderWriter);
		check(bSerializeVersionSucceeded);
		HeaderWriter << *this;
		
		// Convert to flat IoBuffer
		LibraryBuffer = FIoBuffer(HeaderWriter.TotalSize() + FlatShaderCode.NumBytes());
		OffsetToShaderCode = HeaderWriter.TotalSize();

		FMemory::Memcpy(LibraryBuffer.GetData(), HeaderWriter.GetData(), HeaderWriter.TotalSize());
		FMemory::Memcpy(LibraryBuffer.GetData() + OffsetToShaderCode, FlatShaderCode.GetData(), FlatShaderCode.NumBytes());

		return true;
	}
#endif

	// Loads and decompresses the specified shader and appends it to the end of the output container.
	void DecompressShaderFromLibraryBuffer(int32 ShaderIndex, TArray<uint8>& OutDecompressedShader) const
	{
		const FShaderCodeEntry& Entry = ShaderEntries[ShaderIndex];
		FSerializedShaderArchive::DecompressShaderEntryAndAppend(Entry, GetShaderCodeDataStart() + Entry.Offset, OutDecompressedShader);
	}

	// Returns a pointer to the start of shader code payload to read from.
	const uint8* GetShaderCodeDataStart() const
	{
		return LibraryBuffer.GetData() + OffsetToShaderCode;
	}

	// Returns an array view of the shader code payload.
	TConstArrayView64<uint8> GetShaderCodeView() const
	{
		return TConstArrayView64<uint8>(GetShaderCodeDataStart(), static_cast<int64>(LibraryBuffer.GetSize()) - OffsetToShaderCode);
	}

	// Returns the container target file this archive was loaded from.
	FORCEINLINE const FContainerTargetFile* GetTargetFile() const
	{
		return TargetFile;
	}
};

static void ValidateAndLogShaderGroupStats(
	const FShaderLibraryNameInfo& LibraryNameInfo, const FIoStoreShaderCodeArchiveHeader& IoStoreLibraryHeader, const FSerializedShaderArchive& SerializedShaders)
{
	int64 TotalUncompressedSizeViaGroups = 0; // calculate total uncompressed size twice for a sanity check
	int64 TotalUncompressedSizeViaShaders = 0;
	int64 TotalIndividuallyCompressedSize = 0;
	int64 TotalGroupCompressedSize = 0;

	for (int32 IdxGroup = 0, NumGroups = IoStoreLibraryHeader.ShaderGroupEntries.Num(); IdxGroup < NumGroups; ++IdxGroup)
	{
		const FIoStoreShaderGroupEntry& Group = IoStoreLibraryHeader.ShaderGroupEntries[IdxGroup];
		TotalGroupCompressedSize += Group.CompressedSize;
		TotalUncompressedSizeViaGroups += Group.UncompressedSize;

		// now go via shaders
		for (uint32 IdxShaderInGroup = 0; IdxShaderInGroup < Group.NumShaders; ++IdxShaderInGroup)
		{
			int32 ShaderIndex = IoStoreLibraryHeader.ShaderIndices[Group.ShaderIndicesOffset + IdxShaderInGroup];
			const FShaderCodeEntry& IndividuallyCompressedShader = SerializedShaders.ShaderEntries[ShaderIndex];

			TotalIndividuallyCompressedSize += IndividuallyCompressedShader.Size;
			TotalUncompressedSizeViaShaders += IndividuallyCompressedShader.GetUncompressedSize();
		}
	}

	checkf(TotalUncompressedSizeViaGroups == TotalUncompressedSizeViaShaders,
		TEXT("Sanity check failure: total uncompressed shader size differs if calculated via shader groups (%lld) or individual shaders (%lld)"),
		TotalUncompressedSizeViaGroups, TotalUncompressedSizeViaShaders
	);

	UE_LOGF(LogIoStore, Display,
		"%ls(%ls): Recompressed %d shaders as %d groups. Library size changed from %lld KB (%.2f : 1 ratio) to %lld KB (%.2f : 1 ratio), %.2f%% of previous.",
		*LibraryNameInfo.GetLibraryName(),
		*LibraryNameInfo.ShaderPlatformName.ToString(),
		SerializedShaders.ShaderEntries.Num(),
		IoStoreLibraryHeader.ShaderGroupEntries.Num(),
		TotalIndividuallyCompressedSize / 1024,
		TotalIndividuallyCompressedSize > 0 ? static_cast<double>(TotalUncompressedSizeViaShaders) / static_cast<double>(TotalIndividuallyCompressedSize) : 0.0,
		TotalGroupCompressedSize / 1024,
		TotalGroupCompressedSize > 0 ? static_cast<double>(TotalUncompressedSizeViaShaders) / static_cast<double>(TotalGroupCompressedSize) : 0.0,
		TotalIndividuallyCompressedSize > 0 ? 100.0 * static_cast<double>(TotalGroupCompressedSize) / static_cast<double>(TotalIndividuallyCompressedSize) : 0.0
	);
}

static bool ParseIoStoreShaderLibraryName(FShaderLibraryNameInfo& OutLibraryNameInfo, const TCHAR* InFilename)
{
	if (!OutLibraryNameInfo.ParseFromFilename(InFilename))
	{
		UE_LOGF(LogIoStore, Error, "Invalid shader code library filename '%ls'.", InFilename);
		return false;
	}
	return true;
}

FIoStoreShaderLibraryProcessor::FIoStoreShaderLibraryProcessor()
{
	// Explicit default constructor to ensure AllMonolithicShaderArchives and AllAssignedSerializedShaders need their full type declaration only in this file
}

FIoStoreShaderLibraryProcessor::~FIoStoreShaderLibraryProcessor()
{
	// Explicit default destructor to ensure AllMonolithicShaderArchives and AllAssignedSerializedShaders need their full type declaration only in this file
}

bool FIoStoreShaderLibraryProcessor::LoadSerializedShaders(
	const FContainerTargetFile& TargetFile, FSerializedShaderArchiveBuffer& OutSerializedShaders, bool bIncludeTypeInfo, FShaderLibraryNameInfo* OutNameInfo)
{
	const TCHAR* InFilename = *TargetFile.NormalizedSourcePath;

	// Convert shader format and platform name from *.ushaderbytecode filename
	FShaderLibraryNameInfo LibraryNameInfo;
	if (!ParseIoStoreShaderLibraryName(LibraryNameInfo, InFilename))
	{
		return false;
	}

	if (OutNameInfo)
	{
		*OutNameInfo = LibraryNameInfo;
	}

	// If this is a proxy target file, don't load anything and just return the output library name
	const bool bIsProxyTargetFile = FPathViews::GetPath(InFilename).IsEmpty();
	if (bIsProxyTargetFile)
	{
		return true;
	}

	// Load serialized shaders from *.ushaderbytecode file
	FCookedPackageStore* CurrentPackageStore = PackageStore;
	if (!OutSerializedShaders.LoadFromTargetFile(InFilename, &TargetFile, CurrentPackageStore))
	{
		return false;
	}

	if (CurrentPackageStore && !TargetFile.ChunkId.IsValid())
	{
		CurrentPackageStore = nullptr;
	}

	// Load *.assetinfo.json file for the specified input shader library and store its content in the ShaderMapAssetAssociations container
	{
		const FString AssetInfoFilename = FPaths::GetPath(InFilename) / LibraryNameInfo.GetAssetInfoFilename();
		if (!LoadShaderAssetInfo(AssetInfoFilename, CurrentPackageStore, OutSerializedShaders.GetShaderMapAssetAssociations()))
		{
			UE_LOGF(LogIoStore, Error, "Failed loading shader asset info file '%ls'", *AssetInfoFilename);
			return false;
		}
	}

#if WITH_EDITORONLY_DATA
	// Load *.stinfo file for the specified input shader library
	if (bIncludeTypeInfo)
	{
		const FString TypeInfoFilename = FPaths::GetPath(InFilename) / LibraryNameInfo.GetTypeInfoFilename();
		if (!LoadShaderTypeInfo(TypeInfoFilename, CurrentPackageStore, OutSerializedShaders.ShaderTypes))
		{
			UE_LOGF(LogIoStore, Error, "Failed loading shader type info file '%ls'", *TypeInfoFilename);
			return false;
		}
	}
#endif

	return true;
}

static TArray<uint8> DecompressGroupedShaders(
	const FIoStoreShaderCodeArchiveHeader& IoStoreLibraryHeader,
	const FIoStoreShaderGroupEntry& Group,
	const FSerializedShaderArchiveBuffer& SerializedShaders,
	int32& OutSmallestShaderInGroupByOrdinal)
{
	TArray<uint8> UncompressedGroupMemory;
	UncompressedGroupMemory.Reserve(Group.UncompressedSize);
	const uint64 UncompressedGroupMemoryPtr = reinterpret_cast<uint64>(UncompressedGroupMemory.GetData());

	int32 SmallestShaderInGroupByOrdinal = MAX_int32;

	{
		// Not worth to special-case for group size of 1 (by converting to copy) - such groups are very fast to decompress and recompress.
		const uint8* UncompressedShaderStart = UncompressedGroupMemory.GetData();
		for (uint32 IdxShaderInGroup = 0; IdxShaderInGroup < Group.NumShaders; ++IdxShaderInGroup)
		{
			const int32 ShaderIndex = IoStoreLibraryHeader.ShaderIndices[Group.ShaderIndicesOffset + IdxShaderInGroup];
			SmallestShaderInGroupByOrdinal = FMath::Min(SmallestShaderInGroupByOrdinal, ShaderIndex);
			const FIoStoreShaderCodeEntry& Shader = IoStoreLibraryHeader.ShaderEntries[ShaderIndex];

			checkf((reinterpret_cast<uint64>(UncompressedShaderStart) - UncompressedGroupMemoryPtr) == Shader.UncompressedOffsetInGroup,
				TEXT("Shader uncompressed offset in group does not agree with its actual placement: Shader.UncompressedOffsetInGroup=%llu, actual=%llu"),
				Shader.UncompressedOffsetInGroup, (reinterpret_cast<uint64>(UncompressedShaderStart) - UncompressedGroupMemoryPtr)
			);

			// Load and decompress the shader at the desired offset
			SerializedShaders.DecompressShaderFromLibraryBuffer(ShaderIndex, UncompressedGroupMemory);
			UncompressedShaderStart = UncompressedGroupMemory.GetData() + UncompressedGroupMemory.Num();
		}

		checkf((reinterpret_cast<uint64>(UncompressedShaderStart) - UncompressedGroupMemoryPtr) == Group.UncompressedSize,
			TEXT("Uncompressed shader group size does not agree with the actual results (Group.UncompressedSize=%u, actual=%llu)"),
			Group.UncompressedSize, (reinterpret_cast<uint64>(UncompressedShaderStart) - UncompressedGroupMemoryPtr));
	}

	OutSmallestShaderInGroupByOrdinal = SmallestShaderInGroupByOrdinal;

	return UncompressedGroupMemory;
}

static TArray<uint8> CompressShaderGroup(
	const uint8* UncompressedGroupMemory,
	const FIoStoreShaderGroupEntry& Group,
	FOodleDataCompression::ECompressor InOodleCompressor,
	FOodleDataCompression::ECompressionLevel InOodleLevel)
{
	// Get size estimate for compressed group to allocate memory
	int64 CompressedGroupSize = 0;
	ShaderCodeArchive::CompressShaderWithOodle(nullptr, CompressedGroupSize, UncompressedGroupMemory, Group.UncompressedSize, InOodleCompressor, InOodleLevel);
	checkf(CompressedGroupSize > 0, TEXT("CompressedGroupSize estimate seems wrong (%lld)"), CompressedGroupSize);
	checkf(CompressedGroupSize <= MAX_int32, TEXT("CompressedGroupSize exceeded limit of signed integer (%lld)"), CompressedGroupSize);

	// Now compress shader group
	TArray<uint8> CompressedGroupMemory;
	CompressedGroupMemory.SetNumUninitialized(static_cast<int32>(CompressedGroupSize));
	const bool bCompressed = ShaderCodeArchive::CompressShaderWithOodle(CompressedGroupMemory.GetData(), CompressedGroupSize, UncompressedGroupMemory, Group.UncompressedSize, InOodleCompressor, InOodleLevel);
	checkf(bCompressed, TEXT("We could not compress the shader group after providing an estimated memory."));

	// Now reduce the output memory to its actual compressed size, since we had to over estimate the allocation. We don't have to shrink the capacity to avoid re-allocating memory.
	CompressedGroupMemory.SetNum(CompressedGroupSize, EAllowShrinking::No);

	return CompressedGroupMemory;
}

bool FIoStoreShaderLibraryProcessor::ConvertToIoStoreShaderLibrary(FContainerTargetFile& TargetFile, const FSerializedShaderArchiveBuffer& SerializedShaders, const FString& ProjectDir)
{
	const TCHAR* InFilename = *TargetFile.NormalizedSourcePath;

	// Convert shader format and platform name from *.ushaderbytecode filename
	FShaderLibraryNameInfo LibraryNameInfo;
	if (!ParseIoStoreShaderLibraryName(LibraryNameInfo, InFilename))
	{
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*LibraryNameInfo.GetLibraryName());

	// Create IoStore code archived for all serialized shaders. This will organize the shaders into groups.
	FIoStoreShaderCodeArchiveHeader IoStoreLibraryHeader;
	FIoStoreShaderCodeArchive::CreateIoStoreShaderCodeArchiveHeader(LibraryNameInfo.ShaderPlatformName, SerializedShaders, IoStoreLibraryHeader);
	checkf(SerializedShaders.ShaderEntries.Num() == IoStoreLibraryHeader.ShaderEntries.Num(), TEXT("IoStore header has different number of shaders (%d) than the original library (%d)"),
		IoStoreLibraryHeader.ShaderEntries.Num(), SerializedShaders.ShaderEntries.Num());
	checkf(SerializedShaders.ShaderMapEntries.Num() == IoStoreLibraryHeader.ShaderMapEntries.Num(), TEXT("IoStore header has different number of shadermaps (%d) than the original library (%d)"),
		IoStoreLibraryHeader.ShaderMapEntries.Num(), SerializedShaders.ShaderMapEntries.Num());

	// Load all the shaders, decompress them and recompress into groups. Each code chunk is shader group chunk now.
	// This also updates group's compressed size, the only value left to calculate.
	const int32 GroupCount = IoStoreLibraryHeader.ShaderGroupEntries.Num();
	CodeIoChunks.SetNum(GroupCount);
	
	// Is shader group compression for this container target file enabled?
	const bool bCompressShaderGroup = ShouldCompressShaderGroups(LibraryNameInfo.ChunkName, ProjectDir);

	// Compute the library's own ChunkId once so it can be captured by the parallel lambda below.
	// This is the same value that will be assigned to TargetFile.ChunkId after the ParallelFor.
	const FIoChunkId LibraryChunkId = ShaderCodeArchive::GetShaderCodeArchiveChunkId(LibraryNameInfo.ChunkName, FName(LibraryNameInfo.GetFormatAndPlatformName()));

	ParallelFor(CodeIoChunks.Num(),
		[this, &IoStoreLibraryHeader, &SerializedShaders, bCompressShaderGroup, &LibraryChunkId](int32 GroupIndex)
		{
			FIoStoreShaderGroupEntry& Group = IoStoreLibraryHeader.ShaderGroupEntries[GroupIndex];

			// Decompress all individual shaders of the current group
			int32 SmallestShaderInGroupByOrdinal = MAX_int32;
			const TArray<uint8> UncompressedGroupMemory = DecompressGroupedShaders(IoStoreLibraryHeader, Group, SerializedShaders, SmallestShaderInGroupByOrdinal);
		
			TArray<uint8> CompressedGroupMemory;
			uint32 CompressedGroupMemorySize = 0;
			if (bCompressShaderGroup)
			{
				// Now compress the whole group
				CompressedGroupMemory = CompressShaderGroup(UncompressedGroupMemory.GetData(), Group, this->ShaderOodleCompressor, this->ShaderOodleLevel);
				CompressedGroupMemorySize = static_cast<uint32>(CompressedGroupMemory.Num());
			}

			// Write out IoChunk for current shader group
			{
				FShaderGroupIoChunk& OutCodeIoChunk = this->CodeIoChunks[GroupIndex];
				// Fuse the content-based group hash with the library's ChunkId so that the same
				// shader group present in two different libraries produces distinct ChunkIds,
				// preventing collisions in the IoStore container. Both the code chunk and the
				// serialized header entry must use the same scoped id so the runtime can resolve them.
				const FIoChunkId ScopedChunkId = ShaderCodeArchive::GetLibraryScopedShaderGroupChunkId(IoStoreLibraryHeader.ShaderGroupIoHashes[GroupIndex], LibraryChunkId);
				OutCodeIoChunk.ChunkId = ScopedChunkId;
				IoStoreLibraryHeader.ShaderGroupIoHashes[GroupIndex] = ScopedChunkId;
				// This value is the load order factor for the group (the smaller, the more likely), that IoStore will use to sort the chunks.
				// We calculate it as the smallest shader index in the group, which shouldn't be bad. Arguably a better way would be to get the lowest-numbered shadermap that references _any_ shader in the group, 
				// but it's much, much slower to calculate and the resulting order would be likely the same/similar most of the time
				checkf(SmallestShaderInGroupByOrdinal >= 0 && SmallestShaderInGroupByOrdinal < IoStoreLibraryHeader.ShaderEntries.Num(), TEXT("SmallestShaderInGroupByOrdinal has an invalid value of %d (not within [0, %d) range as expected)"),
					SmallestShaderInGroupByOrdinal, IoStoreLibraryHeader.ShaderEntries.Num());
				OutCodeIoChunk.LoadOrderFactor = static_cast<uint32>(SmallestShaderInGroupByOrdinal);


				// Copy shader group payload into IoBuffer
				if (bCompressShaderGroup && CompressedGroupMemorySize < Group.UncompressedSize)
				{
					OutCodeIoChunk.CodeIoBuffer = FIoBuffer(CompressedGroupMemorySize);
					FMemory::Memcpy(OutCodeIoChunk.CodeIoBuffer.GetData(), CompressedGroupMemory.GetData(), CompressedGroupMemorySize);
					Group.CompressedSize = CompressedGroupMemorySize;
				}
				else
				{
					// store uncompressed (unlikely, but happens for a 200-byte sized shader that happens to get its own group)
					OutCodeIoChunk.CodeIoBuffer = FIoBuffer(Group.UncompressedSize);
					FMemory::Memcpy(OutCodeIoChunk.CodeIoBuffer.GetData(), UncompressedGroupMemory.GetData(), Group.UncompressedSize);
					Group.CompressedSize = Group.UncompressedSize;
				}
			}
		},
		EParallelForFlags::Unbalanced
	);

	// Calculate and log stats
	ValidateAndLogShaderGroupStats(LibraryNameInfo, IoStoreLibraryHeader, SerializedShaders);

	// Override IoChunk for shader library header.
	// The old ChunkId was for serialized shaders, but now we store FIoStoreShaderCodeArchive for this target file, so use a new ChunkId.
	// The name that generates the ChunkId (a hash rather) needs to match an exact name pattern that is later reconstructed when the pakchunk is mounted.
	// See FNamedShaderLibrary::OnPakFileMounted()
	{
		FLargeMemoryWriter IoStoreLibraryAr(0, true);
		FIoStoreShaderCodeArchive::SaveIoStoreShaderCodeArchive(IoStoreLibraryHeader, IoStoreLibraryAr);
		TargetFile.ChunkId = ShaderCodeArchive::GetShaderCodeArchiveChunkId(LibraryNameInfo.ChunkName, FName(LibraryNameInfo.GetFormatAndPlatformName()));
		TargetFile.SourceSize = static_cast<uint64>(IoStoreLibraryAr.TotalSize());
		TargetFile.SourceBuffer.Emplace(FIoBuffer(FIoBuffer::AssumeOwnership, IoStoreLibraryAr.ReleaseOwnership(), TargetFile.SourceSize));

		// Log the shader code archive name with its ChunkId and its target filename, so we can match it with the shader library at runtime if needed.
		UE_LOGF(LogIoStore, Display, "Save IoStoreShaderCodeArchive '%ls-%ls' with ChunkId '%ls' for target file '%ls'",
			*LibraryNameInfo.ChunkName, *LibraryNameInfo.GetFormatAndPlatformName(), *LexToString(TargetFile.ChunkId), *TargetFile.NormalizedSourcePath);
	}

	// Return shadermaps: own hash and array of shader group IoChunk Ids
	{
		const int32 ShaderMapCount = IoStoreLibraryHeader.ShaderMapHashes.Num();
		CodeIoChunkMaps.SetNum(ShaderMapCount);
		ParallelFor(CodeIoChunkMaps.Num(),
			[this, &IoStoreLibraryHeader](int32 ShaderMapIndex)
			{
				FShaderGroupIoChunkMap& OutShaderMap = this->CodeIoChunkMaps[ShaderMapIndex];
				OutShaderMap.ShaderMapHash = IoStoreLibraryHeader.ShaderMapHashes[ShaderMapIndex];
				const FIoStoreShaderMapEntry& ShaderMapEntry = IoStoreLibraryHeader.ShaderMapEntries[ShaderMapIndex];
				int32 LookupIndexEnd = ShaderMapEntry.ShaderIndicesOffset + ShaderMapEntry.NumShaders;
				OutShaderMap.ShaderGroupChunkIds.Reserve(ShaderMapEntry.NumShaders); // worst-case, 1 group == 1 shader
				for (int32 ShaderLookupIndex = ShaderMapEntry.ShaderIndicesOffset; ShaderLookupIndex < LookupIndexEnd; ++ShaderLookupIndex)
				{
					int32 ShaderIndex = IoStoreLibraryHeader.ShaderIndices[ShaderLookupIndex];
					int32 GroupIndex = IoStoreLibraryHeader.ShaderEntries[ShaderIndex].ShaderGroupIndex;
					OutShaderMap.ShaderGroupChunkIds.AddUnique(IoStoreLibraryHeader.ShaderGroupIoHashes[GroupIndex]);
				}
			},
			EParallelForFlags::Unbalanced
		);
	}

	return true;
}

/**
 * Helper class to iterate over each target file of a specific chunk type in all container targets.
 * For a container with thousands of target files, there might only be a handful of ShaderCodeLibrary target files and we need to iterate through them in several passes.
 * So this class caches them and optionally sorts them to speed up iteration later on.
 */
class FContainerTargetFileIterator
{
	TArray<FContainerTargetFile*> CachedTargetFiles;

	/**
	 * Builds a map that maps from ChunkID to its depth in the chunk dependency tree.
	 * The depth of the root's ChunkID is 0. The depth of an arbitrary ChunkID in the tree is the number of links to traverse to travel from the root to the ChunkID.
	 */
	static void BuildChunkdIdToDependencyTreeDepthMap(const FChunkDependencyTreeNode& TreeNode, TMap<int32, int32>& OutChunkIdToTreeDepthMap, int32 TreeDepth = 0)
	{
		OutChunkIdToTreeDepthMap.Add(TreeNode.ChunkID) = TreeDepth;
		for (const FChunkDependencyTreeNode& ChildTreeNode : TreeNode.ChildNodes)
		{
			FContainerTargetFileIterator::BuildChunkdIdToDependencyTreeDepthMap(ChildTreeNode, OutChunkIdToTreeDepthMap, TreeDepth + 1);
		}
	}

public:
	FContainerTargetFileIterator() = default;

	FContainerTargetFileIterator(const TArrayView<FContainerTargetSpec*>& InContainerTargets, EContainerChunkType InChunkType)
	{
		ResetCache(InContainerTargets, InChunkType);
	}

	void ResetCache(const TArrayView<FContainerTargetSpec*>& InContainerTargets, EContainerChunkType InChunkType)
	{
		// Cache all target files for the input chunk type.
		CachedTargetFiles.Reset(CachedTargetFiles.Num());
		for (FContainerTargetSpec* ContainerTarget : InContainerTargets)
		{
			for (FContainerTargetFile& TargetFile : ContainerTarget->TargetFiles)
			{
				if (TargetFile.ChunkType == InChunkType)
				{
					CachedTargetFiles.Add(&TargetFile);
				}
			}
		}
	}

	/**
	 * Sorts this->CachedTargetFiles based on the dependency tree information in InContainerTargetToChunkDependencies.
	 * Each CachedTargetFile is sorted by its container's depth in the dependency tree, from root (depth 0) to leaf.
	 * When traversing the CachedTargetFiles in this order, we are therefore guaranteed to process the CachedTargetFiles
	 * in the container of a parent AssetManager Chunk before processing any in the container of a child AssetManager
	 * Chunk. Parent chunks are always loaded whenever child chunks are loaded. This allows us to remove shaders from
	 * child chunks based on the full knowledge of what shaders exist in their parent chunks.
	 */
	void SortByChunkDependency(const TMap<const FContainerTargetSpec*, FContainerChunkDependency>& InContainerTargetToChunkDependencies)
	{
		auto PrintCachedTargetFiles = [this]() -> FString
			{
				TStringBuilder<1024> OutString;
				for (const FContainerTargetFile* TargetFile : CachedTargetFiles)
				{
					OutString.Appendf(TEXT("\n  %s"), *TargetFile->NormalizedSourcePath);
				}
				return FString(OutString);
			};

		UE_LOGF(LogIoStore, Verbose, "SortByChunkDependency: Target files *before* sorting:%ls", *PrintCachedTargetFiles());

		if (UChunkDependencyInfo* DependencyInfo = GetMutableDefault<UChunkDependencyInfo>())
		{
			// Get chunk dependency tree to sort target files by their level in the tree hiearchy
			if (const FChunkDependencyTreeNode* ChunkDepTreeNode = DependencyInfo->GetOrBuildChunkDependencyGraph())
			{
				TMap<int32, int32> ChunkIdToTreeDepthMap;
				FContainerTargetFileIterator::BuildChunkdIdToDependencyTreeDepthMap(*ChunkDepTreeNode, ChunkIdToTreeDepthMap);

				auto FindChunkIdTreeDepthForTargetFile = [&InContainerTargetToChunkDependencies, &ChunkIdToTreeDepthMap](const FContainerTargetFile& TargetFile) -> int32
					{
						if (const FContainerChunkDependency* ContainerChunkDependency = InContainerTargetToChunkDependencies.Find(TargetFile.ContainerTarget))
						{
							if (const int32* TreeDepth = ChunkIdToTreeDepthMap.Find(ContainerChunkDependency->ChunkId))
							{
								return *TreeDepth;
							}
						}

						// TargetFiles that are not in a container (should be impossible, but not important to this algorithm)
						// or that are in a container which is not part of the AssetManager's chunk dependency tree (could be possible, depending on the project's config data)
						// can be processed in any order since they don't have parents. We arbitrarily process them first by setting their level to INDEX_NONE.
						return INDEX_NONE;
					};

				// Use StableSort to prevent indeterminism during staging.
				CachedTargetFiles.StableSort([&FindChunkIdTreeDepthForTargetFile](FContainerTargetFile& Lhs, FContainerTargetFile& Rhs) -> bool {
					const int32 LhsTreeDepth = FindChunkIdTreeDepthForTargetFile(Lhs);
					const int32 RhsTreeDepth = FindChunkIdTreeDepthForTargetFile(Rhs);
					return LhsTreeDepth < RhsTreeDepth;
				});

				UE_LOGF(LogIoStore, Verbose, "SortByChunkDependency: Target files *after* sorting:%ls", *PrintCachedTargetFiles());
			}
		}
	}

	void ForEachTargetFile(const TFunction<void(FContainerTargetFile&)>& TargetFileCallback)
	{
		for (FContainerTargetFile* TargetFile : CachedTargetFiles)
		{
			TargetFileCallback(*TargetFile);
		}
	}
};

// Returns true if the specified target file refers to a Global shadermap library
static bool IsTargetFileGlobalShaderLibrary(const FContainerTargetFile& TargetFile)
{
	return FPathViews::GetCleanFilename(TargetFile.NormalizedSourcePath).StartsWith(TEXT("ShaderArchive-Global-"));
}

// Returns true if the specified target file must not be moved during chunk re-assignment.
// This applies to the global Global shader library. If needed, further conditions can be added here to also include Chunk0 for instance.
static bool IsTargetFileExcludedFromChunkReassignment(const FContainerTargetFile& TargetFile)
{
	return IsTargetFileGlobalShaderLibrary(TargetFile);
}

// Generate shader chunk name from pakchunk.
// This needs to match the shader library name that will be loaded through IoStore when pakchunks are mounted at runtime.
// Format is "ShaderArchive-<PROJECT>_Chunk<ID>-<SHADERFORMAT>-<SHADERPLATFORM>", e.g. "ShaderArchive_Lyra_Chunk0-SF_VULKAN_SM6-VULKAN_SM6".
static FString GetShaderChunkNameFromPakChunk(const FStringView& PakChunkName, const TCHAR* ProjectName, const TCHAR* InFormatAndPlatformName)
{
	int32 ChunkId = 0;
	FString ChunkIdSuffix;
	(void)FShaderLibraryNameInfo::ParsePakChunkId(PakChunkName, ChunkId, ChunkIdSuffix);
	return FString::Printf(TEXT("ShaderArchive-%s_Chunk%d%s-%s"), ProjectName, ChunkId, *ChunkIdSuffix, InFormatAndPlatformName);
}

// Returns true if a parsed pakchunk suffix denotes an Install-On-Demand container.
// Pakchunk suffixes can combine multiple tokens (e.g. "ondemandoptional"), so we substring-match
// against the IAD-class tokens. The substring "iad" matches both "iad" and "fatiad".
static bool IsIADChunkSuffix(FStringView ChunkIdSuffix)
{
	return ChunkIdSuffix.Contains(TEXTVIEW("iad")) || ChunkIdSuffix.Contains(TEXTVIEW("ondemand"));
}

// Returns true if a parsed pakchunk suffix denotes a container that is not guaranteed to be installed
// when the game runs. This includes IAD containers and "optional" containers.
// Used to exclude such containers from acting as parent sources during shadermap dedup, since shadermaps
// in a not-guaranteed-installed parent can't be relied upon.
static bool IsExternalChunkSuffix(FStringView ChunkIdSuffix)
{
	return IsIADChunkSuffix(ChunkIdSuffix) || ChunkIdSuffix.Contains(TEXTVIEW("optional"));
}

// Matches a configured container reference (e.g. "pakchunk1") against a real container name (e.g. "pakchunk1-WindowsClient")
// with platform-suffix tolerance but no prefix ambiguity. Specifically:
//   "pakchunk1"  matches "pakchunk1" and "pakchunk1-WindowsClient" but NOT "pakchunk10" / "pakchunk100" / "pakchunk1optional".
// The configured name must equal the container name OR be followed by a '-' platform separator. Prevents the silent
// misconfiguration where a substring match would route shaders to "pakchunk10" when the user wrote "pakchunk1".
static bool DoesContainerNameMatchSpec(FStringView ContainerName, FStringView Spec)
{
	if (!ContainerName.StartsWith(Spec))
	{
		return false;
	}
	return ContainerName.Len() == Spec.Len() || ContainerName[Spec.Len()] == TEXT('-');
}

// Adds a new container target file that acts as a proxy for shader libraries that are re-assigned to a new container target.
static FContainerTargetFile* AddShaderLibraryHeaderToContainerTarget(FContainerTargetSpec* ContainerTarget, const FStringView& PakChunkName, const TCHAR* ProjectName, const TCHAR* InFormatAndPlatformName)
{
	check(ContainerTarget);
	FContainerTargetFile& ShaderLibraryTargetFile = ContainerTarget->TargetFiles.AddDefaulted_GetRef();
	ShaderLibraryTargetFile.ContainerTarget = ContainerTarget;
	ShaderLibraryTargetFile.NormalizedSourcePath = GetShaderChunkNameFromPakChunk(PakChunkName, ProjectName, InFormatAndPlatformName);
	ShaderLibraryTargetFile.ChunkType = EContainerChunkType::ShaderCodeLibrary;
	ShaderLibraryTargetFile.SourceSize = 0llu;
	ShaderLibraryTargetFile.ChunkId = FIoChunkId::InvalidChunkId;
	return &ShaderLibraryTargetFile;
}

// Returns true if the container target has any shader library target file of the specified format.
static bool HasContainerShaderLibraryFormat(const FContainerTargetSpec& InContainerTarget, FStringView InFormatAndPlatformName)
{
	for (const FContainerTargetFile& TargetFile : InContainerTarget.TargetFiles)
	{
		if (TargetFile.ChunkType == EContainerChunkType::ShaderCodeLibrary &&
			TargetFile.NormalizedSourcePath.Contains(InFormatAndPlatformName))
		{
			return true;
		}
	}
	return false;
}

// Returns the name of the project that is being packaged and all target shader-platform formats.
static bool GetPackagedProjectNameAndTargetFormats(TArrayView<FContainerTargetSpec*> InContainerTargets, FString& OutProjectName, FString& OutProjectDir, TArray<FString>& OutFormatAndPlatformNames)
{
	OutProjectName.Empty();
	OutProjectDir.Empty();
	OutFormatAndPlatformNames.Empty();

	// Read project name from command line, since FApp::GetProjectName() returns "UnrealPak" instead of the project that is being packaged and `-ProjectName=` is not a required command line argument.
	// The first command line argument is always the *.uproject filename.
	{
		FString ProjectFilename;
		const TCHAR* CmdLine = FCommandLine::GetOriginal();
		if (FParse::Token(CmdLine, ProjectFilename, false, TEXT(' ')))
		{
			OutProjectName = FPaths::GetBaseFilename(ProjectFilename);
			OutProjectDir = FPaths::GetPath(ProjectFilename);
		}
		else
		{
			return false;
		}
	}

	// Heuristic to find all shader formats relevant for the container targets.
	FContainerTargetFileIterator{ InContainerTargets, EContainerChunkType::ShaderCodeLibrary }
		.ForEachTargetFile([&OutProjectName, &OutFormatAndPlatformNames](FContainerTargetFile& TargetFile) -> void {
			FShaderLibraryNameInfo LibraryNameInfo;
			LibraryNameInfo.ParseFromFilename(*TargetFile.NormalizedSourcePath);
			OutFormatAndPlatformNames.AddUnique(LibraryNameInfo.GetFormatAndPlatformName());
		});

	return !OutProjectName.IsEmpty() && !OutFormatAndPlatformNames.IsEmpty();
}

void FIoStoreShaderLibraryProcessor::ProcessShaderLibraries(const FIoStoreArguments& Arguments, TArrayView<FContainerTargetSpec*> ContainerTargets, FIoStoreShaderLibraryOutput& Output)
{
	IOSTORE_CPU_SCOPE(ProcessShaderLibraries);

	const double LibraryStart = FPlatformTime::Seconds();
	TArray<FString> FormatAndPlatformNames;
	FString ProjectName;
	FString ProjectDir;

	// Cache input arguments
	PackageStore = Arguments.PackageStore.Get();
	ShaderOodleCompressor = Arguments.ShaderOodleCompressor;
	ShaderOodleLevel = Arguments.ShaderOodleLevel;

	// Resolve the packaged project name and directory from the .uproject path on the command line.
	// FApp::GetProjectName() returns "UnrealPak" in this context, so we must derive it from the command line.
	GetPackagedProjectNameAndTargetFormats(ContainerTargets, ProjectName, ProjectDir, FormatAndPlatformNames);

	FContainerTargetFileIterator TargetFileIterator;

	if (IsChunkOverrideEnabled(ProjectDir))
	{
#if WITH_EDITORONLY_DATA
		UE_LOGF(LogIoStore, Display, "Shader library chunk override is enabled");

		// Resolve [ShaderCodeLibrary.PakChunkOverride] +FormatTargetContainer entries before proxy generation, so the
		// subsequent proxy/drop/assign passes can consult FormatToOverrideContainer. Pass the cook's format list so
		// the resolver can drop rules whose format isn't produced (catches typos and cross-platform misconfiguration).
		ResolveFormatTargetContainerOverrides(ContainerTargets, FormatAndPlatformNames);

		//@lh-todo:
		//  We need to generate all target files for ShaderCodeLibrary chunks. The shader library headers for both IAD and non-IAD
		//  must be put into non-IAD containers, so they are always mounted and immediately installed.
		//  We shouldn't put those headers into the IAD containers just to move them back out into non-IAD containers.
		//  Clean up this code so that ProcessShaderLibraryFromTargetFile() can emplace shader library headers
		//  and shader code entries into separate containers; Also see 'MoveShaderLibraryTOCsToInstalledContainers' below.
		// Proxy shader library targets
		{
			IOSTORE_CPU_SCOPE(ProxyShaderLibraryTargets);

			if (!ProjectName.IsEmpty() && !FormatAndPlatformNames.IsEmpty())
			{
				// Add proxy target files for all containers that don't contain a shader library yet.
				// This provides the shader library targets for pakchunks that were overridden but didn't have an assigned shader library previously.
				//
				// When a format is listed in [ShaderCodeLibrary.PakChunkOverride] +FormatTargetContainer, only its override
				// container gets a proxy. All other containers are skipped for that format because their shaders will be
				// redirected into the override container during AssignShaderArchives, and any pre-existing source target
				// files for that format in other containers will be dropped post-preload.
				for (const FString& FormatAndPlatformName : FormatAndPlatformNames)
				{
					if (FContainerTargetSpec* const* OverrideTarget = FormatToOverrideContainer.Find(FormatAndPlatformName))
					{
						FContainerTargetSpec* OverrideContainer = *OverrideTarget;
						if (!HasContainerShaderLibraryFormat(*OverrideContainer, FormatAndPlatformName))
						{
							FContainerTargetFile* ProxyTargetFile = AddShaderLibraryHeaderToContainerTarget(OverrideContainer, OverrideContainer->Name.ToString(), *ProjectName, *FormatAndPlatformName);
							UE_LOGF(LogIoStore, Display, "Add shader library proxy target file '%ls' to override container '%ls' (format '%ls')",
								*ProxyTargetFile->NormalizedSourcePath, *OverrideContainer->Name.ToString(), *FormatAndPlatformName);
						}
					}
					else
					{
						for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
						{
							if (!HasContainerShaderLibraryFormat(*ContainerTarget, FormatAndPlatformName))
							{
								// Add new proxy target file for shader library, since it doesn't have one in the current container target for this format.
								FContainerTargetFile* ProxyTargetFile = AddShaderLibraryHeaderToContainerTarget(ContainerTarget, ContainerTarget->Name.ToString(), *ProjectName, *FormatAndPlatformName);
								UE_LOGF(LogIoStore, Display, "Add shader library proxy target file '%ls' to container '%ls'", *ProxyTargetFile->NormalizedSourcePath, *ContainerTarget->Name.ToString());
							}
						}
					}
				}
			}
			else
			{
				// If no other container had any shader libraries we could obtain the shader formats from,
				// skip creating proxy targets as there is nothing to re-assign shaders to (e.g. when packaging a Server).
				UE_LOGF(LogIoStore, Display, "Skipped shader library proxy targets due to missing shader format information from other containers (%d total)", ContainerTargets.Num());
			}
		}

		// Cache all target files and sort them by chunk dependency to ensure we deduplicate redundant shadermaps as tight as possible. 
		{
			for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
			{
				BuildContainerTargetChunkDependencies(ContainerTarget);
			}

			TargetFileIterator.ResetCache(ContainerTargets, EContainerChunkType::ShaderCodeLibrary);
			TargetFileIterator.SortByChunkDependency(ContainerTargetToChunkDependencies);
		}

		// Preload all shader archives, so we can re-assign shaders to their targets from monolithic libraries
		{
			IOSTORE_CPU_SCOPE(PreloadShaderLibraries);
			TargetFileIterator.ForEachTargetFile([this, &Arguments, &Output](FContainerTargetFile& TargetFile) -> void {
				if (!IsTargetFileExcludedFromChunkReassignment(TargetFile) && !PreloadShaderArchive(TargetFile))
				{
					UE_LOGF(LogIoStore, Warning, "Failed preloading shader library '%ls'", *TargetFile.NormalizedSourcePath);
				}
			});
		}

		// Normalize monolithic shader library source target files to the per-pakchunk "_Chunk<N>" naming form.
		// When the cooker emits a single shared shader library per format (e.g. "ShaderArchive-<Project>-PCD3D_SM6-PCD3D_SM6")
		// instead of pre-chunked libraries, the source target file's name lacks the "_Chunk<N>" suffix that the runtime uses
		// to look up a shader library on pakchunk mount (see FNamedShaderLibrary::OnPakFileMounted). Rewrite the source path
		// here, after preload has finished reading from it, so that AssignShaderArchives and ConvertToIoStoreShaderLibrary
		// produce ChunkIds that match the runtime's expected name pattern for the container the library lives in.
		{
			IOSTORE_CPU_SCOPE(NormalizeMonolithicShaderLibraryNames);
			static constexpr FStringView ChunkNameToken = TEXTVIEW("_Chunk");
			TargetFileIterator.ForEachTargetFile([&ProjectName](FContainerTargetFile& TargetFile) -> void {
				if (IsTargetFileExcludedFromChunkReassignment(TargetFile))
				{
					return;
				}
				FShaderLibraryNameInfo LibraryNameInfo;
				if (!LibraryNameInfo.ParseFromFilename(*TargetFile.NormalizedSourcePath))
				{
					return;
				}
				// Detect the "_Chunk<N>" suffix produced by AddShaderLibraryHeaderToContainerTarget. If absent, the file is
				// a monolithic source library that needs to be remapped onto its container's pakchunk identity.
				const int32 ChunkSuffixPos = LibraryNameInfo.ChunkName.Find(ChunkNameToken, ESearchCase::CaseSensitive, ESearchDir::FromEnd);
				const int32 DigitPosAfterChunkToken = ChunkSuffixPos + ChunkNameToken.Len();
				const bool bAlreadyChunked = ChunkSuffixPos != INDEX_NONE
					&& DigitPosAfterChunkToken < LibraryNameInfo.ChunkName.Len()
					&& FChar::IsDigit(LibraryNameInfo.ChunkName[DigitPosAfterChunkToken]);
				if (bAlreadyChunked)
				{
					return;
				}
				check(TargetFile.ContainerTarget);
				const FString OldName = TargetFile.NormalizedSourcePath;
				TargetFile.NormalizedSourcePath = GetShaderChunkNameFromPakChunk(
					TargetFile.ContainerTarget->Name.ToString(),
					*ProjectName,
					*LibraryNameInfo.GetFormatAndPlatformName());
				TargetFile.ChunkId = FIoChunkId::InvalidChunkId;
				UE_LOGF(LogIoStore, Display,
					"Normalized monolithic shader library source '%ls' to '%ls' for container '%ls'",
					*OldName, *TargetFile.NormalizedSourcePath, *TargetFile.ContainerTarget->Name.ToString());
			});
		}

		// Drop shader library source target files for overridden formats from any container that is NOT the override target.
		// Their shaders were already pooled into AllMonolithicShaderArchives during preload, and the override container
		// will receive every shader for that format in AssignShaderArchives. Leaving these files in place would produce
		// stale empty target files in the wrong containers, and the iterator would still try to convert them.
		if (FormatToOverrideContainer.Num() > 0)
		{
			IOSTORE_CPU_SCOPE(DropOverriddenFormatSourceFiles);
			bool bDroppedAny = false;
			for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
			{
				for (auto Iter = ContainerTarget->TargetFiles.CreateIterator(); Iter; ++Iter)
				{
					if (Iter->ChunkType != EContainerChunkType::ShaderCodeLibrary)
					{
						continue;
					}
					if (IsTargetFileExcludedFromChunkReassignment(*Iter))
					{
						continue;
					}
					FShaderLibraryNameInfo LibraryNameInfo;
					if (!LibraryNameInfo.ParseFromFilename(*Iter->NormalizedSourcePath))
					{
						continue;
					}
					const FString FormatAndPlatformName = LibraryNameInfo.GetFormatAndPlatformName();
					if (FContainerTargetSpec* const* OverrideTarget = FormatToOverrideContainer.Find(FormatAndPlatformName))
					{
						if (*OverrideTarget != ContainerTarget)
						{
							UE_LOGF(LogIoStore, Display,
								"Dropping shader library target '%ls' from container '%ls'; format '%ls' is overridden to container '%ls'",
								*Iter->NormalizedSourcePath, *ContainerTarget->Name.ToString(), *FormatAndPlatformName, *(*OverrideTarget)->Name.ToString());
							Iter.RemoveCurrent();
							bDroppedAny = true;
						}
					}
				}
			}

			// TArray::RemoveCurrent invalidates the iterator's cached FContainerTargetFile pointers.
			// Rebuild the iterator before the assign pass uses it again.
			if (bDroppedAny)
			{
				TargetFileIterator.ResetCache(ContainerTargets, EContainerChunkType::ShaderCodeLibrary);
				TargetFileIterator.SortByChunkDependency(ContainerTargetToChunkDependencies);
			}
		}

		// Now assign shader archives to their final target files
		{
			IOSTORE_CPU_SCOPE(AssignShaderArchives);
			TargetFileIterator.ForEachTargetFile([this, &Arguments, &Output](FContainerTargetFile& TargetFile) -> void {
				if (!IsTargetFileExcludedFromChunkReassignment(TargetFile) && !AssignShaderArchives(TargetFile))
				{
					UE_LOGF(LogIoStore, Warning, "Failed assigning shader archives for target library '%ls'", *TargetFile.NormalizedSourcePath);
				}
			});
		}
#else
		UE_LOGF(LogIoStore, Warning, "Shader library chunk override is enabled, but not supported (WITH_EDITORONLY_DATA=0)");
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		UE_LOGF(LogIoStore, Display, "Shader library chunk override is disabled");

		TargetFileIterator.ResetCache(ContainerTargets, EContainerChunkType::ShaderCodeLibrary);
	}

	{
		IOSTORE_CPU_SCOPE(ConvertShaderLibraries);
		TargetFileIterator.ForEachTargetFile([this, &Arguments, &Output, &ProjectDir](FContainerTargetFile& TargetFile) -> void {
			if (!ProcessShaderLibraryFromTargetFile(Arguments, TargetFile, Output, ProjectDir))
			{
				UE_LOGF(LogIoStore, Warning, "Failed converting shader library '%ls'", *TargetFile.NormalizedSourcePath);
			}
		});
	}

	{
		IOSTORE_CPU_SCOPE(UpdatePackageStoreShaders);
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			for (FCookedPackage* Package : ContainerTarget->Packages)
			{
				UpdatePackageStoreShaders(Package, ContainerTarget->Name);
			}
		}
	}

	// Transfer ownership of shader information (all previously allocated FShaderInfo entries) to the container targets
	{
		IOSTORE_CPU_SCOPE(AddShaderCodeToContainer);
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			AddAllReferencedShadersToContainerTarget(ContainerTarget, Output);
		}
	}

	// Finally, move all shader library header chunks from OnDemand containers back into their equivalent standard containers.
	// This ensures the header (or table-of-content rather) is installed before any shaders are loaded on-demand.
	// The alternative would be to install the header from the OnDemand container as soon as its mounted,
	// but that eliminates the purpose of OnDemand data and would unnecessarily increase download sizes.
	if (IsChunkOverrideEnabled(ProjectDir))
	{
		IOSTORE_CPU_SCOPE(MoveShaderLibraryTOCsToInstalledContainers);

		auto GetConditionalChunkId = [](const FContainerTargetSpec& InContainerTarget, bool bOnDemandOnly) -> int32
			{
				int32 ChunkId = INDEX_NONE;
				FString ChunkIdSuffix;
				(void)FShaderLibraryNameInfo::ParsePakChunkId(InContainerTarget.Name.ToString(), ChunkId, ChunkIdSuffix);
				const bool bIsOnDemandChunk = IsIADChunkSuffix(ChunkIdSuffix);
				return bIsOnDemandChunk == bOnDemandOnly ? ChunkId : INDEX_NONE;
			};

		// First, try to find a pakchunk that was specified as primary target for IAD shader library headers to be loaded before any of such containers are mounted
		FContainerTargetSpec* PrimaryContainerForOnDemandShaderLibTOC = [&ContainerTargets]() ->FContainerTargetSpec*
			{
				const FString PrimaryContainerName = GetPrimaryContainerForOnDemandShaderLibTOC();
				if (!PrimaryContainerName.IsEmpty())
				{
					// Boundary-aware match (see DoesContainerNameMatchSpec): plain Contains() would mis-resolve when one
					// pakchunk name is a prefix of another (e.g. "pakchunk1" matching "pakchunk10").
					for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
					{
						if (DoesContainerNameMatchSpec(ContainerTarget->Name.ToString(), PrimaryContainerName))
						{
							return ContainerTarget;
						}
					}
				}
				return nullptr;
			}();

		auto FindNonOnDemandContainerTarget = [&ContainerTargets, &GetConditionalChunkId, PrimaryContainerForOnDemandShaderLibTOC](const FContainerTargetSpec& InContainerTarget, int32 InChunkId) -> FContainerTargetSpec*
			{
				// First, try to find a pakchunk that was specified as primary target for IAD shader library headers to be loaded before any of such containers are mounted
				if (PrimaryContainerForOnDemandShaderLibTOC)
				{
					return PrimaryContainerForOnDemandShaderLibTOC;
				}

				// Try to find non-OnDemand container with same chunk ID (e.g. "Chunk100" for input container "Chunk100iad")
				for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
				{
					if (GetConditionalChunkId(*ContainerTarget, false) == InChunkId)
					{
						return ContainerTarget;
					}
				}

				// Otherwise, try to find container for "Chunk0" since that is always mounted
				constexpr int32 kAlwaysInstalledChunkId = 0;
				for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
				{
					if (GetConditionalChunkId(*ContainerTarget, false) == kAlwaysInstalledChunkId)
					{
						return ContainerTarget;
					}
				}

				// Failed to find a suitable container
				return nullptr;
			};

		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			if (const int32 ChunkId = GetConditionalChunkId(*ContainerTarget, true); ChunkId != INDEX_NONE)
			{
				for (auto Iter = ContainerTarget->TargetFiles.CreateIterator(); Iter; ++Iter)
				{
					if (Iter->ChunkType == EContainerChunkType::ShaderCodeLibrary)
					{
						// We found a shader code library chunk in an OnDemand container.
						if (FContainerTargetSpec* NewContainerTarget = FindNonOnDemandContainerTarget(*ContainerTarget, ChunkId))
						{
							// Move the target file into the regular container.
							UE_LOGF(LogIoStore, Display, "Moved ShaderCodeLibrary header '%ls' (ChunkId='%ls') out of OnDemand container '%ls' into regular container '%ls'",
								*Iter->NormalizedSourcePath, *LexToString(Iter->ChunkId), *ContainerTarget->Name.ToString(), *NewContainerTarget->Name.ToString());
							NewContainerTarget->TargetFiles.Add(MoveTemp(*Iter));
							Iter.RemoveCurrent();
						}
						else
						{
							UE_LOGF(LogIoStore, Error, "Failed to move ShaderCodeLibrary header '%ls' (ChunkId='%ls') out of OnDemand container '%ls'",
								*Iter->NormalizedSourcePath, *LexToString(Iter->ChunkId), *ContainerTarget->Name.ToString());
						}
					}
				}
			}
		}
	}

#if WITH_EDITORONLY_DATA
	// Dump shader chunk re-assignment to text file if debug information is enabled
	if (IsChunkOverrideEnabled(ProjectDir) && IsChunkOverrideDebugEnabled())
	{
		for (const FString& FormatAndPlatformName : FormatAndPlatformNames)
		{
			LogChunkOverrideStats(FName(FormatAndPlatformName));

			const FString DebugLibFolder = FSerializedShaderArchive::MakeDebugDirectory();
			IFileManager::Get().MakeDirectory(*DebugLibFolder, true);
			const FString DebugFilename = DebugLibFolder / FString::Printf(TEXT("ShaderChunkOverride-%s.txt"), *FormatAndPlatformName);
			if (TUniquePtr<FArchive> DebugFileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*DebugFilename)))
			{
				if (!DumpChunkOverrideDebugInfo(FName(FormatAndPlatformName), *DebugFileWriter))
				{
					UE_LOGF(LogIoStore, Warning, "Failed to save debug output: '%ls'", *DebugFilename);
				}
			}
		}
	}
#endif

	const double LibraryEnd = FPlatformTime::Seconds();
	UE_LOGF(LogIoStore, Display, "Shaders processed in %.2f seconds", LibraryEnd - LibraryStart);
}

bool FIoStoreShaderLibraryProcessor::ProcessShaderLibraryFromTargetFile(const FIoStoreArguments& Arguments, FContainerTargetFile& TargetFile, FIoStoreShaderLibraryOutput& Output, const FString& ProjectDir)
{
	IOSTORE_CPU_SCOPE(ConvertShaderLibrary);

	FContainerTargetSpec* ContainerTarget = TargetFile.ContainerTarget;
	checkf(ContainerTarget, TEXT("Target file for shader library '%s' is missing its parent node (ContainerTarget is null)"), *TargetFile.NormalizedSourcePath);

	if (IsChunkOverrideEnabled(ProjectDir) && !IsTargetFileExcludedFromChunkReassignment(TargetFile))
	{
#if WITH_EDITORONLY_DATA
		// Find shader archive that was assigned to the current target file and convert it into an IoStore shader library
		if (TUniquePtr<FSerializedShaderArchiveBuffer> const* SerializedShaders = AllAssignedSerializedShaders.Find(&TargetFile))
		{
			if (!ConvertToIoStoreShaderLibrary(TargetFile, **SerializedShaders, ProjectDir))
			{
				return false;
			}
		}
		else
		{
			return true; // Ok, no shader library assigned to this targe file
		}
#else
		ensureMsgf(false, TEXT("Cannot re-assign shader archives, because WITH_EDITORONLY_DATA=0"));
		return false;
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		// Without chunk override, convert shader library directly from archive chunk to pakchunk
		FShaderLibraryNameInfo LibraryNameInfo;
		FSerializedShaderArchiveBuffer SerializedShaders;
		if (!LoadSerializedShaders(TargetFile, SerializedShaders, false, &LibraryNameInfo))
		{
			return false;
		}
		ShaderMapAssetAssociations.Append(SerializedShaders.GetShaderMapAssetAssociations());

		// Keep track of old and new pakchunks if debug information is enabled (also for non-chunked targets)
		if (IsChunkOverrideDebugEnabled())
		{
			AddShaderMapsToPakChunkDebugInfo(TargetFile, SerializedShaders, FName(LibraryNameInfo.GetFormatAndPlatformName()), ChunkAssignment_Old | ChunkAssignment_New);
		}

		if (!ConvertToIoStoreShaderLibrary(TargetFile, SerializedShaders, ProjectDir))
		{
			return false;
		}
	}

	const bool bIsGlobalShaderLibrary = IsTargetFileGlobalShaderLibrary(TargetFile);
	const FShaderInfo::EShaderType ShaderType = bIsGlobalShaderLibrary ? FShaderInfo::Global : FShaderInfo::Normal;
	FlushCodeIoChunks(ContainerTarget, Output, ShaderType);

	return true;
}

#if WITH_EDITORONLY_DATA

// Print the set of chunk dependency IDs to string for logging, e.g. "None", "0", "0, 1" etc.
static FString PrintChunkDependenciesToString(const TSet<int32>& InChunkDependencies)
{
	TStringBuilder<1024> OutString;
	if (InChunkDependencies.IsEmpty())
	{
		OutString.Append(TEXT("None"));
	}
	else
	{
		for (int32 ChunkId : InChunkDependencies)
		{
			if (OutString.Len() > 0)
			{
				OutString.Append(TEXT(", "));
			}
			OutString.Appendf(TEXT("%d"), ChunkId);
		}
	}
	return FString(OutString);
}

// Reads +FormatTargetContainer entries from [ShaderCodeLibrary.PakChunkOverride] in GEngineIni and resolves each one
// against the active container target list. Populates FormatToOverrideContainer for use by subsequent passes.
//
// Accepted ini schema:
//   [ShaderCodeLibrary.PakChunkOverride]
//   +FormatTargetContainer=(Format="PCD3D_SM5-PCD3D_SM5",Container="pakchunk0")
//   ...
//
// Format must match the full <ShaderFormat>-<ShaderPlatform> string parsed from a *.ushaderbytecode filename.
// Container is matched by boundary-aware comparison against FContainerTargetSpec::Name (see DoesContainerNameMatchSpec):
// the configured value must equal the container name exactly OR be the same name minus its '-<Platform>' suffix.
// So "pakchunk0" resolves to "pakchunk0-WindowsClient" or "pakchunk0-Windows", but NOT to "pakchunk00" or
// "pakchunk0optional". To target a suffix-bearing container, write its name in full (e.g. "pakchunk0optional").
void FIoStoreShaderLibraryProcessor::ResolveFormatTargetContainerOverrides(TArrayView<FContainerTargetSpec*> ContainerTargets, TArrayView<const FString> InFormatAndPlatformNames)
{
	FormatToOverrideContainer.Empty();
	AllPackagesAcrossContainers.Empty();

	TArray<FString> ConfigEntries;
	GConfig->GetArray(TEXT("ShaderCodeLibrary.PakChunkOverride"), TEXT("FormatTargetContainer"), ConfigEntries, GEngineIni);

	for (const FString& Entry : ConfigEntries)
	{
		FString Format;
		FString Container;
		const TCHAR* Stream = *Entry;
		if (!FParse::Value(Stream, TEXT("Format="), Format) || !FParse::Value(Stream, TEXT("Container="), Container))
		{
			UE_LOGF(LogIoStore, Warning,
				"Malformed [ShaderCodeLibrary.PakChunkOverride] +FormatTargetContainer entry '%ls' ignored. Expected '(Format=\"...\",Container=\"...\")'.",
				*Entry);
			continue;
		}

		// Validate the configured format actually appears in this cook. Catches typos and stray cross-platform config
		// (e.g. a PCD3D_SM5 rule running on a Console cook). Skipped if the cook produced no formats at all -- that case
		// is already reported elsewhere as "Skipped shader library proxy targets due to missing shader format information".
		if (InFormatAndPlatformNames.Num() > 0 && !InFormatAndPlatformNames.Contains(Format))
		{
			UE_LOGF(LogIoStore, Warning,
				"FormatTargetContainer override for format '%ls' has no matching shader library in this cook; ignoring rule. "
				"Check spelling, or scope this config to platforms that produce the format.",
				*Format);
			continue;
		}

		// Resolve container by boundary-aware match against ContainerTarget->Name. The configured value must equal the
		// container name exactly, or be the same name minus its '-<Platform>' suffix. So "pakchunk32" matches
		// "pakchunk32-WindowsClient" but NOT "pakchunk320" or "pakchunk32optional". Plain substring matching would
		// silently mis-route shaders to a similarly-prefixed container.
		FContainerTargetSpec* ResolvedContainer = nullptr;
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			if (DoesContainerNameMatchSpec(ContainerTarget->Name.ToString(), Container))
			{
				ResolvedContainer = ContainerTarget;
				break;
			}
		}
		if (!ResolvedContainer)
		{
			UE_LOGF(LogIoStore, Warning,
				"FormatTargetContainer override for format '%ls' references container '%ls' which does not exist; ignoring rule.",
				*Format, *Container);
			continue;
		}

		// If the resolved target is an IAD container, the existing TOC-move pass will still relocate the shader library
		// header out of it. Inform the user so the configuration's effect is unsurprising.
		{
			int32 ChunkId = INDEX_NONE;
			FString ChunkIdSuffix;
			(void)FShaderLibraryNameInfo::ParsePakChunkId(ResolvedContainer->Name.ToString(), ChunkId, ChunkIdSuffix);
			if (IsIADChunkSuffix(ChunkIdSuffix))
			{
				UE_LOGF(LogIoStore, Display,
					"FormatTargetContainer override for format '%ls' targets IAD container '%ls'; "
					"shader code will land in this container, but the shader library TOC will still relocate per OnDemand rules.",
					*Format, *ResolvedContainer->Name.ToString());
			}
		}

		if (FContainerTargetSpec** ExistingEntry = FormatToOverrideContainer.Find(Format))
		{
			UE_LOGF(LogIoStore, Error,
				"FormatTargetContainer override for format '%ls' is specified more than once; keeping the first mapping ('%ls') and ignoring the duplicate ('%ls').",
				*Format, *(*ExistingEntry)->Name.ToString(), *ResolvedContainer->Name.ToString());
			continue;
		}

		UE_LOGF(LogIoStore, Display,
			"FormatTargetContainer override active: format '%ls' -> container '%ls'",
			*Format, *ResolvedContainer->Name.ToString());
		FormatToOverrideContainer.Add(Format, ResolvedContainer);
	}

	// Populate AllPackagesAcrossContainers iff at least one override resolved. The override-target assignment path uses
	// this set as a permissive package filter for FEditorShaderCodeArchive::CreateNamedChunk -- effectively "include
	// every shadermap referenced by any package," which is the intended semantics for a format-redirect override.
	if (FormatToOverrideContainer.Num() > 0)
	{
		for (const FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			for (const FCookedPackage* CookedPackage : ContainerTarget->Packages)
			{
				AllPackagesAcrossContainers.Add(CookedPackage->PackageName);
			}
		}
	}
}

void FIoStoreShaderLibraryProcessor::BuildContainerTargetChunkDependencies(const FContainerTargetSpec* ContainerTarget)
{
	if (UChunkDependencyInfo* DependencyInfo = GetMutableDefault<UChunkDependencyInfo>())
	{
		FContainerChunkDependency& ContainerChunkDependency = ContainerTargetToChunkDependencies.Add(ContainerTarget);

		FString ContainerTargetChunkIdSuffix;
		if (FShaderLibraryNameInfo::ParsePakChunkId(ContainerTarget->Name.ToString(), ContainerChunkDependency.ChunkId, ContainerTargetChunkIdSuffix))
		{
			// Get chunk dependency tree to determine what shadermaps are already included in a parent chunk
			DependencyInfo->GetOrBuildChunkDependencyGraph(ContainerChunkDependency.ChunkId);
			DependencyInfo->GetChunkDependencies(ContainerChunkDependency.ChunkId, ContainerChunkDependency.ChunkDependencies);
			ContainerChunkDependency.ChunkDependencies.Remove(ContainerChunkDependency.ChunkId);
			// External containers (IAD or Optional) are not guaranteed to be installed alongside their dependent chunks,
			// so they must not act as parent dedup sources for shadermaps in child chunks.
			ContainerChunkDependency.bIsExternalContainer = IsExternalChunkSuffix(ContainerTargetChunkIdSuffix);

			// Log information about chunk dependencies for shader library
			UE_LOGF(LogIoStore, Display, "Processing chunk dependency for container '%ls': ChunkId=%d, bIsExternalContainer=%ls, ChunkDependencies=(%ls)",
				*ContainerTarget->Name.ToString(), ContainerChunkDependency.ChunkId, ContainerChunkDependency.bIsExternalContainer ? TEXT("Yes") : TEXT("No"),
				*PrintChunkDependenciesToString(ContainerChunkDependency.ChunkDependencies));
		}
	}
}

bool FIoStoreShaderLibraryProcessor::PreloadShaderArchive(const FContainerTargetFile& TargetFile)
{
	// Preload serialized shader archive for target file
	FShaderLibraryNameInfo LibraryNameInfo;
	FSerializedShaderArchiveBuffer SerializedShaders;
	const bool bIncludeTypeInfo = true;
	if (!LoadSerializedShaders(TargetFile, SerializedShaders, bIncludeTypeInfo, &LibraryNameInfo))
	{
		return false;
	}
	ShaderMapAssetAssociations.Append(SerializedShaders.GetShaderMapAssetAssociations());

	// Append serialized shaders to monolithic library so we can later create new chunks from it
	TUniquePtr<FEditorShaderCodeArchive>& MonolithicShaderArchive = AllMonolithicShaderArchives.FindOrAdd(LibraryNameInfo.GetFormatAndPlatformName());
	if (!MonolithicShaderArchive)
	{
		MonolithicShaderArchive = TUniquePtr<FEditorShaderCodeArchive>(new FEditorShaderCodeArchive(FName(LibraryNameInfo.GetFormatAndPlatformName())));
	}
	MonolithicShaderArchive->AppendFromArchive(SerializedShaders, SerializedShaders.GetShaderCodeView(), ShaderCodeArchive::ECookShaderLibrarySource::CurrentCook);

	// Keep track of old and new pakchunks if debug information is enabled
	if (IsChunkOverrideDebugEnabled())
	{
		AddShaderMapsToPakChunkDebugInfo(TargetFile, SerializedShaders, FName(LibraryNameInfo.GetFormatAndPlatformName()), ChunkAssignment_Old);
	}

	return true;
}

void FIoStoreShaderLibraryProcessor::AddShaderMapsToPakChunkDebugInfo(
	const FContainerTargetFile& TargetFile, const FSerializedShaderArchiveBuffer& SerializedShaders,
	FName FormatAndPlatformName, int32 ChunkAssignmentType)
{
	check(TargetFile.ContainerTarget);
	FMonolithicArchiveDebugInfo& DebugInfo = ChunkAssignmentDebugInfo.FindOrAdd(FormatAndPlatformName);
	for (const FShaderHash& ShaderMapHash : SerializedShaders.ShaderMapHashes)
	{
		FChunkAssignmentInfo& ShaderMapInfo = DebugInfo.ShaderMaps.FindOrAdd(ShaderMapHash);
		if ((ChunkAssignmentType & ChunkAssignment_New) != 0)
		{
			ShaderMapInfo.NewPakChunks.Add(TargetFile.ContainerTarget->Name);
		}
		if ((ChunkAssignmentType & ChunkAssignment_Old) != 0)
		{
			ShaderMapInfo.OldPakChunks.Add(TargetFile.ContainerTarget->Name);
		}
	}
}

void FIoStoreShaderLibraryProcessor::LogChunkOverrideStats(FName FormatAndPlatformName)
{
	if (const FMonolithicArchiveDebugInfo* DebugInfo = ChunkAssignmentDebugInfo.Find(FormatAndPlatformName))
	{
		const int32 NumUniqueShaderMaps = DebugInfo->ShaderMaps.Num();

		int32 NumShaderMapCopiesInOldChunks = 0;
		int32 NumShaderMapCopiesInNewChunks = 0;

		for (const TPair<FShaderHash, FChunkAssignmentInfo>& Pair : DebugInfo->ShaderMaps)
		{
			NumShaderMapCopiesInOldChunks += Pair.Value.OldPakChunks.Num();
			NumShaderMapCopiesInNewChunks += Pair.Value.NewPakChunks.Num();
		}

		UE_LOGF(LogIoStore, Display,
			"Shader library collection %ls shadermap stats:\n" "  Unique shader maps:              %d\n" "  Shader map copies in old chunks: %d (%.3f : 1 ratio)\n" "  Shader map copies in new chunks: %d (%.3f : 1 ratio)",
			*FormatAndPlatformName.ToString(),
			NumUniqueShaderMaps,
			NumShaderMapCopiesInOldChunks,
			NumUniqueShaderMaps > 0 ? static_cast<double>(NumShaderMapCopiesInOldChunks) / static_cast<double>(NumUniqueShaderMaps) : 0.0,
			NumShaderMapCopiesInNewChunks,
			NumUniqueShaderMaps > 0 ? static_cast<double>(NumShaderMapCopiesInNewChunks) / static_cast<double>(NumUniqueShaderMaps) : 0.0
		);
	}
}

bool FIoStoreShaderLibraryProcessor::DumpChunkOverrideDebugInfo(FName FormatAndPlatformName, FArchive& Ar)
{
	if (!ChunkAssignmentDebugInfo.Contains(FormatAndPlatformName))
	{
		return false;
	}

	// Generate debug output text
	FUtf8String DebugOutput;

	auto AppendPakChunkNames = [&DebugOutput](const UTF8CHAR* InListName, const TArray<FName>& InPakChunks) -> void
		{
			DebugOutput.Appendf(UTF8TEXT("\n  %s: "), InListName);
			if (InPakChunks.IsEmpty())
			{
				DebugOutput.Append(UTF8TEXT("<None>"));
			}
			else
			{
				for (int32 PakChunkIndex = 0; PakChunkIndex < InPakChunks.Num(); ++PakChunkIndex)
				{
					if (PakChunkIndex > 0)
					{
						DebugOutput.Append(UTF8TEXT(", "));
					}
					DebugOutput.Append(TCHAR_TO_UTF8(*InPakChunks[PakChunkIndex].ToString()));
				}
			}
		};

	auto IsAssignmentUnchanged = [](const FChunkAssignmentInfo& InAssignment) -> bool
		{
			if (InAssignment.OldPakChunks.Num() != InAssignment.NewPakChunks.Num())
			{
				return false;
			}
			for (const FName& OldName : InAssignment.OldPakChunks)
			{
				if (!InAssignment.NewPakChunks.Contains(OldName))
				{
					return false;
				}
			}
			return true;
		};

	TOptional<int32> MaxNumIoChunksPerShadermap, MinNumIoChunksPerShadermap;
	uint64 TotalNumberOfIoChunksReferences = 0; // Used to determine average number of IoChunks per shadermap

	const FMonolithicArchiveDebugInfo& DebugInfo = ChunkAssignmentDebugInfo[FormatAndPlatformName];
	for (const TPair<FShaderHash, FChunkAssignmentInfo>& Pair : DebugInfo.ShaderMaps)
	{
		DebugOutput.Appendf("Shadermap %s", TCHAR_TO_UTF8(*Pair.Key.ToString()));
		{
			// Print old and new containers this shadermap resides in
			if (IsAssignmentUnchanged(Pair.Value))
			{
				AppendPakChunkNames(UTF8TEXT("Unchanged"), Pair.Value.OldPakChunks);
			}
			else
			{
				AppendPakChunkNames(UTF8TEXT("Old"), Pair.Value.OldPakChunks);
				AppendPakChunkNames(UTF8TEXT("New"), Pair.Value.NewPakChunks);
			}

			// Print IoChunks used by current shadermap
			DebugOutput.Append(UTF8TEXT("\n  IoChunks: "));
			if (const TArray<FIoChunkId>* IoChunkIds = ShaderMapToGroupChunkIds.Find(Pair.Key))
			{
				TotalNumberOfIoChunksReferences += static_cast<uint64>(IoChunkIds->Num());
				MaxNumIoChunksPerShadermap = FMath::Max<int32>(MaxNumIoChunksPerShadermap.Get(0), IoChunkIds->Num());
				MinNumIoChunksPerShadermap = FMath::Min<int32>(MinNumIoChunksPerShadermap.Get(MAX_int32), IoChunkIds->Num());

				for (int32 IoChunkIndex = 0; IoChunkIndex < IoChunkIds->Num(); ++IoChunkIndex)
				{
					if (IoChunkIndex > 0)
					{
						DebugOutput.Append(UTF8TEXT(", "));
					}
					DebugOutput.Append(TCHAR_TO_UTF8(*LexToString((*IoChunkIds)[IoChunkIndex])));
				}
			}
			else
			{
				DebugOutput.Append(UTF8TEXT("<None>"));
			}
		}
		DebugOutput.Append("\n\n");
	}

	const double AvgNumIoChunksPerShadermap = DebugInfo.ShaderMaps.Num() > 0 ? static_cast<double>(TotalNumberOfIoChunksReferences) / static_cast<double>(DebugInfo.ShaderMaps.Num()) : 0.0;

	// Write stats
	FUtf8String StatsOutput;

	StatsOutput.Appendf(
		UTF8TEXT("------------------------------------------\n")
		UTF8TEXT("Stats\n")
		UTF8TEXT("  Minimum IoChunks per shadermap: %d\n")
		UTF8TEXT("  Average IoChunks per shadermap: %.2f\n")
		UTF8TEXT("  Maximum IoChunks per shadermap: %d\n")
		UTF8TEXT("------------------------------------------\n\n"),
		MinNumIoChunksPerShadermap.Get(0),
		AvgNumIoChunksPerShadermap,
		MaxNumIoChunksPerShadermap.Get(0)
	);

	// Write debug output to file
	Ar.Serialize(StatsOutput.GetCharArray().GetData(), StatsOutput.Len() * sizeof(UTF8CHAR));
	Ar.Serialize(DebugOutput.GetCharArray().GetData(), DebugOutput.Len() * sizeof(UTF8CHAR));

	return true;
}

bool FIoStoreShaderLibraryProcessor::AssignShaderArchives(const FContainerTargetFile& TargetFile)
{
	// Convert shader format and platform name from *.ushaderbytecode filename
	FShaderLibraryNameInfo LibraryNameInfo;
	if (!ParseIoStoreShaderLibraryName(LibraryNameInfo, *TargetFile.NormalizedSourcePath))
	{
		return false;
	}

	// Get container for current target file and extract ChunkId from it
	const FContainerTargetSpec* ContainerTarget = TargetFile.ContainerTarget;
	check(ContainerTarget);

	// Gather all package names in container target
	TSet<FName> PackagesInChunk;
	for (const FCookedPackage* CookedPackage : ContainerTarget->Packages)
	{
		PackagesInChunk.Add(CookedPackage->PackageName);
	}

	// Gather all chunk dependencies to deduplicate redundant shadermap entries
	FContainerChunkDependency* ContainerChunkDependency = ContainerTargetToChunkDependencies.Find(ContainerTarget);

	// Lambda to determine what shadermaps to exclude from current target file to deduplicate them across their chunk dependency tree
	auto ExcludeShaderMap = [this, ContainerChunkDependency, ContainerTarget](const FShaderHash& ShaderMapHash) -> bool
		{
			// Check if shadermap hash is already stored in any of the current container's chunk dependencies
			for (int32 ChunkDependencyId : ContainerChunkDependency->ChunkDependencies)
			{
				if (const FContainerShaderArchives* ContainerInfo = this->ChunkIdToContainerInfo.Find(ChunkDependencyId))
				{
					for (const FSerializedShaderArchiveBuffer* ShaderArchive : ContainerInfo->AssignedSerializedShaders)
					{
						if (ShaderArchive->ShaderMapHashes.Contains(ShaderMapHash))
						{
							// Found shadermap in a container that is a parent of the current container, so exclude this shadermap from assigning it to this one
							return true;
						}
					}
				}
			}
			return false;
		};

	// Build intermediate shader archive and assign relevant shaders from preloaded serialized shaders
	const FString FormatAndPlatformName = LibraryNameInfo.GetFormatAndPlatformName();

	// Detect whether this target file is the configured FormatTargetContainer override for its format.
	// Override targets pull every shader for the format, regardless of asset associations or chunk dependency tree,
	// and must not act as a dedup parent (since their content doesn't reflect "what packages in this chunk reference").
	bool bIsFormatOverrideTarget = false;
	if (FContainerTargetSpec* const* OverrideTarget = FormatToOverrideContainer.Find(FormatAndPlatformName))
	{
		bIsFormatOverrideTarget = (*OverrideTarget == ContainerTarget);
	}

	if (TUniquePtr<FEditorShaderCodeArchive>* FoundMonolithicShaderArchive = AllMonolithicShaderArchives.Find(FormatAndPlatformName))
	{
		FEditorShaderCodeArchive* MonolithicShaderArchive = FoundMonolithicShaderArchive->Get();
		check(MonolithicShaderArchive);

		TUniquePtr<FEditorShaderCodeArchive> NewChunkShaderArchive;
		if (bIsFormatOverrideTarget)
		{
			// Override path: take every asset and every shadermap for this format. The override container becomes
			// the sole destination for the format's shaders, so neither the per-package filter nor the parent-chunk
			// dedup applies. CreateArchiveFromFilteredAssets is private on FEditorShaderCodeArchive, so we go through
			// the public CreateNamedChunk and feed it the union of every package across every container -- which makes
			// the asset-name filter inside CreateNamedChunk pass for every shadermap referenced by any package.
			NewChunkShaderArchive = MonolithicShaderArchive->CreateNamedChunk(
				LibraryNameInfo.ChunkName, AllPackagesAcrossContainers);
		}
		else
		{
			// Normal path: create new chunk for packages.
			// All containers were added to ContainerTargetToChunkDependencies by the ProcessShaderLibraries's call to BuildContainerTargetChunkDependencies before calling AssignShaderArchives.
			FShaderMapAssetAssociations::FShaderMapFilterFunction ShaderMapFilterFunction =
				ensureMsgf(ContainerChunkDependency != nullptr, TEXT("Failed to find container chunk dependency information for container target '%s'"), *ContainerTarget->Name.ToString())
					? ExcludeShaderMap
					: FShaderMapAssetAssociations::FShaderMapFilterFunction(nullptr);
			NewChunkShaderArchive = MonolithicShaderArchive->CreateNamedChunk(LibraryNameInfo.ChunkName, PackagesInChunk, ShaderMapFilterFunction);
		}
		check(NewChunkShaderArchive);

		// Convert constructed intermediate shader archive to final serialized archive with flat buffer
		TUniquePtr<FSerializedShaderArchiveBuffer> AssignedShaderArchive = TUniquePtr<FSerializedShaderArchiveBuffer>(new FSerializedShaderArchiveBuffer());
		{
			if (AssignedShaderArchive->LoadFromEditorArchive(*NewChunkShaderArchive))
			{
				// Log stats and debug info for newly assigned shader archive. Without pakchunk override, this is only logged during cooking; see SaveShaderCodeChunk().
				NewChunkShaderArchive->DumpStatsAndDebugInfo();
			}
			else
			{
				UE_LOGF(LogIoStore, Error, "Could not serialize shaders from monolithic archive '%ls'", *LibraryNameInfo.GetLibraryName());
				return false;
			}

			// Link shader archive to current container chunk dependency if this is not an external container, i.e. no OnDemand or Optional ID suffix.
			// This information can be retrieved by subsequent shader archives in `ExcludeShaderMap` using their chunk dependency IDs
			// to check what shadermaps are already contained in their parent chunks to deduplicate them.
			// Override targets are excluded: they hold every shader for their format, not the natural per-package
			// subset, so they would mis-represent "what shadermaps a parent chunk normally contains".
			if (!bIsFormatOverrideTarget && ContainerChunkDependency && ContainerChunkDependency->ChunkId != INDEX_NONE && !ContainerChunkDependency->bIsExternalContainer)
			{
				FContainerShaderArchives& ContainerInfo = ChunkIdToContainerInfo.FindOrAdd(ContainerChunkDependency->ChunkId);
				ContainerInfo.AssignedSerializedShaders.Add(AssignedShaderArchive.Get());
			}

			// Keep track of old and new pakchunks if debug information is enabled
			if (IsChunkOverrideDebugEnabled())
			{
				AddShaderMapsToPakChunkDebugInfo(TargetFile, *AssignedShaderArchive, FName(FormatAndPlatformName), ChunkAssignment_New);
			}
		}
		AllAssignedSerializedShaders.Add(&TargetFile, MoveTemp(AssignedShaderArchive));
	}
	else
	{
		UE_LOGF(LogIoStore, Error, "Could not find monolithic shader archive for format '%ls'", *FormatAndPlatformName);
		return false;
	}
	return true;
}

#endif // WITH_EDITORONLY_DATA

void FIoStoreShaderLibraryProcessor::FlushCodeIoChunks(FContainerTargetSpec* ContainerTarget, FIoStoreShaderLibraryOutput& Output, FShaderInfo::EShaderType ShaderType)
{
	TSet<FShaderInfo*>& ContainerTargetShaderInfos = ContainerTargetToShaderLibrary.FindOrAdd(ContainerTarget);

	for (const FShaderGroupIoChunk& CodeChunk : CodeIoChunks)
	{
		const FIoChunkId& ShaderChunkId = CodeChunk.ChunkId;
		FShaderInfo* ShaderInfo = ChunkIdToShaderInfoMap.FindRef(ShaderChunkId);
		if (!ShaderInfo)
		{
			ShaderInfo = new FShaderInfo();
			ShaderInfo->ShaderGroupChunk = CodeChunk;
			Output.ShaderInfos.Add(ShaderInfo);
			ChunkIdToShaderInfoMap.Add(ShaderChunkId, ShaderInfo);
		}
		else
		{
			// first, make sure that the code is exactly the same
			if (ShaderInfo->ShaderGroupChunk.CodeIoBuffer != CodeChunk.CodeIoBuffer)
			{
				UE_LOGF(LogIoStore, Error, "Collision of two shader code chunks, same Id (%ls), different code. Packaged game will likely crash, not being able to decompress the shaders.", *LexToString(ShaderChunkId));
			}

			// If we already exist, then we have two separate LoadOrderFactors,
			// which one we got first affects build determinism. Take the lower.
			if (ShaderInfo->ShaderGroupChunk.LoadOrderFactor > CodeChunk.LoadOrderFactor)
			{
				ShaderInfo->ShaderGroupChunk.LoadOrderFactor = CodeChunk.LoadOrderFactor;
			}
		}

		const FShaderInfo::EShaderType* CurrentShaderTypeInContainer = ShaderInfo->TypeInContainer.Find(ContainerTarget);
		if (!CurrentShaderTypeInContainer || *CurrentShaderTypeInContainer != FShaderInfo::Global)
		{
			// If a shader is both global and shared consider it to be global
			ShaderInfo->TypeInContainer.Add(ContainerTarget, ShaderType);
		}

		ContainerTargetShaderInfos.Add(ShaderInfo);
	}

	for (FShaderGroupIoChunkMap& ShaderMap : CodeIoChunkMaps)
	{
		TArray<FIoChunkId>& ShaderMapChunkIds = ShaderMapToGroupChunkIds.FindOrAdd(ShaderMap.ShaderMapHash);
		ShaderMapChunkIds.Append(MoveTemp(ShaderMap.ShaderGroupChunkIds));
	}

	// Reset intermediate data for processing target files
	CodeIoChunks.Empty();
	CodeIoChunkMaps.Empty();
}

// Adds a new entry of type EContainerChunkType::ShaderCode to the specified container target
static void AddShaderCodeToContainerTarget(FContainerTargetSpec* ContainerTarget, const FShaderInfo& ShaderInfo)
{
	check(ContainerTarget);
	FContainerTargetFile& ShaderTargetFile = ContainerTarget->TargetFiles.AddDefaulted_GetRef();
	ShaderTargetFile.ContainerTarget = ContainerTarget;
	ShaderTargetFile.ChunkId = ShaderInfo.ShaderGroupChunk.ChunkId;
	ShaderTargetFile.ChunkType = EContainerChunkType::ShaderCode;
	ShaderTargetFile.bForceUncompressed = true;
	ShaderTargetFile.SourceBuffer.Emplace(ShaderInfo.ShaderGroupChunk.CodeIoBuffer);
	ShaderTargetFile.SourceSize = ShaderInfo.ShaderGroupChunk.CodeIoBuffer.DataSize();
}

void FIoStoreShaderLibraryProcessor::UpdatePackageStoreShaders(FCookedPackage* Package, const FName& ContainerTargetName)
{
	// 1. Update ShaderInfos with which packages we reference.
	// 2. Add to packages which shaders we use.
	// 3. Add to PackageStore what shaders we use.
	if (const FShaderMapAssetAssociations::FAssociatedAssetData* FindAssetData = ShaderMapAssetAssociations.FindAsset(Package->PackageName))
	{
		for (const FShaderHash& ShaderMapHash : FindAssetData->ShaderMaps)
		{
			const TArray<FIoChunkId>* FindChunkIds = ShaderMapToGroupChunkIds.Find(ShaderMapHash);
			if (!FindChunkIds)
			{
				UE_LOGF(LogIoStore, Warning, "Package '%ls' in '%ls' referencing missing shader map '%ls'", *Package->PackageName.ToString(), *ContainerTargetName.ToString(), *ShaderMapHash.ToString());
				continue;
			}
			Package->ShaderMapHashes.Add(ShaderMapHash);

			for (const FIoChunkId& ShaderChunkId : *FindChunkIds)
			{
				FShaderInfo* ShaderInfo = ChunkIdToShaderInfoMap.FindRef(ShaderChunkId);
				if (!ShaderInfo)
				{
					UE_LOGF(LogIoStore, Warning, "Package '%ls' in '%ls' referencing missing shader with chunk id '%ls'", *Package->PackageName.ToString(), *ContainerTargetName.ToString(), *LexToString(ShaderChunkId));
					continue;
				}

				check(ShaderInfo);
				ShaderInfo->ReferencedByPackages.Add(Package);
				Package->Shaders.AddUnique(ShaderInfo);
			}
		}
	}
	Algo::Sort(Package->ShaderMapHashes);
}

void FIoStoreShaderLibraryProcessor::AddAllReferencedShadersToContainerTarget(FContainerTargetSpec* ContainerTarget, FIoStoreShaderLibraryOutput& Output)
{
	const TSet<FShaderInfo*>* FoundContainerShaderLibraryShaders = ContainerTargetToShaderLibrary.Find(ContainerTarget);
	if (!FoundContainerShaderLibraryShaders)
	{
		// Early exit if package doesn't contain any shaders
		return;
	}

	for (FCookedPackage* Package : ContainerTarget->Packages)
	{
		for (FShaderInfo* ShaderInfo : Package->Shaders)
		{
			if (ShaderInfo->ReferencedByPackages.Num() == 1)
			{
				FShaderInfo::EShaderType* ShaderType = ShaderInfo->TypeInContainer.Find(ContainerTarget);
				if (ShaderType && *ShaderType != FShaderInfo::Global)
				{
					*ShaderType = FShaderInfo::Inline;
				}
			}
		}
	}

	for (FShaderInfo* ShaderInfo : *FoundContainerShaderLibraryShaders)
	{
		FShaderAssociationInfo::FShaderChunkInfoKey ShaderChunkInfoKey = { ContainerTarget->Name, ShaderInfo->ShaderGroupChunk.ChunkId };
		FShaderAssociationInfo::FShaderChunkInfo& ShaderChunkInfo = Output.AssocInfo.ShaderChunkInfos.Add(ShaderChunkInfoKey);

		const FShaderInfo::EShaderType* ShaderType = ShaderInfo->TypeInContainer.Find(ContainerTarget);
		check(ShaderType);
		if (*ShaderType == FShaderInfo::Global)
		{
			ContainerTarget->GlobalShaders.Add(ShaderInfo);
			ShaderChunkInfo.Type = FShaderAssociationInfo::FShaderChunkInfo::Global;
		}
		else if (*ShaderType == FShaderInfo::Inline)
		{
			ContainerTarget->InlineShaders.Add(ShaderInfo);

			ShaderChunkInfo.Type = FShaderAssociationInfo::FShaderChunkInfo::Package;
			checkf(ShaderInfo->ReferencedByPackages.Num() == 1, TEXT("Inline shader chunks must be referenced by 1 package only, but shader chunk %s is referenced by %d"),
				*LexToString(ShaderInfo->ShaderGroupChunk.ChunkId), ShaderInfo->ReferencedByPackages.Num());

			ShaderChunkInfo.ReferencedByPackages.Add(ShaderInfo->ReferencedByPackages[FSetElementId::FromInteger(0)]->PackageName);
		}
		else if (ShaderInfo->ReferencedByPackages.Num() > 1)
		{
			ContainerTarget->SharedShaders.Add(ShaderInfo);

			ShaderChunkInfo.Type = FShaderAssociationInfo::FShaderChunkInfo::Package;

			for (const FCookedPackage* Package : ShaderInfo->ReferencedByPackages)
			{
				ShaderChunkInfo.ReferencedByPackages.Add(Package->PackageName);

				Output.AssocInfo.AddPackageShaderInfoKey(Package->PackageName, ShaderChunkInfoKey);
			}
		}
		else
		{
			//
			// Note that we can get here with shaders that get split off in to another container (e.g. sm6 shaders). 
			// Since they are in a different pakChunk they can't get inlined or shared. However, they still "belong" to
			// the referencing packages for association purposes.
			//
			if (ShaderInfo->ReferencedByPackages.Num())
			{
				ShaderChunkInfo.Type = FShaderAssociationInfo::FShaderChunkInfo::Package;

				for (const FCookedPackage* Package : ShaderInfo->ReferencedByPackages)
				{
					ShaderChunkInfo.ReferencedByPackages.Add(Package->PackageName);

					Output.AssocInfo.AddPackageShaderInfoKey(Package->PackageName, ShaderChunkInfoKey);
				}
			}
			else
			{
				ShaderChunkInfo.Type = FShaderAssociationInfo::FShaderChunkInfo::Orphan;
			}

			ContainerTarget->UniqueShaders.Add(ShaderInfo);
		}
		AddShaderCodeToContainerTarget(ContainerTarget, *ShaderInfo);
	}
}

} // namespace UE::IoStore::Private
