// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookMetadata.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "IoStoreUtilitiesCore.h"
#include "IO/IoChunkId.h"
#include "Misc/CoreMiscDefines.h"
#include "Hash/ShaderHash.h"
#include "ShaderMapAssetAssociation.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

struct FShaderLibraryNameInfo;
#if WITH_EDITORONLY_DATA
class FEditorShaderCodeArchive;
#endif

namespace UE::IoStore::Private
{

struct FContainerTargetSpec;
struct FContainerTargetFile;
struct FCookedPackage;
struct FIoStoreArguments;
struct FShaderInfo;

// Carries association between shaders and packages from shader processing to size assignment.
struct FShaderAssociationInfo
{
	struct FShaderChunkInfo
	{
		enum EType
		{
			// This shader is referenced by one or more packages and can assign its size to
			// those packages
			Package,
			// This shader is needed by the baseline engine in some manner and is a flat
			// size cost.
			Global,
			// This shader isn't global or referenced by packages - theoretically shouldn't
			// exist.
			Orphan
		};

		EType Type;
		TArray<FName> ReferencedByPackages;
		uint32 CompressedSize = 0;
		UE::Cook::EPluginSizeTypes SizeType;

		// If we are shared across plugins, this is our pseudo plugin name. This is generated during size assignment.
		FString SharedPluginName;
	};

	struct FShaderChunkInfoKey
	{
		FName PakChunkName;
		FIoChunkId IoChunkId;

		friend uint32 GetTypeHash(const FShaderChunkInfoKey& Key)
		{
			return GetTypeHash(Key.IoChunkId);
		}
		friend bool operator == (const FShaderChunkInfoKey& LHS, const FShaderChunkInfoKey& RHS)
		{
			return LHS.PakChunkName == RHS.PakChunkName && LHS.IoChunkId == RHS.IoChunkId;
		}
	};

	TMap<FShaderChunkInfoKey, FShaderChunkInfo> ShaderChunkInfos;

	// The list of shaders referenced by each package. This can be used to look up in to ContainerShaderInfos.
	TMap<FName /* PackageName */, TArray<FShaderChunkInfoKey>> PackageShaderMap;

	// Adds a shader chunk info key to the PackageShaderMap for the specified package name.
	FORCEINLINE void AddPackageShaderInfoKey(const FName& PackageName, const FShaderChunkInfoKey& InfoKey)
	{
		TArray<FShaderAssociationInfo::FShaderChunkInfoKey>& PackageShaders = PackageShaderMap.FindOrAdd(PackageName);
		PackageShaders.Add(InfoKey);
	}
};

//@lh-todo
// Move this information into FContainerSourceSpec and FContainerTargetSpec.
// This information should be determined in `ParseContainerGenerationArguments()` when the source containers are built,
// and then transferred to the target containers in `InitializeContainerTargetsAndPackages()`.
// More testing beyond shaders for this is required, so keep this information private to the shader library processor for now.
// Same goes for `IsTargetFileGlobalShaderLibrary()`, which should be determined as soon as the source container is generated
// instead of parsing the filename later on to extract those flags.
struct FContainerChunkDependency
{
	int32 ChunkId = INDEX_NONE;
	bool bIsExternalContainer = false; // Is the container loaded externally? E.g. pakchunk100iad, pakchunk100ondemand, etc.
	TSet<int32> ChunkDependencies;
};

struct FIoStoreShaderLibraryOutput
{
	TArray<FShaderInfo*> ShaderInfos;
	FShaderAssociationInfo AssocInfo;
};

class FSerializedShaderArchiveBuffer;

// Processor to load cooked shader libraries and package them into IoStore entries
class FIoStoreShaderLibraryProcessor
{
	// Shader map hash with its IoChunk Ids of all its shader groups
	struct FShaderGroupIoChunkMap
	{
		FShaderHash ShaderMapHash;
		TArray<FIoChunkId> ShaderGroupChunkIds;
	};

	struct FContainerShaderArchives
	{
		TArray<FSerializedShaderArchiveBuffer*> AssignedSerializedShaders;
	};

	// Input arguments
	FCookedPackageStore* PackageStore = nullptr;
	FOodleDataCompression::ECompressor ShaderOodleCompressor = FOodleDataCompression::ECompressor::NotSet;
	FOodleDataCompression::ECompressionLevel ShaderOodleLevel = FOodleDataCompression::ECompressionLevel::None;

	// Persitent data for entire shader library processing
	TMap<FIoChunkId, FShaderInfo*> ChunkIdToShaderInfoMap;
	TMap<FShaderHash, TArray<FIoChunkId>> ShaderMapToGroupChunkIds; // Maps from shader map hash to its set of shader group IoChunk Ids
	TMap<FContainerTargetSpec*, TSet<FShaderInfo*>> ContainerTargetToShaderLibrary;
	FShaderMapAssetAssociations ShaderMapAssetAssociations;
	TMap<const FContainerTargetSpec*, FContainerChunkDependency> ContainerTargetToChunkDependencies;

	// Intermediate data while processing a subset of target files
	TArray<FShaderGroupIoChunk> CodeIoChunks;
	TArray<FShaderGroupIoChunkMap> CodeIoChunkMaps; // Maps from shadermap index to shader group IoChunks

#if WITH_EDITORONLY_DATA
	// Intermediate data for chunk re-assignment
	TMap<FString, TUniquePtr<FEditorShaderCodeArchive>> AllMonolithicShaderArchives; // Shader format to archive
	TMap<const FContainerTargetFile*, TUniquePtr<FSerializedShaderArchiveBuffer>> AllAssignedSerializedShaders;
	TMap<int32, FContainerShaderArchives> ChunkIdToContainerInfo; // Maps ChunkId to the container information with all their assigned shader archives

	// Resolved [ShaderCodeLibrary.PakChunkOverride] +FormatTargetContainer entries. Maps a shader Format-Platform string
	// (e.g. "PCD3D_SM5-PCD3D_SM5") to the container its shaders should be redirected into. Populated by
	// ResolveFormatTargetContainerOverrides at the start of ProcessShaderLibraries when the chunk override is enabled.
	TMap<FString, FContainerTargetSpec*> FormatToOverrideContainer;

	// Union of every package across every container, used as the "include all" asset filter when assigning shaders into
	// a FormatTargetContainer override target. Populated alongside FormatToOverrideContainer; empty when no overrides exist.
	TSet<FName> AllPackagesAcrossContainers;

	// Debug information (used with CVar `r.ShaderCodeLibrary.PakChunkOverride.DumpDebugInfo`)
	enum EChunkAssignmentType
	{
		ChunkAssignment_Old = (1 << 0),
		ChunkAssignment_New = (1 << 1),
	};
	struct FChunkAssignmentInfo
	{
		TArray<FName> OldPakChunks;
		TArray<FName> NewPakChunks;
	};
	struct FMonolithicArchiveDebugInfo
	{
		TMap<FShaderHash, FChunkAssignmentInfo> ShaderMaps;
	};
	TMap<FName, FMonolithicArchiveDebugInfo> ChunkAssignmentDebugInfo;
#endif

public:
	FIoStoreShaderLibraryProcessor();
	~FIoStoreShaderLibraryProcessor();

	void ProcessShaderLibraries(const FIoStoreArguments& Arguments, TArrayView<FContainerTargetSpec*> ContainerTargets, FIoStoreShaderLibraryOutput& Output);

private:
	bool ProcessShaderLibraryFromTargetFile(const FIoStoreArguments& Arguments, FContainerTargetFile& TargetFile, FIoStoreShaderLibraryOutput& Output, const FString& ProjectDir);

#if WITH_EDITORONLY_DATA
	// Reads [ShaderCodeLibrary.PakChunkOverride] +FormatTargetContainer entries from GEngineIni, resolves each
	// referenced container against ContainerTarget->Name, validates that the configured format
	// is one this cook actually produces, and populates FormatToOverrideContainer. Rules whose format isn't present
	// in InFormatAndPlatformNames are dropped with a Warning -- catches typos and stray cross-platform config.
	void ResolveFormatTargetContainerOverrides(TArrayView<FContainerTargetSpec*> ContainerTargets, TArrayView<const FString> InFormatAndPlatformNames);

	// Builds the chunk dependency information for the specified container target. See ContainerTargetToChunkDependencies.
	void BuildContainerTargetChunkDependencies(const FContainerTargetSpec* ContainerTarget);

	// Preloads the *.ushaderbytecode for the specified target file for later chunk assignment.
	bool PreloadShaderArchive(const FContainerTargetFile& TargetFile);

	// Assigns preloaded shader archives to their respective target file (i.e. to their respective pakchunks).
	bool AssignShaderArchives(const FContainerTargetFile& TargetFile);

	void AddShaderMapsToPakChunkDebugInfo(const FContainerTargetFile& TargetFile, const FSerializedShaderArchiveBuffer& SerializedShaders,
		FName FormatAndPlatformName, int32 ChunkAssignmentType);

	// Logs statistics about how many shadermaps have been duplicated and deduplicated.
	void LogChunkOverrideStats(FName FormatAndPlatformName);

	// Writes out a debug file with information about all the shaders that have been re-assigned
	// to a difference chunk (if r.ShaderCodeLibrary.PakChunkOverride is enabled).
	bool DumpChunkOverrideDebugInfo(FName FormatAndPlatformName, FArchive& Ar);
#endif // WITH_EDITORONLY_DATA

	// Loads the serialized shaders of the *.ushaderbytecode file of the specified target.
	bool LoadSerializedShaders(const FContainerTargetFile& TargetFile, FSerializedShaderArchiveBuffer& OutSerializedShaders, bool bIncludeTypeInfo = false, FShaderLibraryNameInfo* OutNameInfo = nullptr);

	// Converts the *.ushaderbytecode file of the specified target to an IoStore shader code library.
	bool ConvertToIoStoreShaderLibrary(FContainerTargetFile& TargetFile, const FSerializedShaderArchiveBuffer& SerializedShaders, const FString& ProjectDir);

	// Flushes all generated shader code IoChunks to the output.
	void FlushCodeIoChunks(FContainerTargetSpec* ContainerTarget, FIoStoreShaderLibraryOutput& Output, FShaderInfo::EShaderType ShaderType);

	void UpdatePackageStoreShaders(FCookedPackage* Package, const FName& ContainerTargetName);
	void AddAllReferencedShadersToContainerTarget(FContainerTargetSpec* ContainerTarget, FIoStoreShaderLibraryOutput& Output);

};

} // namespace UE::IoStore::Private
