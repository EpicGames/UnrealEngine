// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreUtilities.h"
#include "IoStoreUtilitiesCore.h"
#include "IoStoreWriter.h"
#include "IoStoreShaderLibraryProcessor.h"

#include "IoStoreLooseFiles.h"
#include "Algo/BinarySearch.h"
#include "Algo/TopologicalSort.h"
#include "Async/AsyncWork.h"
#include "CookedPackageStore.h"
#include "CookMetadata.h"
#include "CookMetadataFiles.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheInterface.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMemory.h"
#include "Hash/CityHash.h"
#include "Hash/xxhash.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "IO/IoContainerMeta.h"
#include "IO/IoDispatcher.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Misc/Base64.h"
#include "Misc/AES.h"
#include "Misc/CoreDelegates.h"
#include "Misc/KeyChainUtilities.h"
#include "Misc/WildcardString.h"
#include "Modules/ModuleManager.h"
#include "Serialization/Archive.h"
#include "Serialization/BulkData.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/BufferWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/AsyncLoading2.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Settings/ProjectPackagingSettings.h" // for EAssetRegistryWritebackMethod
#include "Serialization/PackageStore.h"
#include "UObject/Class.h"
#include "UObject/NameBatchSerialization.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/ObjectResource.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopeLock.h"
#include "Async/ParallelFor.h"
#include "Async/AsyncFileHandle.h"
#include "Async/Async.h"
#include "RSA.h"
#include "Misc/AssetRegistryInterface.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/FeedbackContext.h"
#include "Serialization/LargeMemoryReader.h"
#include "Misc/StringBuilder.h"
#include "Async/Future.h"
#include "Algo/MaxElement.h"
#include "Algo/Sort.h"
#include "Algo/StableSort.h"
#include "Algo/IsSorted.h"
#include "PackageStoreOptimizer.h"
#include "ShaderCodeArchive.h"
#include "ZenStoreHttpClient.h"
#include "IPlatformFilePak.h"
#include "ZenCookArtifactReader.h"
#include "ZenStoreWriter.h"
#include "IO/IoContainerHeader.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "IO/IoStore.h"
#include "ZenFileSystemManifest.h"
#include "IPlatformFileSandboxWrapper.h"
#include "Misc/PathViews.h"
#include "HAL/FileManagerGeneric.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/ZenPackageHeader.h"
#include "String/ParseTokens.h"
#include "StudioTelemetry.h"
#include "Tasks/Task.h"
#include "Templates/Greater.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, IoStoreUtilities);

using namespace UE::IoStore::Private;

#define IOSTORE_CPU_SCOPE(NAME) TRACE_CPUPROFILER_EVENT_SCOPE(IoStore##NAME);
#define IOSTORE_CPU_SCOPE_DATA(NAME, DATA) TRACE_CPUPROFILER_EVENT_SCOPE(IoStore##NAME);

TRACE_DECLARE_MEMORY_COUNTER(IoStoreSourceReadsUsedBufferMemory, TEXT("IoStoreWriter/SourceReadsUsedBufferMemory"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreSourceReadsInflight, TEXT("IoStoreWriter/SourceReadsInflight"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreSourceReadsDone, TEXT("IoStoreWriter/SourceReadsDone"));

// Helper to format numbers with comma separators to help readability: 1,234 vs 1234.
static FString NumberString(uint64 N) { return FText::AsNumber(N).ToString(); }

// Used for tracking how the chunk will get deployed and thus what classification its size is.
struct FIoStoreChunkSource
{
	FIoStoreTocChunkInfo ChunkInfo;
	UE::Cook::EPluginSizeTypes SizeType;
};

static FPackageId PackageIdFromChunkId(const FIoChunkId& ChunkId)
{
	return FPackageId::FromValue(*(int64*)(ChunkId.GetData()));
}

/** Utility to format the number of bytes to a string. The units used will scale with the number of bytes. */
static FString FormatMemoryString(uint64 Bytes)
{
	if (Bytes < 1024)
	{
		return FString::Printf(TEXT("%8u   B"), (uint32)Bytes);
	}
	else if (Bytes < 1024 * 1024)
	{
		return FString::Printf(TEXT("%8.2f KiB"), (double)Bytes / 1024.0);
	}
	else if (Bytes < 1024 * 1024 * 1024)
	{
		return FString::Printf(TEXT("%8.2f MiB"), (double)Bytes / 1024.0 / 1024.0);
	}
	else
	{
		return FString::Printf(TEXT("%8.2f GiB"), (double)Bytes / 1024.0 / 1024.0 / 1024.0);
	}
}

static const FName DefaultCompressionMethod = NAME_Zlib;
static const uint64 DefaultCompressionBlockSize = 64 << 10;
static const uint64 DefaultCompressionBlockAlignment = 64 << 10;
static const uint64 DefaultMemoryMappingAlignment = 16 << 10;

static TUniquePtr<FIoStoreReader> CreateIoStoreReader(const TCHAR* Path, const FKeyChain& KeyChain);

/*
* Provides access to previously compressed chunks to the iostore writer, allowing
* a) avoiding recompressing things and b) tweaks to compressors dont cause massive patches.
*/
class FIoStoreChunkDatabase : public IIoStoreWriterReferenceChunkDatabase
{
public:
	static const uint8 IoChunkTypeCount = (uint8)EIoChunkType::MAX;

	struct FRequestedChunkInfo
	{
		FIoChunkId Id;
		FIoHash ChunkHash;
	};

	TArray<TUniquePtr<FIoStoreReader>> Readers;
	struct FReaderChunks
	{
		FString ContainerPathAndFileName;
		FString ContainerName;
		FIoContainerId ContainerId;

		int32 ReaderIndex = -1;

		TMap<FIoChunkId, uint32> ChunkIds; // maps to index for chunk specific structures

		TArray<FIoHash> ChangedChunkHashes;
		TArray<FIoStoreTocChunkInfo> ChunkInfos;
		TArray<std::atomic_uint8_t> ChunkChanged;

		~FReaderChunks() = default;
		FReaderChunks() = default;
		FReaderChunks(FReaderChunks&&) = delete;
		FReaderChunks(FReaderChunks&) = delete;
		FReaderChunks& operator=(FReaderChunks&) = delete;

		FCriticalSection NewChunksLock;
		TArray<FRequestedChunkInfo> NewChunks;
		uint64 NewChunksByType[IoChunkTypeCount] = { 0 };

		std::atomic_uint64_t FulfillBytes = 0;
		std::atomic_uint64_t TotalRequestCount = 0;
		std::atomic_uint64_t ChangedCountByType[IoChunkTypeCount] = {0};
		std::atomic_uint64_t UsedChunksByType[IoChunkTypeCount] = {0};
	};

	struct FMissingContainerInfo
	{
		FMissingContainerInfo(FString InContainerName) {ContainerName = InContainerName;}
		TArray<FRequestedChunkInfo> Chunks;
		FCriticalSection ChunksLock;

		FString ContainerName;
		std::atomic_uint64_t TotalRequestCount = 0;
		std::atomic_uint64_t RequestedChunkCount[IoChunkTypeCount] = {0};
	};


	// Since we are notified of all containers, every id should be in our database,
	// or in the missing list.
	TMap<FIoContainerId, TUniquePtr<FReaderChunks>> ChunkDatabase;
	TMap<FIoContainerId, TUniquePtr<FMissingContainerInfo>> MissingContainers;

	std::atomic_int64_t MatchCount = 0;
	std::atomic_int64_t RequestCount = 0;
	int32 ContainerNotFound = 0;

	// Bytes we actually delivered to the iostore writer
	int64 FulfillBytes = 0;
	int64 FulfillBytesPerChunk[(int8)EIoChunkType::MAX] = {};

	uint32 CompressionBlockSize = 0;
	bool bValid = false;

	bool Init(FString InGlobalContainerFileName, FString InAdditionalContainersPath, const FKeyChain& InDecryptionKeychain)
	{
		double StartTime = FPlatformTime::Seconds();

		// Try and catch common pathing mistakes since honestly we only care about the path even though we ask
		// for the global.utoc.
		FString ContainersDirectory = FPaths::GetPath(MoveTemp(InGlobalContainerFileName));
		FPaths::NormalizeDirectoryName(ContainersDirectory);
		FString GlobalFileName = ContainersDirectory / TEXT("global.utoc");

		TUniquePtr<FIoStoreReader> GlobalReader = CreateIoStoreReader(*GlobalFileName, InDecryptionKeychain);
		if (GlobalReader.IsValid() == false)
		{
			UE_LOGF(LogIoStore, Warning, "Failed to open reference chunk container %ls", *GlobalFileName);
			return false;
		}

		FPaths::NormalizeDirectoryName(InAdditionalContainersPath);

		TArray<FString> ContainerFilePaths;

		TArray<FString> FoundContainerFiles;
		{
			IFileManager::Get().FindFiles(FoundContainerFiles, *(ContainersDirectory / TEXT("*.utoc")), true, false);
			for (const FString& Filename : FoundContainerFiles)
			{
				ContainerFilePaths.Emplace(ContainersDirectory / Filename);
			}
		}

		if (InAdditionalContainersPath.Len())
		{
			FoundContainerFiles.Empty();
			IFileManager::Get().FindFiles(FoundContainerFiles, *(InAdditionalContainersPath / TEXT("*.utoc")), true, false);
			for (const FString& Filename : FoundContainerFiles)
			{
				ContainerFilePaths.Emplace(InAdditionalContainersPath / Filename);
			}
		}

		CompressionBlockSize = 0;
		int64 IoChunkCount = 0;
		for (FString& ContainerFilePath : ContainerFilePaths)
		{
			TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*ContainerFilePath, InDecryptionKeychain);
			if (Reader.IsValid() == false)
			{
				UE_LOGF(LogIoStore, Warning, "Failed to open reference chunk container %ls", *ContainerFilePath);
				return false;
			}

			// Work around the fact that we recently changed container ids to be unique and detect .o containers
			// and regenerate the Id.
			FIoContainerId ContainerId = Reader->GetContainerId();
			if (ContainerFilePath.Contains(FPackagePath::GetOptionalSegmentExtensionModifier()))
			{
				FIoContainerId CheckContainerId = FIoContainerId::FromName(*FPaths::GetBaseFilename(ContainerFilePath));
				if (CheckContainerId != ContainerId)
				{
					UE_LOGF(LogIoStore, Display, "Reference Chunk: adjusting optional segment container id (0x%llx -> 0x%llx) for %ls", ContainerId.Value(), CheckContainerId.Value(), *ContainerFilePath);
					ContainerId = CheckContainerId;
				}
			}

			TUniquePtr<FReaderChunks>* ExistingContainer = ChunkDatabase.Find(ContainerId);
			if (ExistingContainer)
			{
				UE_LOGF(LogIoStore, Error, "Reference chunk database loaded containers with the same id:");
				UE_LOGF(LogIoStore, Error, "    %ls", *ExistingContainer[0]->ContainerPathAndFileName);
				UE_LOGF(LogIoStore, Error, "    %ls", *ContainerFilePath);
				return false;
			}


			TUniquePtr<FReaderChunks> ReaderChunks(new FReaderChunks());
			ReaderChunks->ContainerName = Reader->GetContainerName();
			ReaderChunks->ContainerPathAndFileName = MoveTemp(ContainerFilePath);
			ReaderChunks->ContainerId = ContainerId;

			int32 ChunkCount = Reader->GetChunkCount();
			ReaderChunks->ChunkInfos.Reserve(ChunkCount);
			ReaderChunks->ChangedChunkHashes.SetNumZeroed(ChunkCount);

			uint32 ChunkIndex = 0;
			Reader->EnumerateChunks([ReaderChunks = ReaderChunks.Get(), &ChunkIndex](FIoStoreTocChunkInfo&& ChunkInfo)
			{
				ReaderChunks->ChunkIds.Add(ChunkInfo.Id, ChunkIndex);
				ReaderChunks->ChunkInfos.Add(MoveTemp(ChunkInfo));
				ChunkIndex++;
				return true;
			});

			ReaderChunks->ChunkChanged.SetNumZeroed(ReaderChunks->ChunkIds.Num());

			if (Readers.Num() == 0)
			{
				CompressionBlockSize = Reader->GetCompressionBlockSize();
			}
			else if (Reader->GetCompressionBlockSize() != CompressionBlockSize)
			{
				UE_LOGF(LogIoStore, Warning, "Reference chunk containers had different compression block sizes, failing to init reference db (%u and %u)", CompressionBlockSize, Reader->GetCompressionBlockSize());
				return false;
			}

			IoChunkCount += ChunkCount;
			ReaderChunks->ReaderIndex = Readers.Num();

			ChunkDatabase.Add(ReaderChunks->ContainerId, MoveTemp(ReaderChunks));
			Readers.Add(MoveTemp(Reader));
		}

		UE_LOGF(LogIoStore, Display, "Reference Chunk loaded %d containers and %ls chunks, in %.1f seconds", Readers.Num(), *FText::AsNumber(IoChunkCount).ToString(), FPlatformTime::Seconds() - StartTime);
		bValid = true;
		return true;
	}

	virtual void NotifyAddedToWriter(const FIoContainerId& InContainerId, const FString& InContainerName) override
	{
		// If we don't have this container in our chunk database, add it to a tracking
		// structure so we can see how many chunks we're missing.
		if (ChunkDatabase.Contains(InContainerId) == false)
		{
			TUniquePtr<FMissingContainerInfo> MissingContainer = MakeUnique<FMissingContainerInfo>(InContainerName);
			MissingContainers.Add(InContainerId, MoveTemp(MissingContainer));
		}
	}

	virtual bool GetContainerFilePath(const FIoContainerId& InContainerId, FString& OutContainerFilePath) override
	{
		TUniquePtr<FReaderChunks>* ReaderChunksPtr = ChunkDatabase.Find(InContainerId);
		if (ReaderChunksPtr == nullptr)
		{
			return false;
		}

		OutContainerFilePath = (*ReaderChunksPtr)->ContainerPathAndFileName;
		return true;
	}

	virtual uint32 GetCompressionBlockSize() const override
	{
		return CompressionBlockSize;
	}

	// Returns whether we expect to be able to load the chunk from the reference chunk database.
	// This can be called from any thread, though in the presence of existing hashes it's single threaded.
	virtual bool ChunkExists(const FIoContainerId& InContainerId, const FIoHash& InChunkHash, const FIoChunkId& InChunkId, int32& OutNumChunkBlocks)
	{
		if (!bValid)
		{
			return false;
		}
		RequestCount.fetch_add(1, std::memory_order_relaxed);

		uint8 ChunkType = (uint8)InChunkId.GetChunkType();

		FRequestedChunkInfo RCI;
		RCI.ChunkHash = InChunkHash;
		RCI.Id = InChunkId;

		TUniquePtr<FReaderChunks>* ReaderChunksPtr = ChunkDatabase.Find(InContainerId);
		if (ReaderChunksPtr == nullptr)
		{
			// Container doesn't exist - could have been added/created or maybe wrong source project.
			TUniquePtr<FMissingContainerInfo>* MissingContainerPtr = MissingContainers.Find(InContainerId);

			if (MissingContainerPtr == nullptr)
			{
				UE_LOGF(LogIoStore, Warning, "We got a container that was never added! Id = 0x%llx", InContainerId.Value());
			}
			else
			{
				// We expect most chunks to have containers so we don't stress about this lock too much
				FMissingContainerInfo* MissingContainer = MissingContainerPtr->Get();
				{
					FScopeLock Locker(&MissingContainer->ChunksLock);
					MissingContainer->Chunks.Add(MoveTemp(RCI));
				}

				MissingContainer->RequestedChunkCount[ChunkType].fetch_add(1, std::memory_order_relaxed);
				if (MissingContainer->TotalRequestCount.fetch_add(1, std::memory_order_relaxed) == 0)
				{
					ContainerNotFound++;
				}
			}

			return false;
		}

		// We have the container at least.
		FReaderChunks* ReaderChunks = ReaderChunksPtr->Get();
		ReaderChunks->TotalRequestCount.fetch_add(1, std::memory_order_relaxed);

		uint32_t* ChunkIndex = ReaderChunks->ChunkIds.Find(InChunkId);
		if (ChunkIndex == nullptr)
		{
			// Chunk doesn't exist - it's entirely new.
			// We expect most chunks to already exist so we don't stress about this lock too much
			FScopeLock Locker(&ReaderChunks->NewChunksLock);
			ReaderChunks->NewChunksByType[ChunkType]++;
			ReaderChunks->NewChunks.Add(MoveTemp(RCI));
			return false;
		}

		// We have the chunk - does the hash match?
		const FIoStoreTocChunkInfo& ChunkInfo = ReaderChunks->ChunkInfos[*ChunkIndex];
		if (ChunkInfo.ChunkHash != InChunkHash)
		{
			// Chunk exists, but it's changed.
			ReaderChunks->ChangedCountByType[ChunkType].fetch_add(1, std::memory_order_relaxed);
			ReaderChunks->ChunkChanged[*ChunkIndex].store(1, std::memory_order_relaxed);

			// This should be safe because chunk ids are unique within a container,
			// so we are the only ones poking at this index.
			ReaderChunks->ChangedChunkHashes[*ChunkIndex] = MoveTemp(RCI.ChunkHash);
			return false;
		}

		// We match.
		MatchCount.fetch_add(1, std::memory_order_relaxed);
		ReaderChunks->UsedChunksByType[ChunkType].fetch_add(1, std::memory_order_relaxed);
		OutNumChunkBlocks = IntCastChecked<int32>(ChunkInfo.NumCompressedBlocks);
		return true;
	}


	// This function was not written to be thread safe as it's only ever called from
	// the iostore begindispatch thread (i.e. is not async)
	virtual UE::Tasks::FTask RetrieveChunk(const FIoContainerId& InContainerId, const FIoHash& InChunkHash, const FIoChunkId& InChunkId, TUniqueFunction<void(TIoStatusOr<FIoStoreCompressedReadResult>)> InCompleteCallback)
	{
		if (!bValid)
		{
			InCompleteCallback(FIoStatus(EIoErrorCode::InvalidCode, TEXT("IoStoreChunkDatabase not initialized")));
			return UE::Tasks::MakeCompletedTask<void>();
		}

		TUniquePtr<FReaderChunks>* ReaderChunksPtr = ChunkDatabase.Find(InContainerId);
		if (ReaderChunksPtr == nullptr)
		{
			// This should never happen now as we wrap this in a ChunkExists call.
			InCompleteCallback(FIoStatus(EIoErrorCode::InvalidCode, TEXT("RetrieveChunk can't find the container - invariant violated!")));
			return UE::Tasks::MakeCompletedTask<void>();
		}

		FReaderChunks* ReaderChunks = ReaderChunksPtr->Get();

		uint32_t* ChunkIndex = ReaderChunks->ChunkIds.Find(InChunkId);
		if (ChunkIndex == nullptr)
		{
			// This should never happen now as we wrap this in a ChunkExists call.
			InCompleteCallback(FIoStatus(EIoErrorCode::InvalidCode, TEXT("RetrieveChunk can't find the chunk - invariant violated!")));
			return UE::Tasks::MakeCompletedTask<void>();
		}

		FIoStoreTocChunkInfo& ChunkInfo = ReaderChunks->ChunkInfos[*ChunkIndex];

		uint64 TotalCompressedSize = 0;
		uint64 TotalUncompressedSize = 0;
		uint32 CompressedBlockCount = 0;
		Readers[ReaderChunks->ReaderIndex]->EnumerateCompressedBlocksForChunk(InChunkId, [&TotalUncompressedSize, &CompressedBlockCount, &TotalCompressedSize](const FIoStoreTocCompressedBlockInfo& BlockInfo)
		{			
			TotalCompressedSize += BlockInfo.CompressedSize;
			TotalUncompressedSize += BlockInfo.UncompressedSize;
			CompressedBlockCount ++;
			return true;
		});

		FulfillBytesPerChunk[(int8)ChunkInfo.ChunkType] += TotalCompressedSize;
		FulfillBytes += TotalCompressedSize;
		ReaderChunks->FulfillBytes.fetch_add(TotalCompressedSize, std::memory_order::relaxed);

		//
		// At this point we know we can use the block so we can go async.
		//
		return UE::Tasks::Launch(TEXT("ReadCompressed"), [this, Id = InChunkId, ReaderIndex = ReaderChunks->ReaderIndex, CompleteCallback = MoveTemp(InCompleteCallback)]()
		{
			TIoStatusOr<FIoStoreCompressedReadResult> Result = Readers[ReaderIndex]->ReadCompressed(Id, FIoReadOptions());
			CompleteCallback(Result);
		}, UE::Tasks::ETaskPriority::Normal);
	}

	void WriteCSV(const FString& InOutputFileName, const TArray<struct FCookedPackage*>& InPackages);

	void LogSummary(const TArray<FIoStoreWriterResult>& IoStoreWriterResults)
	{
		if (!bValid)
		{
			return;
		}

		// In order to try and get a number that's closer to the actual patch size, we need
		// to strip out the data that isn't actually distributed with the same and is instead
		// streamed. This is OptionalBulkData and currently is only recognizable here by scanning
		// the container name. (this is NOT OptionalSegment!)

		uint64 TotalEntryBytes = 0;
		uint64 TotalMissBytes = 0;
		uint64 TotalOptionalMissBytes = 0;

		for (const FIoStoreWriterResult& Result : IoStoreWriterResults)
		{
			bool bIsOptional = Result.ContainerName.Contains(TEXT("optional"));

			TotalEntryBytes += Result.TotalEntryCompressedSize;
			TotalMissBytes += Result.ReferenceCacheMissBytes;

			if (bIsOptional)
			{
				TotalOptionalMissBytes += Result.ReferenceCacheMissBytes;
			}
		}

		uint64 FulfillBytesOptional = 0;
		for (const TPair<FIoContainerId, TUniquePtr<FReaderChunks>>& ReaderPair : ChunkDatabase)
		{
			bool bIsOptional = ReaderPair.Value->ContainerName.Contains(TEXT("optional"));
			if (bIsOptional)
			{
				FulfillBytesOptional += ReaderPair.Value->FulfillBytes.load();
			}
		}

		uint64 TotalCandidateBytes = FulfillBytes + TotalMissBytes;
		uint64 TotalOptionalCandidateBytes = FulfillBytesOptional + TotalOptionalMissBytes;

		uint64 TotalNonOptCandidateBytes = TotalCandidateBytes - TotalOptionalCandidateBytes;
		uint64 TotalNonOptFulfillBytes = FulfillBytes - FulfillBytesOptional;

		UE_LOGF(LogIoStore, Display, "Reference Chunk Database:");
		UE_LOGF(LogIoStore, Display, "    %ls reused bytes out of %ls candidate bytes - %.1f%% hit rate.",
			*NumberString(FulfillBytes),
			*NumberString(TotalCandidateBytes),
			100.0 * FulfillBytes / (TotalCandidateBytes));
		if (TotalOptionalCandidateBytes)
		{
			UE_LOGF(LogIoStore, Display, "    %ls reused non-optional bytes out of %ls candidate non-optional bytes - %.1f%% hit rate.",
				*NumberString(TotalNonOptFulfillBytes),
				*NumberString(TotalNonOptCandidateBytes),
				100.0 * TotalNonOptFulfillBytes / (TotalNonOptCandidateBytes));
		}
		UE_LOGF(LogIoStore, Display, "    %ls candidate bytes out of %ls io chunk bytes - %.1f%% coverage.",
			*NumberString(TotalCandidateBytes),
			*NumberString(TotalEntryBytes),
			100.0 * TotalCandidateBytes / (TotalEntryBytes));

		UE_LOGF(LogIoStore, Display, "    %ls chunks matched out of %ls requests", 
			*NumberString(MatchCount.load(std::memory_order_relaxed)),
			*NumberString(RequestCount.load(std::memory_order_relaxed)));

		FString ChunkNames[FIoStoreChunkDatabase::IoChunkTypeCount];
		for (uint8 TypeIndex = 0; TypeIndex < FIoStoreChunkDatabase::IoChunkTypeCount; TypeIndex++)
		{
			ChunkNames[TypeIndex] = LexToString((EIoChunkType)TypeIndex);
		}

		ChunkDatabase.ValueSort([](const TUniquePtr<FReaderChunks>& A, const TUniquePtr<FReaderChunks>& B)
		{
			return A->TotalRequestCount.load(std::memory_order_relaxed) > B->TotalRequestCount.load(std::memory_order_relaxed);
		});

		for (const TPair<FIoContainerId, TUniquePtr<FReaderChunks>>& ReaderPair: ChunkDatabase)
		{
			FReaderChunks* Reader = ReaderPair.Value.Get();

			for (uint8 i = 0; i < FIoStoreChunkDatabase::IoChunkTypeCount; i++)
			{
				if (Reader->ChangedCountByType[i] ||
					Reader->NewChunksByType[i] ||
					Reader->UsedChunksByType[i])
				{
					UE_LOGF(LogIoStore, Display, "        %ls[%ls]:    %ls changed, %ls new, %ls reused",
						*Reader->ContainerName,
						*ChunkNames[i],
						*NumberString(Reader->ChangedCountByType[i]),
						*NumberString(Reader->NewChunksByType[i]),
						*NumberString(Reader->UsedChunksByType[i]));
				}
			}
		}

		if (ContainerNotFound)
		{
			UE_LOGF(LogIoStore, Display, "    %ls containers were requested that weren't available. This means the ",
				*NumberString(ContainerNotFound));
			UE_LOGF(LogIoStore, Display, "    previous release didn't have these containers. If that doesn't sound right verify");
			UE_LOGF(LogIoStore, Display, "    that you used reference containers from the same project. Missing containers:");

			for (const TPair<FIoContainerId, TUniquePtr<FMissingContainerInfo>>& MissingContainer : MissingContainers)
			{
				for (uint8 i = 0; i < FIoStoreChunkDatabase::IoChunkTypeCount; i++)
				{
					if (MissingContainer.Value->RequestedChunkCount[i])
					{
						UE_LOGF(LogIoStore, Display, "        %ls[%ls]: %ls requests",
							*MissingContainer.Value->ContainerName,
							*ChunkNames[i],
							*NumberString(MissingContainer.Value->RequestedChunkCount[i].load(std::memory_order_relaxed)));
					}
				}
			}
		}
	}
};

[[nodiscard]] static bool LoadKeyChain(const TCHAR* CmdLine, FKeyChain& OutCryptoSettings)
{
	OutCryptoSettings.SetSigningKey(InvalidRSAKeyHandle);
	OutCryptoSettings.GetEncryptionKeys().Empty();

	// First, try and parse the keys from a supplied crypto key cache file
	FString CryptoKeysCacheFilename;
	if (FParse::Value(CmdLine, TEXT("cryptokeys="), CryptoKeysCacheFilename))
	{
		if(IFileManager::Get().FileExists(*CryptoKeysCacheFilename))
		{
			UE_LOGF(LogIoStore, Display, "Parsing crypto keys from a crypto key cache file '%ls'", *CryptoKeysCacheFilename);
			KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, OutCryptoSettings);
		}
		else
		{
			UE_LOGF(LogIoStore, Error, "Unable to parse crypto keys as the cache file '%ls' does not exist!", *CryptoKeysCacheFilename);
			return false;
		}
	}
	else if (FParse::Param(CmdLine, TEXT("encryptionini")))
	{
		FString ProjectDir, EngineDir, Platform;

		if (FParse::Value(CmdLine, TEXT("projectdir="), ProjectDir, false)
			&& FParse::Value(CmdLine, TEXT("enginedir="), EngineDir, false)
			&& FParse::Value(CmdLine, TEXT("platform="), Platform, false))
		{
			UE_LOGF(LogIoStore, Warning, "A legacy command line syntax is being used for crypto config. Please update to using the -cryptokey parameter as soon as possible as this mode is deprecated");

			FConfigFile EngineConfig;

			FConfigCacheIni::LoadExternalIniFile(EngineConfig, TEXT("Engine"), *FPaths::Combine(EngineDir, TEXT("Config\\")), *FPaths::Combine(ProjectDir, TEXT("Config/")), true, *Platform);
			bool bDataCryptoRequired = false;
			EngineConfig.GetBool(TEXT("PlatformCrypto"), TEXT("PlatformRequiresDataCrypto"), bDataCryptoRequired);

			if (!bDataCryptoRequired)
			{
				return true;
			}

			FConfigFile ConfigFile;
			FConfigCacheIni::LoadExternalIniFile(ConfigFile, TEXT("Crypto"), *FPaths::Combine(EngineDir, TEXT("Config\\")), *FPaths::Combine(ProjectDir, TEXT("Config/")), true, *Platform);
			bool bSignPak = false;
			bool bEncryptPakIniFiles = false;
			bool bEncryptPakIndex = false;
			bool bEncryptAssets = false;
			bool bEncryptPak = false;

			if (ConfigFile.Num())
			{
				UE_LOGF(LogIoStore, Display, "Using new format crypto.ini files for crypto configuration");

				static const TCHAR* SectionName = TEXT("/Script/CryptoKeys.CryptoKeysSettings");

				ConfigFile.GetBool(SectionName, TEXT("bEnablePakSigning"), bSignPak);
				ConfigFile.GetBool(SectionName, TEXT("bEncryptPakIniFiles"), bEncryptPakIniFiles);
				ConfigFile.GetBool(SectionName, TEXT("bEncryptPakIndex"), bEncryptPakIndex);
				ConfigFile.GetBool(SectionName, TEXT("bEncryptAssets"), bEncryptAssets);
				bEncryptPak = bEncryptPakIniFiles || bEncryptPakIndex || bEncryptAssets;

				if (bSignPak)
				{
					FString PublicExpBase64, PrivateExpBase64, ModulusBase64;
					ConfigFile.GetString(SectionName, TEXT("SigningPublicExponent"), PublicExpBase64);
					ConfigFile.GetString(SectionName, TEXT("SigningPrivateExponent"), PrivateExpBase64);
					ConfigFile.GetString(SectionName, TEXT("SigningModulus"), ModulusBase64);

					TArray<uint8> PublicExp, PrivateExp, Modulus;
					FBase64::Decode(PublicExpBase64, PublicExp);
					FBase64::Decode(PrivateExpBase64, PrivateExp);
					FBase64::Decode(ModulusBase64, Modulus);

					OutCryptoSettings.SetSigningKey(FRSA::CreateKey(PublicExp, PrivateExp, Modulus));

					UE_LOGF(LogIoStore, Display, "Parsed signature keys from config files.");
				}

				if (bEncryptPak)
				{
					FString EncryptionKeyString;
					ConfigFile.GetString(SectionName, TEXT("EncryptionKey"), EncryptionKeyString);

					if (EncryptionKeyString.Len() > 0)
					{
						TArray<uint8> Key;
						FBase64::Decode(EncryptionKeyString, Key);
						check(Key.Num() == sizeof(FAES::FAESKey::Key));
						FNamedAESKey NewKey;
						NewKey.Name = TEXT("Default");
						NewKey.Guid = FGuid();
						FMemory::Memcpy(NewKey.Key.Key, &Key[0], sizeof(FAES::FAESKey::Key));
						OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
						UE_LOGF(LogIoStore, Display, "Parsed AES encryption key from config files.");
					}
				}
			}
			else
			{
				static const TCHAR* SectionName = TEXT("Core.Encryption");

				UE_LOGF(LogIoStore, Display, "Using old format encryption.ini files for crypto configuration");

				FConfigCacheIni::LoadExternalIniFile(ConfigFile, TEXT("Encryption"), *FPaths::Combine(EngineDir, TEXT("Config\\")), *FPaths::Combine(ProjectDir, TEXT("Config/")), true, *Platform);
				ConfigFile.GetBool(SectionName, TEXT("SignPak"), bSignPak);
				ConfigFile.GetBool(SectionName, TEXT("EncryptPak"), bEncryptPak);

				if (bSignPak)
				{
					FString RSAPublicExp, RSAPrivateExp, RSAModulus;
					ConfigFile.GetString(SectionName, TEXT("rsa.publicexp"), RSAPublicExp);
					ConfigFile.GetString(SectionName, TEXT("rsa.privateexp"), RSAPrivateExp);
					ConfigFile.GetString(SectionName, TEXT("rsa.modulus"), RSAModulus);

					//TODO: Fix me!
					//OutSigningKey.PrivateKey.Exponent.Parse(RSAPrivateExp);
					//OutSigningKey.PrivateKey.Modulus.Parse(RSAModulus);
					//OutSigningKey.PublicKey.Exponent.Parse(RSAPublicExp);
					//OutSigningKey.PublicKey.Modulus = OutSigningKey.PrivateKey.Modulus;

					UE_LOGF(LogIoStore, Display, "Parsed signature keys from config files.");
				}

				if (bEncryptPak)
				{
					FString EncryptionKeyString;
					ConfigFile.GetString(SectionName, TEXT("aes.key"), EncryptionKeyString);
					FNamedAESKey NewKey;
					NewKey.Name = TEXT("Default");
					NewKey.Guid = FGuid();
					if (EncryptionKeyString.Len() == 32 && TCString<TCHAR>::IsPureAnsi(*EncryptionKeyString))
					{
						for (int32 Index = 0; Index < 32; ++Index)
						{
							NewKey.Key.Key[Index] = (uint8)EncryptionKeyString[Index];
						}
						OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
						UE_LOGF(LogIoStore, Display, "Parsed AES encryption key from config files.");
					}
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogIoStore, Display, "Using command line for crypto configuration");

		FString EncryptionKeyString;
		FParse::Value(CmdLine, TEXT("aes="), EncryptionKeyString, false);

		if (EncryptionKeyString.Len() > 0)
		{
			UE_LOGF(LogIoStore, Warning, "A legacy command line syntax is being used for crypto config. Please update to using the -cryptokey parameter as soon as possible as this mode is deprecated");

			FNamedAESKey NewKey;
			NewKey.Name = TEXT("Default");
			NewKey.Guid = FGuid();
			const uint32 RequiredKeyLength = sizeof(NewKey.Key);

			// Error checking
			if (EncryptionKeyString.Len() < RequiredKeyLength)
			{
				UE_LOGF(LogIoStore, Error, "AES encryption key must be %d characters long", RequiredKeyLength);
				return false;
			}

			if (EncryptionKeyString.Len() > RequiredKeyLength)
			{
				UE_LOGF(LogIoStore, Warning, "AES encryption key is more than %d characters long, so will be truncated!", RequiredKeyLength);
				EncryptionKeyString.LeftInline(RequiredKeyLength);
			}

			if (!FCString::IsPureAnsi(*EncryptionKeyString))
			{
				UE_LOGF(LogIoStore, Error, "AES encryption key must be a pure ANSI string!");
				return false;
			}

			const auto AsAnsi = StringCast<ANSICHAR>(*EncryptionKeyString);
			check(AsAnsi.Length() == RequiredKeyLength);
			FMemory::Memcpy(NewKey.Key.Key, AsAnsi.Get(), RequiredKeyLength);
			OutCryptoSettings.GetEncryptionKeys().Add(NewKey.Guid, NewKey);
			UE_LOGF(LogIoStore, Display, "Parsed AES encryption key from command line.");
		}
	}

	FString EncryptionKeyOverrideGuidString;
	FGuid EncryptionKeyOverrideGuid;
	if (FParse::Value(CmdLine, TEXT("EncryptionKeyOverrideGuid="), EncryptionKeyOverrideGuidString))
	{
		UE_LOGF(LogIoStore, Display, "Using encryption key override '%ls'", *EncryptionKeyOverrideGuidString);
		FGuid::Parse(EncryptionKeyOverrideGuidString, EncryptionKeyOverrideGuid);
	}
	OutCryptoSettings.SetPrincipalEncryptionKey(OutCryptoSettings.GetEncryptionKeys().Find(EncryptionKeyOverrideGuid));

	return true;
}

using FPackageNameMap = TMap<FName, FCookedPackage*>;
using FPackageIdMap = TMap<FPackageId, FCookedPackage*>;

enum EAggregate
{
	Installed,
	Optional,
	IAS,
	IAD,
	NumTypes
};

const FString AggregateNames[EAggregate::NumTypes] =
{
	TEXT("Installed"),
	TEXT("Optional"),
	TEXT("IAS"),
	TEXT("IAD")
};

class FChunkDataSummary
{
public:

	class FChunkData
	{
	public:
		uint64 PackageCount[EAggregate::NumTypes];
		uint64 UncompressedSize[EAggregate::NumTypes];
		uint64 CompressedSize[EAggregate::NumTypes];

		FChunkData()
		{
			for (uint32 i = 0; i < EAggregate::NumTypes; i++)
			{
				PackageCount[i] = UncompressedSize[i] = CompressedSize[i] = 0;
			}
		}
	};
	
	void AddPackage(const FString& PakChunk, EAggregate Type, uint64 UncompressedSize, uint64 CompressedSize)
	{
		// See if we already have an entry for this chunk
		FChunkDataSummary::FChunkData* ChunkData = ChunkDataMap.Find(PakChunk);

		if (ChunkData == nullptr)
		{
			// Add a new entry for this chunk
			ChunkData = &ChunkDataMap.Emplace(PakChunk);
		}

		ChunkData->UncompressedSize[Type] += UncompressedSize;
		ChunkData->CompressedSize[Type] += CompressedSize;
		ChunkData->PackageCount[Type]++;
	}

	void SendTelemetryEvent()
	{
		if (FStudioTelemetry::Get().IsSessionRunning())
		{
			const int SchemaVersion = 2;

			for (TMap< FString, FChunkData>::TConstIterator it(ChunkDataMap); it; ++it)
			{
				const FChunkData& ChunkData = (*it).Value;

				for (uint32 i = 0; i < EAggregate::NumTypes; i++)
				{
					// Only send an event if the PakChunk contains one or more packages
					if (ChunkData.PackageCount[i] > 0)
					{
						// Build an event from the aggregated data for each PakChunk for each DeliveryType
						TArray<FAnalyticsEventAttribute> Attributes;

						Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
						Attributes.Emplace(TEXT("PakChunk"), (*it).Key);
						Attributes.Emplace(TEXT("DeliveryType"), *AggregateNames[i]);
						Attributes.Emplace(TEXT("PackageCount"), ChunkData.PackageCount[i]);
						Attributes.Emplace(TEXT("Size"), ChunkData.UncompressedSize[i]);
						Attributes.Emplace(TEXT("CompressedSize"), ChunkData.CompressedSize[i]);

						// Send the event to the telemetry system
						FStudioTelemetry::Get().RecordEvent(TEXT("Core.IoStoreChunk.Summary"), Attributes);
					}
				}	
			}
		}
	}

private:

	TMap< FString, FChunkData> ChunkDataMap;
};

class FChunkEntryCsv
{
public:
	FChunkEntryCsv() = default;
	~FChunkEntryCsv()
	{
		if (OutputArchive.IsValid())
		{
			OutputArchive->Flush();
			ChunkDataSummary.SendTelemetryEvent();
		}
	}

	void SetPartitionSize(uint64 InPartitionSize)
	{
		if (InPartitionSize > 0)
		{
			PartitionSize = InPartitionSize;
		}
		else
		{
			PartitionSize = TNumericLimits<uint64>::Max();
		}
	}

	void CreateOutputFile(const TCHAR* OutputFilename)
	{
		OutputArchive.Reset(IFileManager::Get().CreateFileWriter(OutputFilename));
		if (OutputArchive.IsValid())
		{
			OutputArchive->Logf(TEXT("OrderInContainer, ChunkId, PackageId, PackageName, Filename, ContainerName, Offset, OffsetOnDisk, Size, CompressedSize, Hash, ChunkType, ClassType, PakChunk, Platform, InstallType, DeliveryType, PartitionIndex, OffsetInPartition"));
		}
	}

	void AddChunk(const FString& ContainerName, int32 Index, const FIoStoreTocChunkInfo& Info, FPackageId PackageId, const FString& PackageName, const FString& ClassType)
	{
		if (OutputArchive.IsValid())
		{
			FString PakChunk = TEXT("global");
			FString Platform = TEXT("Unknown");
			FString InstallType = TEXT("Base");
			FString DeliveryType = TEXT("Installed");

			// Extract the PakChunk name and the Platform from the container name. Have to go from end because
			// named containers use - as a delimited and we want the last instance for the platform.
			ContainerName.Split(TEXT("-"), &PakChunk, &Platform, ESearchCase::CaseSensitive, ESearchDir::FromEnd);

			// Determine the Delivery Type Installed/IAS/IAD
			if (ContainerName.Contains(TEXT("iad")))
			{
				// IAD data
				DeliveryType = TEXT("IAD");
				ChunkDataSummary.AddPackage(PakChunk, EAggregate::IAD, Info.Size, Info.CompressedSize);
			}
			else if (ContainerName.Contains(TEXT("ondemand")))
			{
				// IAS data
				DeliveryType = TEXT("IAS");
				ChunkDataSummary.AddPackage(PakChunk, EAggregate::IAS, Info.Size, Info.CompressedSize);
			}
			else
			{
				// Mandatory or Optionally Installed data
				if (ContainerName.Contains(TEXT("optional")))
				{
					ChunkDataSummary.AddPackage(PakChunk, EAggregate::Optional, Info.Size, Info.CompressedSize);
				}
				else
				{
					ChunkDataSummary.AddPackage(PakChunk, EAggregate::Installed, Info.Size, Info.CompressedSize);
				}
			}

			// Determine the Install Type Base/Optional
			if (ContainerName.Contains(TEXT("optional")))
			{
				InstallType = TEXT("Optional");
			}
			
			OutputArchive->Logf(TEXT("%d, %s, 0x%s, %s, %s, %s, %lld, %lld, %lld, %lld, 0x%s, %s, %s, %s, %s, %s, %s, %llu, %llu"),
				Index,
				*LexToString(Info.Id),
				*LexToString(PackageId),
				*PackageName,
				*Info.FileName,
				*ContainerName,
				Info.Offset,
				Info.OffsetOnDisk,
				Info.Size,
				Info.CompressedSize,
				*LexToString(Info.ChunkHash),
				*LexToString(Info.ChunkType),
				*ClassType,
				*PakChunk,
				*Platform,
				*InstallType,
				*DeliveryType,
				Info.OffsetOnDisk / PartitionSize,
				Info.OffsetOnDisk % PartitionSize 
			);
		}
	}	

private:
	uint64 PartitionSize=TNumericLimits<uint64>::Max();
	TUniquePtr<FArchive> OutputArchive;
	FChunkDataSummary ChunkDataSummary;
};

void SortPackagesInLoadOrderRecursive(TArray<FCookedPackage*>& Result, FCookedPackage* Package, TArray<FCookedPackage*>& S, TArray<FCookedPackage*>& P, int32& C, const TMap<FCookedPackage*, TArray<FCookedPackage*>>& ReverseEdges)
{
	Package->PreOrderNumber = C;
	++C;
	S.Push(Package);
	P.Push(Package);
	const TArray<FCookedPackage*>* FindParents = ReverseEdges.Find(Package);
	if (FindParents)
	{
		for (FCookedPackage* Parent : *FindParents)
		{
			if (!Parent->bPermanentMark)
			{
				if (Parent->PreOrderNumber < 0)
				{
					SortPackagesInLoadOrderRecursive(Result, Parent, S, P, C, ReverseEdges);
				}
				else
				{
					while (P.Top()->PreOrderNumber > Parent->PreOrderNumber)
					{
						P.Pop(EAllowShrinking::No);
					}
				}
			}
		}
	}
	if (P.Top() == Package)
	{
		FCookedPackage* InStronglyConnectedComponent;
		do
		{
			InStronglyConnectedComponent = S.Top();
			S.Pop(EAllowShrinking::No);
			InStronglyConnectedComponent->bPermanentMark = true;
			Result.Add(InStronglyConnectedComponent);
		} while (InStronglyConnectedComponent != Package);
		P.Pop(EAllowShrinking::No);
	}
}

void SortPackagesInLoadOrder(TArray<FCookedPackage*>& Packages, const TMap<FPackageId, FCookedPackage*>& PackagesMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SortPackagesInLoadOrder);
	Algo::Sort(Packages, [](const FCookedPackage* A, const FCookedPackage* B)
		{
			return A->GlobalPackageId < B->GlobalPackageId;
		});

	TMap<FCookedPackage*, TArray<FCookedPackage*>> ReverseEdges;
	ReverseEdges.Reserve(PackagesMap.Num());
	for (FCookedPackage* Package : Packages)
	{
		for (FPackageId ImportedPackageId : Package->PackageStoreEntry.ImportedPackageIds)
		{
			FCookedPackage* FindImportedPackage = PackagesMap.FindRef(ImportedPackageId);
			if (FindImportedPackage)
			{
				TArray<FCookedPackage*>& SourceArray = ReverseEdges.FindOrAdd(FindImportedPackage);
				SourceArray.Add(Package);
			}
		}
	}
	for (auto& KV : ReverseEdges)
	{
		TArray<FCookedPackage*>& SourceArray = KV.Value;
		Algo::Sort(SourceArray, [](const FCookedPackage* A, const FCookedPackage* B)
			{
				return A->GlobalPackageId < B->GlobalPackageId;
			});
	}

	// Path based strongly connected components + topological sort of the components
	TArray<FCookedPackage*> Result;
	Result.Reserve(Packages.Num());
	TArray<FCookedPackage*> S;
	TArray<FCookedPackage*> P;
	for (FCookedPackage* Package : Packages)
	{
		if (!Package->bPermanentMark)
		{
			S.Reset();
			P.Reset();
			int32 C = 0;
			SortPackagesInLoadOrderRecursive(Result, Package, S, P, C, ReverseEdges);
		}
	}
	check(Result.Num() == Packages.Num());
	Algo::Reverse(Result);
	Swap(Packages, Result);
	uint64 LoadOrder = 0;
	for (FCookedPackage* Package : Packages)
	{
		Package->LoadOrder = LoadOrder++;
	}
}

class FClusterStatsCsv
{
public:

	~FClusterStatsCsv()
	{
		if (OutputArchive)
		{
			OutputArchive->Flush();
		}
	}

	void CreateOutputFile(const FString& Path)
	{
		OutputArchive.Reset(IFileManager::Get().CreateFileWriter(*Path));
		if (OutputArchive)
		{
			OutputArchive->Logf(TEXT("PackageName,ClusterUExpBytes,BytesToRead,ClustersToRead,ClusterOwner,OrderFile,OrderIndex"));
		}
	}

	void AddPackage(FName PackageName, int64 ClusterUExpBytes, int64 DepUExpBytes, uint32 NumTouchedClusters, FName ClusterOwner, const FFileOrderMap* BlameOrderMap, uint64 LocalOrder)
	{
		if (!OutputArchive.IsValid())
		{
			return;
		}
		OutputArchive->Logf(TEXT("%s,%lld,%lld,%u,%s,%s,%llu"),
			*PackageName.ToString(),
			ClusterUExpBytes,
			DepUExpBytes,
			NumTouchedClusters,
			*ClusterOwner.ToString(),
			BlameOrderMap ? *BlameOrderMap->Name : TEXT("None"),
			LocalOrder
		);
	}

	void Close()
	{
		OutputArchive.Reset();
	}

private:
	TUniquePtr<FArchive> OutputArchive;
};
FClusterStatsCsv ClusterStatsCsv;

// If bClusterByOrderFilePriority is false
//		Order packages by the order of OrderMaps with their associated priority 
//		e.g. Order map 1 with priority 0 (A, B, C) 
//			 Order map 2 with priority 10 (B, D)
//			Final order: (A, C, B, D)
//		Then cluster packages in this order. 
// If bClusterByOrderFilePriority is true
//		Cluster packages first by OrderMaps in priority order, then concatenate clusters in the array order of OrderMaps.
//		e.g. Order map 1 with priority 0 (A, B, C) 
//			 Order map 2 with priority 10 (B, D)
//			Cluster packages B, D, then A, C
//			Then reassemble clusters A, C, B, D
static void AssignPackagesDiskOrder(
	const TArray<FCookedPackage*>& Packages,
	const TArray<FFileOrderMap>& OrderMaps,
	const FPackageIdMap& PackageIdMap,
	const FIoStoreOrderingOptions& OrderingOptions,
	bool bAssignOverrideDiskLayoutOrder)
{
	IOSTORE_CPU_SCOPE(AssignPackagesDiskOrder);

	struct FCluster
	{
		TArray<FCookedPackage*> Packages;
		int32 OrderFileIndex; // Index in OrderMaps of the FFileOrderMap which contained Packages.Last()
		int32 ClusterSequence;

		FCluster(int32 InOrderFileIndex, int32 InClusterSequence)
			: OrderFileIndex(InOrderFileIndex)
			, ClusterSequence(InClusterSequence)
		{
		}
	};

	struct FPackageAndOrder
	{
		FCookedPackage* Package = nullptr;
		int64 LocalOrder;
		const FFileOrderMap* OrderMap;

		FPackageAndOrder(FCookedPackage* InPackage, int64 InLocalOrder, const FFileOrderMap* InOrderMap)
			: Package(InPackage)
			, LocalOrder(InLocalOrder)
			, OrderMap(InOrderMap)
		{
			check(OrderMap);
		}
	};

	// Order maps sorted by priority
	TArray<const FFileOrderMap*> PriorityOrderMaps;
	for (const FFileOrderMap& Map : OrderMaps)
	{
		PriorityOrderMaps.Add(&Map);
	}

	// Create a fallback order map to avoid null checks later
	// Lowest priority, last index
	FFileOrderMap FallbackOrderMap(MIN_int32, MAX_int32);
	FallbackOrderMap.Name = TEXT("Fallback");

	// Generate a simple alphabetically sorted fallback order if requested
	if (OrderingOptions.FallbackOrderMode == EFallbackOrderMode::Alphabetical)
	{
		TArray<FName> SortedPackageNames;
		SortedPackageNames.Reserve(Packages.Num());
		for (FCookedPackage* Package : Packages)
		{
			SortedPackageNames.Add(Package->PackageName);
		}

		Algo::Sort(SortedPackageNames, FNameLexicalLess());

		int64 SortIndex = 0;
		for (FName PackageName : SortedPackageNames)
		{
			FallbackOrderMap.PackageNameToOrder.Add(PackageName, SortIndex++);
		}
		PriorityOrderMaps.Add(&FallbackOrderMap);
	}

	Algo::StableSortBy(PriorityOrderMaps, [](const FFileOrderMap* Map) { return Map->Priority; }, TGreater<int32>());

	TArray<FPackageAndOrder> SortedPackages;
	SortedPackages.Reserve(Packages.Num());
	for (FCookedPackage* Package : Packages)
	{
		// Default to the fallback order map 
		// Reverse the bundle load order for the fallback map (so that packages are considered before their imports)
		const FFileOrderMap* UsedOrderMap = &FallbackOrderMap;
		int64 LocalOrder = -int64(Package->LoadOrder);

		for (const FFileOrderMap* OrderMap : PriorityOrderMaps)
		{
			if (const int64* Order = OrderMap->PackageNameToOrder.Find(Package->PackageName))
			{
				LocalOrder = *Order;
				UsedOrderMap = OrderMap;
				break;
			}
		}

		SortedPackages.Emplace(Package, LocalOrder, UsedOrderMap);
	}
	const FFileOrderMap* LastBlameOrderMap = nullptr;
	int32 LastAssignedCount = 0;
	int64 LastAssignedUExpSize = 0, AssignedUExpSize = 0;
	int64 LastAssignedBulkSize = 0, AssignedBulkSize = 0;

	if (OrderingOptions.bClusterByOrderFilePriority)
	{
		// Sort by priority of the order map
		Algo::Sort(SortedPackages, [](const FPackageAndOrder& A, const FPackageAndOrder& B) {
			// Packages in the same order map should be sorted by their local ordering
			if (A.OrderMap == B.OrderMap)
			{
				return A.LocalOrder < B.LocalOrder;
			}

			// First priority, then index
			if (A.OrderMap->Priority != B.OrderMap->Priority)
			{
				return A.OrderMap->Priority > B.OrderMap->Priority;
			}

			check(A.OrderMap->Index != B.OrderMap->Index);
			return A.OrderMap->Index < B.OrderMap->Index;
		});
	}
	else
	{
		// Sort by the order of the order map (...)
		Algo::Sort(SortedPackages, [](const FPackageAndOrder& A, const FPackageAndOrder& B) {
			// Packages in the same order map should be sorted by their local ordering
			if (A.OrderMap == B.OrderMap)
			{
				return A.LocalOrder < B.LocalOrder;
			}

			// Blame order priority is not considered for the order in which we cluster packages, only for the order in which we assign packages to an order map
			return A.OrderMap->Index < B.OrderMap->Index;
		});
	}

	// Keep these containers outside of inner loops to reuse allocated memory across iterations
	// No need to allocate & free them on every single iteration, just reset element count before using them
	TSet<FCluster*> ClustersToRead;
	TSet<FCookedPackage*> VisitedDeps;
	TArray<FCookedPackage*> DepQueue;
	TArray<FCluster*> OrderedClustersToRead;

	int32 ClusterSequence = 0;
	TMap<FCookedPackage*, FCluster*> PackageToCluster;
	TArray<FCluster*> Clusters;
	TSet<FCookedPackage*> AssignedPackages;
	TArray<FCookedPackage*> ProcessStack;
	PackageToCluster.Reserve(SortedPackages.Num());
	AssignedPackages.Reserve(SortedPackages.Num());
	for (FPackageAndOrder& Entry : SortedPackages)
	{
		checkSlow(Entry.OrderMap); // Without this, Entry.OrderMap != LastBlameOrderMap convinces static analysis that Entry.OrderMap may be null
		if (Entry.OrderMap != LastBlameOrderMap)
		{
			if( LastBlameOrderMap != nullptr )
			{
				UE_LOGF(LogIoStore, Display, "Ordered %d/%d packages using order file %ls. %.2fMB UExp data. %.2fmb bulk data.", 
				AssignedPackages.Num() - LastAssignedCount, Packages.Num(), *LastBlameOrderMap->Name,
				(AssignedUExpSize - LastAssignedUExpSize) / 1024.0 / 1024.0,
				(AssignedBulkSize - LastAssignedBulkSize) / 1024.0 / 1024.0
				 );
			}
			LastAssignedCount = AssignedPackages.Num();
			LastAssignedUExpSize= AssignedUExpSize;
			LastAssignedBulkSize = AssignedBulkSize;
			LastBlameOrderMap = Entry.OrderMap;
		}
		if (!AssignedPackages.Contains(Entry.Package))
		{
			FCluster* Cluster = new FCluster(Entry.OrderMap->Index, ClusterSequence++);
			Clusters.Add(Cluster);
			ProcessStack.Push(Entry.Package);

			bool bDoClustering = true;
			if (OrderingOptions.FallbackOrderMode == EFallbackOrderMode::Alphabetical && Entry.OrderMap == &FallbackOrderMap)
			{
				// Disable clustering if we're doing alphabetical fallback ordering
				bDoClustering = false;
			}

			int64 ClusterBytes = 0;
			while (ProcessStack.Num())
			{
				FCookedPackage* PackageToProcess = ProcessStack.Pop(EAllowShrinking::No);
				if (!AssignedPackages.Contains(PackageToProcess))
				{
					AssignedPackages.Add(PackageToProcess);
					Cluster->Packages.Add(PackageToProcess);
					PackageToCluster.Add(PackageToProcess, Cluster);
					ClusterBytes += PackageToProcess->UExpSize;
					AssignedUExpSize += PackageToProcess->UExpSize;
					AssignedBulkSize += PackageToProcess->TotalBulkDataSize;

					if (bDoClustering)
					{
						// Add referenced dependencies to the package
						for (const FPackageId& ReferencedPackageId : PackageToProcess->PackageStoreEntry.ImportedPackageIds)
						{
							FCookedPackage* FindReferencedPackage = PackageIdMap.FindRef(ReferencedPackageId);
							if (FindReferencedPackage)
							{
								ProcessStack.Push(FindReferencedPackage);
							}
						}
					}
				}
			}
			// This is all just for stats, to compute the "NumClustersToRead" column of the CSV, an approximation of seeks per cluster:
			for (FCookedPackage* Package : Cluster->Packages)
			{
				int64 BytesToRead = 0;

				ClustersToRead.Reset();
				VisitedDeps.Reset();
				DepQueue.Reset();

				DepQueue.Push(Package);
				while (DepQueue.Num() > 0)
				{
					FCookedPackage* Cursor = DepQueue.Pop(EAllowShrinking::No);
					if( VisitedDeps.Contains(Cursor) == false)
					{
						VisitedDeps.Add(Cursor);
						BytesToRead += Cursor->UExpSize;
						if (FCluster* ReadCluster = PackageToCluster.FindRef(Cursor))
						{
							ClustersToRead.Add(ReadCluster);
						}
						
						for (const FPackageId& ImportedPackageId : Cursor->PackageStoreEntry.ImportedPackageIds)
						{
							FCookedPackage* FindReferencedPackage = PackageIdMap.FindRef(ImportedPackageId);
							if (FindReferencedPackage)
							{
								DepQueue.Push(FindReferencedPackage);
							}
						}
					}
				}

				OrderedClustersToRead.Reset(ClustersToRead.Num());
				for (FCluster* ClusterToRead : ClustersToRead)
				{
					OrderedClustersToRead.Add(ClusterToRead);
				}
				Algo::SortBy(OrderedClustersToRead, [](FCluster* C) { return C->ClusterSequence; }, TLess<int32>());

				int32 NumClustersToRead = 1; // Could replace with "min seeks"
				for (int32 i = 1; i < OrderedClustersToRead.Num(); ++i)
				{
					if (OrderedClustersToRead[i]->ClusterSequence != OrderedClustersToRead[i - 1]->ClusterSequence + 1)
					{
						++NumClustersToRead;
					}
				}

				FName ClusterOwner = Entry.Package->PackageName;
				ClusterStatsCsv.AddPackage(Package->PackageName, Package == Entry.Package ? ClusterBytes : 0, BytesToRead, NumClustersToRead, ClusterOwner, Entry.OrderMap, Entry.LocalOrder);
			}
		}
	}
	UE_LOGF(LogIoStore, Display, "Ordered %d packages using fallback bundle order", AssignedPackages.Num() - LastAssignedCount);

	check(AssignedPackages.Num() == Packages.Num());
	
	// Sort the clusters 
	if (OrderingOptions.bClusterByOrderFilePriority)
	{
		Algo::StableSortBy(Clusters, [](FCluster* Cluster) { return Cluster->OrderFileIndex; });
	}
	
	if (OrderingOptions.FallbackOrderMode == EFallbackOrderMode::AlphabeticalClustered)
	{
		// Sort the fallback order clusters alphabetically
		Algo::StableSort(Clusters, [FallbackOrderMap](const FCluster* A, const FCluster* B) { 
				if (A->OrderFileIndex == FallbackOrderMap.Index && B->OrderFileIndex == FallbackOrderMap.Index)
				{
					return A->Packages[0]->PackageName.LexicalLess(B->Packages[0]->PackageName);
				}
				return false;
			});	
	}

	for (FCluster* Cluster : Clusters)
	{
		if (OrderingOptions.bAlphaSortClusterPackageLists)
		{
			Algo::SortBy(Cluster->Packages, [](const FCookedPackage* Package) { return Package->PackageName; }, FNameLexicalLess());
		}
		else
		{
			Algo::Sort(Cluster->Packages, [](const FCookedPackage* A, const FCookedPackage* B) { return A->LoadOrder < B->LoadOrder; });
		}
	}

	uint64 LayoutIndex = 0;
	for (FCluster* Cluster : Clusters)
	{
		for (FCookedPackage* Package : Cluster->Packages)
		{
			if (bAssignOverrideDiskLayoutOrder)
			{
				Package->OverrideDiskLayoutOrder = LayoutIndex++;
			}
			else
			{
				Package->DiskLayoutOrder = LayoutIndex++;
			}
		}
		delete Cluster;
	}
	
	ClusterStatsCsv.Close();
}

using FOrderingOptionsToContainersMap = TMap<FIoStoreOrderingOptions, TArray<const FContainerTargetSpec*>>;

static void CreateDiskLayout(
	const TArray<FContainerTargetSpec*>& ContainerTargets,
	const TArray<FCookedPackage*>& Packages,
	const TArray<FFileOrderMap>& OrderMaps,
	const FPackageIdMap& PackageIdMap,
	const FIoStoreOrderingOptions& OrderingOptions,
	const FOrderingOptionsToContainersMap& OrderingOptionsOverrides)
{
	IOSTORE_CPU_SCOPE(CreateDiskLayout);

	{
		const bool bAssignOverrideDiskLayoutOrder = false;
		AssignPackagesDiskOrder(Packages, OrderMaps, PackageIdMap, OrderingOptions, bAssignOverrideDiskLayoutOrder);
	}

	for (const TPair<FIoStoreOrderingOptions, TArray<const FContainerTargetSpec*>>& Override : OrderingOptionsOverrides)
	{
		const bool bAssignOverrideDiskLayoutOrder = true;
		const FIoStoreOrderingOptions& OverrideOrderingOptions = Override.Key;
		AssignPackagesDiskOrder(Packages, OrderMaps, PackageIdMap, OverrideOrderingOptions, bAssignOverrideDiskLayoutOrder);

		const TArray<const FContainerTargetSpec*>& ContainersToApplyTo = Override.Value;
		for (const FContainerTargetSpec* Container : ContainersToApplyTo)
		{
			for (FCookedPackage* Package : Container->Packages)
			{
				Package->DiskLayoutOrder = Package->OverrideDiskLayoutOrder;
			}
		}
	}

	for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
	{
		TArray<FContainerTargetFile*> SortedTargetFiles;
		SortedTargetFiles.Reserve(ContainerTarget->TargetFiles.Num());
		TMap<FIoChunkId, FContainerTargetFile*> ShaderTargetFilesMap;
		ShaderTargetFilesMap.Reserve(ContainerTarget->GlobalShaders.Num() + ContainerTarget->SharedShaders.Num() + ContainerTarget->UniqueShaders.Num() + ContainerTarget->InlineShaders.Num());
		for (FContainerTargetFile& TargetFile : ContainerTarget->TargetFiles)
		{
			if (TargetFile.ChunkType == EContainerChunkType::ShaderCode)
			{
				ShaderTargetFilesMap.Add(TargetFile.ChunkId, &TargetFile);
			}
			else
			{
				SortedTargetFiles.Add(&TargetFile);
			}
		}

		check(ShaderTargetFilesMap.Num() == ContainerTarget->GlobalShaders.Num() + ContainerTarget->SharedShaders.Num() + ContainerTarget->UniqueShaders.Num() + ContainerTarget->InlineShaders.Num());
		Algo::Sort(SortedTargetFiles, [](const FContainerTargetFile* A, const FContainerTargetFile* B)
		{
			if (A->ChunkType != B->ChunkType)
			{
				return A->ChunkType < B->ChunkType;
			}
			if (A->ChunkType == EContainerChunkType::ShaderCodeLibrary)
			{
				return A->DestinationPath < B->DestinationPath;
			}
			if (A->Package != B->Package)
			{
				return A->Package->DiskLayoutOrder < B->Package->DiskLayoutOrder;
			}
			if (A->BulkDataCookedIndex != B->BulkDataCookedIndex)
			{
				return A->BulkDataCookedIndex < B->BulkDataCookedIndex;
			}
			check(A == B)
			return false;
		});

		int32 Index = 0;
		int32 ShaderCodeInsertionIndex = -1;
		while (Index < SortedTargetFiles.Num())
		{
			FContainerTargetFile* TargetFile = SortedTargetFiles[Index];
			if (ShaderCodeInsertionIndex < 0 && TargetFile->ChunkType != EContainerChunkType::ShaderCodeLibrary)
			{
				ShaderCodeInsertionIndex = Index;
			}
			if (TargetFile->ChunkType == EContainerChunkType::PackageData)
			{
				TArray<FContainerTargetFile*, TInlineAllocator<1024>> PackageInlineShaders;

				// Since we are inserting in to a sorted array (on disk order), we have to be stably sorted
				// beforehand
				Algo::Sort(TargetFile->Package->Shaders, FShaderInfo::Sort);
				for (FShaderInfo* Shader : TargetFile->Package->Shaders)
				{
					check(Shader->ReferencedByPackages.Num() > 0);
					FShaderInfo::EShaderType* ShaderType = Shader->TypeInContainer.Find(ContainerTarget);
					if (ShaderType && *ShaderType == FShaderInfo::Inline)
					{
						check(ContainerTarget->InlineShaders.Contains(Shader));
						FContainerTargetFile* ShaderTargetFile = ShaderTargetFilesMap.FindRef(Shader->ShaderGroupChunk.ChunkId);
						check(ShaderTargetFile);
						PackageInlineShaders.Add(ShaderTargetFile);
					}
				}
				if (!PackageInlineShaders.IsEmpty())
				{
					SortedTargetFiles.Insert(PackageInlineShaders, Index + 1);
					Index += PackageInlineShaders.Num();
				}
			}
			++Index;
		}
		if (ShaderCodeInsertionIndex < 0)
		{
			ShaderCodeInsertionIndex = 0;
		}

		auto AddShaderTargetFiles =
			[&ShaderTargetFilesMap, &SortedTargetFiles, &ShaderCodeInsertionIndex]
			(TSet<FShaderInfo*>& Shaders)
		{
			if (!Shaders.IsEmpty())
			{
				TArray<FShaderInfo*> SortedShaders = Shaders.Array();
				Algo::Sort(SortedShaders, FShaderInfo::Sort);
				TArray<FContainerTargetFile*> ShaderTargetFiles;
				ShaderTargetFiles.Reserve(SortedShaders.Num());
				for (const FShaderInfo* ShaderInfo : SortedShaders)
				{
					FContainerTargetFile* ShaderTargetFile = ShaderTargetFilesMap.FindRef(ShaderInfo->ShaderGroupChunk.ChunkId);
					check(ShaderTargetFile);
					ShaderTargetFiles.Add(ShaderTargetFile);
				}
				SortedTargetFiles.Insert(ShaderTargetFiles, ShaderCodeInsertionIndex);
				ShaderCodeInsertionIndex += ShaderTargetFiles.Num();
			}
		};
		AddShaderTargetFiles(ContainerTarget->GlobalShaders);
		AddShaderTargetFiles(ContainerTarget->SharedShaders);
		AddShaderTargetFiles(ContainerTarget->UniqueShaders);

		check(SortedTargetFiles.Num() == ContainerTarget->TargetFiles.Num());

		if (OrderingOptions.bPlaceShadersAtEnd)
		{
			TArray<FContainerTargetFile*> NonShaders;
			TArray<FContainerTargetFile*> Shaders;
			NonShaders.Reserve(SortedTargetFiles.Num());
			Shaders.Reserve(SortedTargetFiles.Num());
			for (FContainerTargetFile* TargetFile : SortedTargetFiles)
			{
				const bool bIsShader =
					TargetFile->ChunkType == EContainerChunkType::ShaderCode ||
					TargetFile->ChunkType == EContainerChunkType::ShaderCodeLibrary;
				(bIsShader ? Shaders : NonShaders).Add(TargetFile);
			}
			SortedTargetFiles = MoveTemp(NonShaders);
			SortedTargetFiles.Append(Shaders);
		}

		uint64 IdealOrder = 0;
		for (FContainerTargetFile* TargetFile : SortedTargetFiles)
		{
			TargetFile->IdealOrder = IdealOrder++;
		}
	}
}

FContainerTargetSpec* AddContainer(
	FName Name,
	TArray<FContainerTargetSpec*>& Containers)
{
	FIoContainerId ContainerId = FIoContainerId::FromName(Name);
	for (FContainerTargetSpec* ExistingContainer : Containers)
	{
		if (ExistingContainer->Name == Name)
		{
			UE_LOGF(LogIoStore, Fatal, "Duplicate container name: '%ls'", *Name.ToString());
			return nullptr;
		}
		if (ExistingContainer->ContainerId == ContainerId)
		{
			UE_LOGF(LogIoStore, Fatal, "Hash collision for container names: '%ls' and '%ls'", *Name.ToString(), *ExistingContainer->Name.ToString());
			return nullptr;
		}
	}
	
	FContainerTargetSpec* ContainerTargetSpec = new FContainerTargetSpec();
	ContainerTargetSpec->Name = Name;
	ContainerTargetSpec->ContainerId = ContainerId;
	Containers.Add(ContainerTargetSpec);
	return ContainerTargetSpec;
}

FCookedPackage& FindOrAddPackage(
	const FIoStoreArguments& Arguments,
	const FName& PackageName,
	TArray<FCookedPackage*>& Packages,
	FPackageNameMap& PackageNameMap,
	FPackageIdMap& PackageIdMap)
{
	FCookedPackage* Package = PackageNameMap.FindRef(PackageName);
	if (!Package)
	{
		FPackageId PackageId = FPackageId::FromName(PackageName);
		if (FCookedPackage* FindById = PackageIdMap.FindRef(PackageId))
		{
			UE_LOGF(LogIoStore, Fatal, "Package name hash collision \"%ls\" and \"%ls", *FindById->PackageName.ToString(), *PackageName.ToString());
		}

		if (const FName* ReleasedPackageName = Arguments.ReleasedPackages.PackageIdToName.Find(PackageId))
		{
			UE_LOGF(LogIoStore, Fatal, "Package name hash collision with base game package \"%ls\" and \"%ls", *ReleasedPackageName->ToString(), *PackageName.ToString());
		}

		Package = new FCookedPackage();
		Package->PackageName = PackageName;
		Package->GlobalPackageId = PackageId;
		Packages.Add(Package);
		PackageNameMap.Add(PackageName, Package);
		PackageIdMap.Add(PackageId, Package);
	}

	return *Package;
}

FLegacyCookedPackage* FindOrAddLegacyPackage(
	const FIoStoreArguments& Arguments,
	const TCHAR* FileName,
	TArray<FCookedPackage*>& Packages,
	FPackageNameMap& PackageNameMap,
	FPackageIdMap& PackageIdMap)
{
	FName PackageName = Arguments.PackageStore->GetPackageNameFromFileName(FileName);
	if (PackageName.IsNone())
	{
		return nullptr;
	}

	FCookedPackage* Package = PackageNameMap.FindRef(PackageName);
	if (!Package)
	{
		FPackageId PackageId = FPackageId::FromName(PackageName);
		if (FCookedPackage* FindById = PackageIdMap.FindRef(PackageId))
		{
			UE_LOGF(LogIoStore, Fatal, "Package name hash collision \"%ls\" and \"%ls", *FindById->PackageName.ToString(), *PackageName.ToString());
		}

		if (const FName* ReleasedPackageName = Arguments.ReleasedPackages.PackageIdToName.Find(PackageId))
		{
			UE_LOGF(LogIoStore, Fatal, "Package name hash collision with base game package \"%ls\" and \"%ls", *ReleasedPackageName->ToString(), *PackageName.ToString());
		}

		Package = new FLegacyCookedPackage();
		Package->PackageName = PackageName;
		Package->GlobalPackageId = PackageId;
		Packages.Add(Package);
		PackageNameMap.Add(PackageName, Package);
		PackageIdMap.Add(PackageId, Package);
	}
	return static_cast<FLegacyCookedPackage*>(Package);
}

static void ParsePackageAssetsFromFiles(TArray<FCookedPackage*>& Packages, const FPackageStoreOptimizer& PackageStoreOptimizer)
{
	IOSTORE_CPU_SCOPE(ParsePackageAssetsFromFiles);
	UE_LOGF(LogIoStore, Display, "Parsing packages...");

	TAtomic<int32> ReadCount {0};
	TAtomic<int32> ParseCount {0};
	const int32 TotalPackageCount = Packages.Num();

	TArray<FPackageFileSummary> PackageFileSummaries;
	PackageFileSummaries.SetNum(TotalPackageCount);

	TArray<FPackageFileSummary> OptionalSegmentPackageFileSummaries;
	OptionalSegmentPackageFileSummaries.SetNum(TotalPackageCount);

	uint8* UAssetMemory = nullptr;
	uint8* OptionalSegmentUAssetMemory = nullptr;

	TArray<uint8*> PackageAssetBuffers;
	PackageAssetBuffers.SetNum(TotalPackageCount);

	TArray<uint8*> OptionalSegmentPackageAssetBuffers;
	OptionalSegmentPackageAssetBuffers.SetNum(TotalPackageCount);

	UE_LOGF(LogIoStore, Display, "Reading package assets...");
	{
		IOSTORE_CPU_SCOPE(ReadUAssetFiles);

		uint64 TotalUAssetSize = 0;
		uint64 TotalOptionalSegmentUAssetSize = 0;
		for (const FCookedPackage* Package : Packages)
		{
			const FLegacyCookedPackage* LegacyPackage = static_cast<const FLegacyCookedPackage*>(Package);
			TotalUAssetSize += LegacyPackage->UAssetSize;
			TotalOptionalSegmentUAssetSize += LegacyPackage->OptionalSegmentUAssetSize;
		}
		UAssetMemory = reinterpret_cast<uint8*>(FMemory::Malloc(TotalUAssetSize));
		uint8* UAssetMemoryPtr = UAssetMemory;
		OptionalSegmentUAssetMemory = reinterpret_cast<uint8*>(FMemory::Malloc(TotalOptionalSegmentUAssetSize));
		uint8* OptionalSegmentUAssetMemoryPtr = OptionalSegmentUAssetMemory;

		for (int32 Index = 0; Index < TotalPackageCount; ++Index)
		{
			FLegacyCookedPackage* Package = static_cast<FLegacyCookedPackage*>(Packages[Index]);
			PackageAssetBuffers[Index] = UAssetMemoryPtr;
			UAssetMemoryPtr += Package->UAssetSize;
			OptionalSegmentPackageAssetBuffers[Index] = OptionalSegmentUAssetMemoryPtr;
			OptionalSegmentUAssetMemoryPtr += Package->OptionalSegmentUAssetSize;
		}

		double StartTime = FPlatformTime::Seconds();

		TAtomic<uint64> TotalReadCount{ 0 };
		TAtomic<uint64> CurrentFileIndex{ 0 };
		ParallelFor(TEXT("ReadingPackageAssets.PF"), TotalPackageCount, 1, [&ReadCount, &PackageAssetBuffers, &OptionalSegmentPackageAssetBuffers, &Packages, &CurrentFileIndex, &TotalReadCount](int32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ReadUAssetFile);
			FLegacyCookedPackage* Package = static_cast<FLegacyCookedPackage*>(Packages[Index]);
			if (Package->UAssetSize)
			{
				TotalReadCount.IncrementExchange();
				uint8* Buffer = PackageAssetBuffers[Index];
				TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*Package->FileName));
				if (FileHandle)
				{
					bool bSuccess = FileHandle->Read(Buffer, Package->UAssetSize);
					UE_CLOGF(!bSuccess, LogIoStore, Warning, "Failed reading file '%ls'", *Package->FileName);
				}
				else
				{
					UE_LOGF(LogIoStore, Warning, "Couldn't open file '%ls'", *Package->FileName);
				}
			}
			if (Package->OptionalSegmentUAssetSize)
			{
				TotalReadCount.IncrementExchange();
				uint8* Buffer = OptionalSegmentPackageAssetBuffers[Index];
				TUniquePtr<IFileHandle> FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*Package->OptionalSegmentFileName));
				if (FileHandle)
				{
					bool bSuccess = FileHandle->Read(Buffer, Package->OptionalSegmentUAssetSize);
					UE_CLOGF(!bSuccess, LogIoStore, Warning, "Failed reading file '%ls'", *Package->OptionalSegmentFileName);
				}
				else
				{
					UE_LOGF(LogIoStore, Warning, "Couldn't open file '%ls'", *Package->OptionalSegmentFileName);
				}
			}

			uint64 LocalFileIndex = CurrentFileIndex.IncrementExchange() + 1;
			UE_CLOGF(LocalFileIndex % 1000 == 0, LogIoStore, Display, "Reading %lld/%d: '%ls'", LocalFileIndex, Packages.Num(), *Package->FileName);
		}, EParallelForFlags::Unbalanced);

		double EndTime = FPlatformTime::Seconds();
		UE_LOGF(LogIoStore, Display, "Packages read %ls files in %.2f seconds, %ls total bytes, %ls bytes per second", 
			*NumberString(TotalReadCount.Load()), 
			EndTime - StartTime, 
			*NumberString(TotalOptionalSegmentUAssetSize + TotalUAssetSize),
			*NumberString((int64)((TotalOptionalSegmentUAssetSize + TotalUAssetSize) / FMath::Max(.001f, EndTime - StartTime))));
	}

	{
		IOSTORE_CPU_SCOPE(SerializeSummaries);

		ParallelFor(TotalPackageCount, [
				&ReadCount,
				&PackageAssetBuffers,
				&OptionalSegmentPackageAssetBuffers,
				&PackageFileSummaries,
				&Packages,
				&PackageStoreOptimizer](int32 Index)
		{
			FLegacyCookedPackage* Package = static_cast<FLegacyCookedPackage*>(Packages[Index]);

			if (Package->UAssetSize)
			{
				uint8* PackageBuffer = PackageAssetBuffers[Index];
				FIoBuffer CookedHeaderBuffer = FIoBuffer(FIoBuffer::Wrap, PackageBuffer, Package->UAssetSize);
				Package->OptimizedPackage = PackageStoreOptimizer.CreatePackageFromCookedHeader(Package->PackageName, CookedHeaderBuffer);
			}
			else
			{
				UE_LOGF(LogIoStore, Display, "Including package %ls without a .uasset file. Excluded by PakFileRules?", *Package->PackageName.ToString());
				Package->OptimizedPackage = PackageStoreOptimizer.CreateMissingPackage(Package->PackageName);
			}
			check(Package->OptimizedPackage->GetId() == Package->GlobalPackageId);
			if (Package->OptionalSegmentUAssetSize)
			{
				uint8* OptionalSegmentPackageBuffer = OptionalSegmentPackageAssetBuffers[Index];
				FIoBuffer OptionalSegmentCookedHeaderBuffer = FIoBuffer(FIoBuffer::Wrap, OptionalSegmentPackageBuffer, Package->OptionalSegmentUAssetSize);
				Package->OptimizedOptionalSegmentPackage = PackageStoreOptimizer.CreatePackageFromCookedHeader(Package->PackageName, OptionalSegmentCookedHeaderBuffer);
				check(Package->OptimizedOptionalSegmentPackage->GetId() == Package->GlobalPackageId);
			}

			// The entry created here will have the correct set of imported packages but the export info will be updated later
			Package->PackageStoreEntry = PackageStoreOptimizer.CreatePackageStoreEntry(Package->OptimizedPackage, Package->OptimizedOptionalSegmentPackage);

		}, EParallelForFlags::Unbalanced);
	}

	FMemory::Free(UAssetMemory);
	FMemory::Free(OptionalSegmentUAssetMemory);
}

static TUniquePtr<FIoStoreReader> CreateIoStoreReader(const TCHAR* Path, const FKeyChain& KeyChain)
{
	TUniquePtr<FIoStoreReader> IoStoreReader(new FIoStoreReader());

	TMap<FGuid, FAES::FAESKey> DecryptionKeys;
	for (const auto& KV : KeyChain.GetEncryptionKeys())
	{
		DecryptionKeys.Add(KV.Key, KV.Value.Key);
	}
	FIoStatus Status = IoStoreReader->Initialize(*FPaths::ChangeExtension(Path, TEXT("")), DecryptionKeys);
	if (Status.IsOk())
	{
		return IoStoreReader;
	}
	else
	{
		UE_LOGF(LogIoStore, Warning, "Failed creating IoStore reader '%ls' [%ls]", Path, *Status.ToString())
		return nullptr;
	}
}

TArray<TUniquePtr<FIoStoreReader>> CreatePatchSourceReaders(const TArray<FString>& Files, const FIoStoreArguments& Arguments)
{
	TArray<TUniquePtr<FIoStoreReader>> Readers;
	for (const FString& PatchSourceContainerFile : Files)
	{
		TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*PatchSourceContainerFile, Arguments.PatchKeyChain);
		if (Reader.IsValid())
		{
			UE_LOGF(LogIoStore, Display, "Loaded patch source container '%ls'", *PatchSourceContainerFile);
			Readers.Add(MoveTemp(Reader));
		}
	}
	return Readers;
}

static void InitializeContainerTargetsAndPackages(
	const FIoStoreArguments& Arguments,
	TArray<FCookedPackage*>& Packages,
	FPackageNameMap& PackageNameMap,
	FPackageIdMap& PackageIdMap,
	TArray<FContainerTargetSpec*>& ContainerTargets)
{
	auto CreateTargetFileFromCookedFile = [
		&Arguments,
		&Packages,
		&PackageNameMap,
		&PackageIdMap](const FContainerSourceFile& SourceFile, FContainerTargetFile& OutTargetFile) -> bool
	{
		const FCookedFileStatData* OriginalCookedFileStatData = Arguments.CookedFileStatMap.Find(SourceFile.NormalizedPath);
		if (!OriginalCookedFileStatData)
		{
			UE_LOGF(LogIoStore, Warning, "File not found: '%ls'", *SourceFile.NormalizedPath);
			return false;
		}

		const FCookedFileStatData* CookedFileStatData = OriginalCookedFileStatData;
		if (CookedFileStatData->FileType == FCookedFileStatData::PackageHeader)
		{
			FStringView NormalizedSourcePathView(SourceFile.NormalizedPath);
			int32 ExtensionStartIndex = GetFullExtensionStartIndex(NormalizedSourcePathView);
			TStringBuilder<512> UexpPath;
			UexpPath.Append(NormalizedSourcePathView.Left(ExtensionStartIndex));
			UexpPath.Append(TEXT(".uexp"));
			CookedFileStatData = Arguments.CookedFileStatMap.Find(*UexpPath);
			if (!CookedFileStatData)
			{
				UE_LOGF(LogIoStore, Warning, "Couldn't find .uexp file for: '%ls'", *SourceFile.NormalizedPath);
				return false;
			}
			OutTargetFile.NormalizedSourcePath = UexpPath;
		}
		else if (CookedFileStatData->FileType == FCookedFileStatData::OptionalSegmentPackageHeader)
		{
			FStringView NormalizedSourcePathView(SourceFile.NormalizedPath);
			int32 ExtensionStartIndex = GetFullExtensionStartIndex(NormalizedSourcePathView);
			TStringBuilder<512> UexpPath;
			UexpPath.Append(NormalizedSourcePathView.Left(ExtensionStartIndex));
			UexpPath.Append(TEXT(".o.uexp"));
			CookedFileStatData = Arguments.CookedFileStatMap.Find(*UexpPath);
			if (!CookedFileStatData)
			{
				UE_LOGF(LogIoStore, Warning, "Couldn't find .o.uexp file for: '%ls'", *SourceFile.NormalizedPath);
				return false;
			}
			OutTargetFile.NormalizedSourcePath = UexpPath;
		}
		else
		{
			OutTargetFile.NormalizedSourcePath = SourceFile.NormalizedPath;
		}
		OutTargetFile.SourceSize = uint64(CookedFileStatData->FileSize);
		
		if (CookedFileStatData->FileType == FCookedFileStatData::ShaderLibrary)
		{
			OutTargetFile.ChunkType = EContainerChunkType::ShaderCodeLibrary;
		}
		else
		{
			FLegacyCookedPackage* Package = FindOrAddLegacyPackage(Arguments, *SourceFile.NormalizedPath, Packages, PackageNameMap, PackageIdMap);
			OutTargetFile.Package = Package;
			if (!OutTargetFile.Package)
			{
				UE_LOGF(LogIoStore, Warning, "Failed to obtain package name from file name '%ls'", *SourceFile.NormalizedPath);
				return false;
			}

			// TODO - Would be nicer to parse this from the package info rather than inferring it from the path
			OutTargetFile.BulkDataCookedIndex = FBulkDataCookedIndex::ParseFromPath(SourceFile.NormalizedPath);

			switch (CookedFileStatData->FileType)
			{
			case FCookedFileStatData::PackageData:
				OutTargetFile.ChunkType = EContainerChunkType::PackageData;
				OutTargetFile.ChunkId = CreateIoChunkId(OutTargetFile.Package->GlobalPackageId.Value(), 0, EIoChunkType::ExportBundleData);
				Package->FileName = SourceFile.NormalizedPath; // .uasset path
				Package->UAssetSize = OriginalCookedFileStatData->FileSize;
				Package->UExpSize = CookedFileStatData->FileSize;
				break;
			case FCookedFileStatData::BulkData:
				OutTargetFile.ChunkType = EContainerChunkType::BulkData;
				OutTargetFile.ChunkId = CreateBulkDataIoChunkId(OutTargetFile.Package->GlobalPackageId.Value(), 0, OutTargetFile.BulkDataCookedIndex.GetValue(), EIoChunkType::BulkData);
				OutTargetFile.Package->TotalBulkDataSize += CookedFileStatData->FileSize;
				break;
			case FCookedFileStatData::OptionalBulkData:
				OutTargetFile.ChunkType = EContainerChunkType::OptionalBulkData;
				OutTargetFile.ChunkId = CreateBulkDataIoChunkId(OutTargetFile.Package->GlobalPackageId.Value(), 0, OutTargetFile.BulkDataCookedIndex.GetValue(), EIoChunkType::OptionalBulkData);
				Package->TotalBulkDataSize += CookedFileStatData->FileSize;
				break;
			case FCookedFileStatData::MemoryMappedBulkData:
				OutTargetFile.ChunkType = EContainerChunkType::MemoryMappedBulkData;
				OutTargetFile.ChunkId = CreateBulkDataIoChunkId(OutTargetFile.Package->GlobalPackageId.Value(), 0, OutTargetFile.BulkDataCookedIndex.GetValue(), EIoChunkType::MemoryMappedBulkData);
				Package->TotalBulkDataSize += CookedFileStatData->FileSize;
				break;
			case FCookedFileStatData::OptionalSegmentPackageData:
				OutTargetFile.ChunkType = EContainerChunkType::OptionalSegmentPackageData;
				OutTargetFile.ChunkId = CreateIoChunkId(OutTargetFile.Package->GlobalPackageId.Value(), 1, EIoChunkType::ExportBundleData);
				Package->OptionalSegmentFileName = SourceFile.NormalizedPath; // .o.uasset path
				Package->OptionalSegmentUAssetSize = OriginalCookedFileStatData->FileSize;
				break;
			case FCookedFileStatData::OptionalSegmentBulkData:
				OutTargetFile.ChunkType = EContainerChunkType::OptionalSegmentBulkData;
				OutTargetFile.ChunkId = CreateBulkDataIoChunkId(OutTargetFile.Package->GlobalPackageId.Value(), 1, OutTargetFile.BulkDataCookedIndex.GetValue(), EIoChunkType::BulkData);
				break;
			default:
				UE_LOGF(LogIoStore, Fatal, "Unexpected file type %d for file '%ls'", CookedFileStatData->FileType, *OutTargetFile.NormalizedSourcePath);
				return false;
			}

			if (CookedFileStatData->FileType == FCookedFileStatData::BulkData ||
				CookedFileStatData->FileType == FCookedFileStatData::OptionalBulkData)
			{
				const FCookedPackageStore::FChunkInfo* ChunkInfo =
					Arguments.PackageStore->GetChunkInfoFromChunkId(OutTargetFile.ChunkId);
				if (!ChunkInfo)
				{
					UE_LOGF(LogIoStore, Warning, "File not found in manifest: '%ls'", *SourceFile.NormalizedPath);
					return false;
				}
				// It is ok for the actual hash to be missing here, if so it will be calculated later
				OutTargetFile.ChunkHash = ChunkInfo->ChunkHash;
			}
		}

		// Only keep the regions for the file if neither compression nor encryption are enabled, otherwise the regions will be meaningless.
		if (Arguments.bFileRegions && !SourceFile.bNeedsCompression && !SourceFile.bNeedsEncryption)
		{
			// Read the matching regions file, if it exists.
			TStringBuilder<512> RegionsFilePath;
			RegionsFilePath.Append(OutTargetFile.NormalizedSourcePath);
			RegionsFilePath.Append(FFileRegion::RegionsFileExtension);
			const FCookedFileStatData* RegionsFileStatData = Arguments.CookedFileStatMap.Find(RegionsFilePath);
			if (RegionsFileStatData)
			{
				TUniquePtr<FArchive> RegionsFile(IFileManager::Get().CreateFileReader(*RegionsFilePath));
				if (!RegionsFile.IsValid())
				{
					UE_LOGF(LogIoStore, Warning, "Failed reading file '%ls'", *RegionsFilePath);
				}
				else
				{
					FFileRegion::SerializeFileRegions(*RegionsFile.Get(), OutTargetFile.FileRegions);
				}
			}
		}

		return true;
	};

	auto CreateTargetFileFromZen = [
		&Arguments,
		&Packages,
		&PackageNameMap,
		&PackageIdMap](const FContainerSourceFile& SourceFile, FContainerTargetFile& OutTargetFile) -> bool
	{
		FCookedPackageStore& PackageStore = *Arguments.PackageStore;
		
		OutTargetFile.NormalizedSourcePath = SourceFile.NormalizedPath;
		
		FStringView Extension = GetFullExtension(SourceFile.NormalizedPath);
		if (Extension == TEXT(".ushaderbytecode"))
		{
			if (const FCookedFileStatData* CookedFileStatData = Arguments.CookedFileStatMap.Find(SourceFile.NormalizedPath))
			{
				OutTargetFile.ChunkType = EContainerChunkType::ShaderCodeLibrary;
				OutTargetFile.SourceSize = uint64(CookedFileStatData->FileSize);
				OutTargetFile.ChunkId = FIoChunkId::InvalidChunkId;
				return true;
			}
			else if (const FCookedPackageStore::FChunkInfo* ChunkInfo = PackageStore.GetChunkInfoFromFileName(SourceFile.NormalizedPath))
			{
				OutTargetFile.ChunkType = EContainerChunkType::ShaderCodeLibrary;
				OutTargetFile.SourceSize = ChunkInfo->ChunkSize;
				OutTargetFile.ChunkId = ChunkInfo->ChunkId;
				return true;
			}
			UE_LOGF(LogIoStore, Warning, "File not found: '%ls'", *SourceFile.NormalizedPath);
			return false;
		}

		const FCookedPackageStore::FChunkInfo* ChunkInfo = PackageStore.GetChunkInfoFromFileName(SourceFile.NormalizedPath);
		if (!ChunkInfo)
		{
			UE_LOGF(LogIoStore, Warning, "File not found in manifest: '%ls'", *SourceFile.NormalizedPath);
			return false;
		}
		OutTargetFile.ChunkId = ChunkInfo->ChunkId;
		if (ChunkInfo->ChunkSize == 0)
		{
			UE_LOGF(LogIoStore, Warning, "Chunk size not found for: '%ls'", *SourceFile.NormalizedPath);
			return false;
		}
		OutTargetFile.SourceSize = ChunkInfo->ChunkSize;
		OutTargetFile.ChunkHash = ChunkInfo->ChunkHash;

		if (ChunkInfo->PackageName.IsNone())
		{
			UE_LOGF(LogIoStore, Warning, "Package name not found for: '%ls'", *SourceFile.NormalizedPath);
			return false;
		}

		OutTargetFile.Package = &FindOrAddPackage(Arguments, ChunkInfo->PackageName, Packages, PackageNameMap, PackageIdMap);
		const FPackageStoreEntryResource* PackageStoreEntry = PackageStore.GetPackageStoreEntry(OutTargetFile.Package->GlobalPackageId);
		if (!PackageStoreEntry)
		{
			UE_LOGF(LogIoStore, Warning, "Failed to find package store entry for package: '%ls'", *ChunkInfo->PackageName.ToString());
			return false;
		}
		OutTargetFile.Package->PackageStoreEntry = *PackageStoreEntry;

		// TODO - Would be nicer to parse this from the package info rather than inferring it from the path
		OutTargetFile.BulkDataCookedIndex = FBulkDataCookedIndex::ParseFromPath(SourceFile.NormalizedPath);

		if (Extension == TEXT(".m.ubulk"))
		{
			OutTargetFile.ChunkType = EContainerChunkType::MemoryMappedBulkData;
			OutTargetFile.Package->TotalBulkDataSize += OutTargetFile.SourceSize;
		}
		else if (Extension == TEXT(".ubulk"))
		{
			OutTargetFile.ChunkType = EContainerChunkType::BulkData;
			OutTargetFile.Package->TotalBulkDataSize += OutTargetFile.SourceSize;
		}
		else if (Extension == TEXT(".uptnl"))
		{
			OutTargetFile.ChunkType = EContainerChunkType::OptionalBulkData;
			OutTargetFile.Package->TotalBulkDataSize += OutTargetFile.SourceSize;
		}
		else if (Extension == TEXT(".uasset") || Extension == TEXT(".umap"))
		{
			OutTargetFile.ChunkType = EContainerChunkType::PackageData;
			OutTargetFile.Package->UAssetSize = OutTargetFile.SourceSize;
		}
		else if (Extension == TEXT(".o.uasset") || Extension == TEXT(".o.umap"))
		{
			OutTargetFile.ChunkType = EContainerChunkType::OptionalSegmentPackageData;
			OutTargetFile.Package->OptionalSegmentUAssetSize = OutTargetFile.SourceSize;
		}
		else if (Extension == TEXT(".o.ubulk"))
		{
			OutTargetFile.ChunkType = EContainerChunkType::OptionalSegmentBulkData;
		}
		else
		{
			UE_LOGF(LogIoStore, Warning, "Unexpected file: '%ls'", *SourceFile.NormalizedPath);
			return false;
		}

		// Only keep the regions for the file if neither compression nor encryption are enabled, otherwise the regions will be meaningless.
		if (Arguments.bFileRegions && !SourceFile.bNeedsCompression && !SourceFile.bNeedsEncryption)
		{
			OutTargetFile.FileRegions = ChunkInfo->FileRegions;
		}
		
		return true;
	};
	
	// Reserve memory for lookup maps
	{
		int32 NumSourceFiles = 0;
		for (const FContainerSourceSpec& ContainerSource : Arguments.Containers)
		{
			NumSourceFiles += ContainerSource.SourceFiles.Num();
		}
		PackageNameMap.Reserve(NumSourceFiles);
		PackageIdMap.Reserve(NumSourceFiles);
	}

	for (const FContainerSourceSpec& ContainerSource : Arguments.Containers)
	{
		FContainerTargetSpec* ContainerTarget = AddContainer(ContainerSource.Name, ContainerTargets);
		ContainerTarget->OutputPath = ContainerSource.OutputPath;
		ContainerTarget->StageLooseFileRootPath = ContainerSource.StageLooseFileRootPath;
		ContainerTarget->bGenerateDiffPatch = ContainerSource.bGenerateDiffPatch;

		if (ContainerSource.bOnDemand)
		{
			ContainerTarget->ContainerFlags |= EIoContainerFlags::OnDemand;
		}

		if (Arguments.bSign)
		{
			ContainerTarget->ContainerFlags |= EIoContainerFlags::Signed;
		}

		if (!ContainerTarget->EncryptionKeyGuid.IsValid() && !ContainerSource.EncryptionKeyOverrideGuid.IsEmpty())
		{
			FGuid::Parse(ContainerSource.EncryptionKeyOverrideGuid, ContainerTarget->EncryptionKeyGuid);
		}

		ContainerTarget->PatchSourceReaders = CreatePatchSourceReaders(ContainerSource.PatchSourceContainerFiles, Arguments);

		{
			IOSTORE_CPU_SCOPE(ProcessSourceFiles);
			bool bHasOptionalSegmentPackages = false;
			ContainerTarget->TargetFiles.Reserve(ContainerSource.SourceFiles.Num());
			for (const FContainerSourceFile& SourceFile : ContainerSource.SourceFiles)
			{
				FContainerTargetFile TargetFile;
				bool bIsValidTargetFile = Arguments.PackageStore->HasZenStoreClient()
					? CreateTargetFileFromZen(SourceFile, TargetFile)
					: CreateTargetFileFromCookedFile(SourceFile, TargetFile);

				if (!bIsValidTargetFile)
				{
					continue;
				}

				TargetFile.ContainerTarget			= ContainerTarget;
				TargetFile.DestinationPath			= SourceFile.DestinationPath;
				TargetFile.bForceUncompressed		= !SourceFile.bNeedsCompression;
				
				if (SourceFile.bNeedsCompression)
				{
					ContainerTarget->ContainerFlags |= EIoContainerFlags::Compressed;
				}

				if (SourceFile.bNeedsEncryption)
				{
					ContainerTarget->ContainerFlags |= EIoContainerFlags::Encrypted;
				}

				if (TargetFile.ChunkType == EContainerChunkType::PackageData)
				{
					check(TargetFile.Package);
					ContainerTarget->Packages.Add(TargetFile.Package);
				}
				else if (TargetFile.ChunkType == EContainerChunkType::OptionalSegmentPackageData)
				{
					bHasOptionalSegmentPackages = true;
				}

				ContainerTarget->TargetFiles.Emplace(MoveTemp(TargetFile));
			}

			if (bHasOptionalSegmentPackages)
			{
				if (ContainerSource.OptionalOutputPath.IsEmpty())
				{
					ContainerTarget->OptionalSegmentOutputPath = ContainerTarget->OutputPath + FPackagePath::GetOptionalSegmentExtensionModifier();
				}
				else
				{
					// if we have an optional output location, use that directory, combined with the name of the output path 
					ContainerTarget->OptionalSegmentOutputPath = FPaths::Combine(ContainerSource.OptionalOutputPath, FPaths::GetCleanFilename(ContainerTarget->OutputPath) + FPackagePath::GetOptionalSegmentExtensionModifier());
				}

				// The IoContainerId is the hash of the name of the container, which gets returned in the results
				// as the output path we provide with the extension removed, which for optional containers means
				// that it contains the .o in the name - so make sure we have a separate id for this.
				ContainerTarget->OptionalSegmentContainerId = FIoContainerId::FromName(*FPaths::GetCleanFilename(ContainerTarget->OptionalSegmentOutputPath));

				UE_LOGF(LogIoStore, Display, "Saving optional container to: '%ls', id: 0x%llx (base container id: 0x%llx)",
					*ContainerTarget->OptionalSegmentOutputPath,
					ContainerTarget->OptionalSegmentContainerId.Value(),
					ContainerTarget->ContainerId.Value());
			}
		}
	}

	Algo::Sort(Packages, [](const FCookedPackage* A, const FCookedPackage* B)
	{
		return A->GlobalPackageId < B->GlobalPackageId;
	});
};

struct FDetectDuplicatesStats
{
	uint64 DeduplicatedChunks = 0;     // chunks with DuplicateOfChunkId set (partial-package dedup)
	uint64 Redirects = 0;              // full-package FPackageRedirectRequest entries emitted
	uint64 PriorReleaseCanonicals = 0; // canonicals chosen from patch-in-place reference DB
	uint64 SizedButNoHash = 0;         // chunks excluded from dedup due to missing zen hash
	uint64 SizedButNoHashBytes = 0;    // sum of their SourceSize (missed-opportunity bytes)
	// on-disk savings are tracked in FIoStoreWriterResult::DeduplicatedSavedBytes, which can only be computed post-compression.

	FDetectDuplicatesStats& operator+=(const FDetectDuplicatesStats& O)
	{
		DeduplicatedChunks     += O.DeduplicatedChunks;
		Redirects              += O.Redirects;
		PriorReleaseCanonicals += O.PriorReleaseCanonicals;
		SizedButNoHash         += O.SizedButNoHash;
		SizedButNoHashBytes    += O.SizedButNoHashBytes;
		return *this;
	}
};

void LogWriterResults(
	const FIoStoreWriterSettings& GeneralIoWriterSettings,
	const TArray<FIoStoreWriterResult>& Results,
	const TMap<FIoContainerId, FDetectDuplicatesStats>& PerWriterDedupStats)
{
	struct FContainerStats
	{
		uint64 TocCount = 0;
		uint64 TocSize = 0;
		uint64 UncompressedContainerSize = 0;
		uint64 CompressedContainerSize = 0;
		uint64 PaddingSize = 0;
		uint64 PatchInPlaceSizeBeforeLayout = 0;
		uint64 PatchInPlacePinnedSizeBeforeDefrag = 0;
		uint64 PatchInPlacePinnedSizeAfterDefrag = 0;
		int64 PatchInPlaceLocalityChange = 0;
	};

	// Container name length can vary wildly, so find the longest one so that we can correctly align the output grids
	int32 ColumnWidth = 10;
	for (const FIoStoreWriterResult& Result : Results)
	{
		int32 ContainerNamePadding = ((Result.ContainerName.Len() / 10) + 1) * 10;
		ColumnWidth = FMath::Max(ColumnWidth, ContainerNamePadding);
	}

	
	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "Container Summary");
	UE_LOGF(LogIoStore, Display, "==================");
	UE_LOGF(LogIoStore, Display, "");
	FString PatchInPlaceHeader = TEXT("");
	if (GeneralIoWriterSettings.bUsePatchInPlaceLayout)
	{
		PatchInPlaceHeader = FString::Printf(TEXT(" %20s %20s %20s %20s"), TEXT("PIP Pinned (MiB)"), TEXT("PIP Defrag (MiB)"), TEXT("PIP Locality (MiB)"), TEXT("PIP Overhead (MiB)"));
	}
	UE_LOGF(LogIoStore, Display, "%-*ls %10ls %15ls %15ls %20ls %20ls%ls", ColumnWidth, TEXT("Container"), TEXT("Flags"), TEXT("Chunk(s) #"), TEXT("TOC (KiB)"), TEXT("Raw Size (MiB)"), TEXT("Size (MiB)"), *PatchInPlaceHeader);

	const int32 SummaryBorderLength = ColumnWidth + 5 + 10 + 15 + 15 + 20 + 20 + (GeneralIoWriterSettings.bUsePatchInPlaceLayout ? 4 + 20 + 20 + 20 + 20 : 0); // +5 to account for spaces between columns
	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(SummaryBorderLength, '-'));
	
	FContainerStats TotalStats;
	FContainerStats OnDemandStats;
	for (const FIoStoreWriterResult& Result : Results)
	{
		FString CompressionInfo = TEXT("-");

		if (Result.CompressionMethod != NAME_None)
		{
			const double Procentage = (double(Result.UncompressedContainerSize - Result.CompressedContainerSize) / double(Result.UncompressedContainerSize)) * 100.0;
			CompressionInfo = FString::Printf(TEXT("(%.2lf%% %s)"),
				Procentage,
				*Result.CompressionMethod.ToString());
		}

		FString PatchInPlaceInfo = TEXT("");
		if (GeneralIoWriterSettings.bUsePatchInPlaceLayout)
		{
			const int64 Overhead = (int64)Result.CompressedContainerSize - Result.PatchInPlaceSizeBeforeLayout;
			const int64 DefragUnpinned = (int64)Result.PatchInPlacePinnedSizeAfterDefrag - Result.PatchInPlacePinnedSizeBeforeDefrag;

			PatchInPlaceInfo = FString::Printf(TEXT(" %20.2lf %20.2lf %20.2lf %20.2lf"),
				(double)Result.PatchInPlacePinnedSizeAfterDefrag / 1024.0 / 1024.0,
				(double)DefragUnpinned / 1024.0 / 1024.0,
				(double)Result.PatchInPlaceLocalityChange / 1024.0 / 1024.0,
				(double)Overhead / 1024.0 / 1024.0
			);
		}

		FString ContainerSettings = FString::Printf(TEXT("%s/%s/%s/%s/%s"),
			EnumHasAnyFlags(Result.ContainerFlags, EIoContainerFlags::Compressed) ? TEXT("C") : TEXT("-"),
			EnumHasAnyFlags(Result.ContainerFlags, EIoContainerFlags::Encrypted) ? TEXT("E") : TEXT("-"),
			EnumHasAnyFlags(Result.ContainerFlags, EIoContainerFlags::Signed) ? TEXT("S") : TEXT("-"),
			EnumHasAnyFlags(Result.ContainerFlags, EIoContainerFlags::Indexed) ? TEXT("I") : TEXT("-"),
			EnumHasAnyFlags(Result.ContainerFlags, EIoContainerFlags::OnDemand) ? TEXT("O") : TEXT("-"));

		FString DeduplicationInfo = TEXT("");
		if (const FDetectDuplicatesStats* DedupEntry = PerWriterDedupStats.Find(Result.ContainerId);
			DedupEntry && (DedupEntry->DeduplicatedChunks > 0 || DedupEntry->Redirects > 0))
		{
			DeduplicationInfo = FString::Printf(TEXT(" (Dedup: %llu chunks, %llu pkg redirects, %.2lf MiB saved)"),
				DedupEntry->DeduplicatedChunks,
				DedupEntry->Redirects,
				(double)Result.DeduplicatedSavedBytes / 1024.0 / 1024.0);
		}

		UE_LOGF(LogIoStore, Display, "%-*ls %10ls %15llu %15.2lf %20.2lf %20.2lf%ls %ls%ls", ColumnWidth,
			*Result.ContainerName,
			*ContainerSettings,
			Result.TocEntryCount,
			(double)Result.TocSize / 1024.0,
			(double)Result.UncompressedContainerSize / 1024.0 / 1024.0,
			(double)Result.CompressedContainerSize / 1024.0 / 1024.0,
			*PatchInPlaceInfo,
			*CompressionInfo,
			*DeduplicationInfo);

		if (EnumHasAnyFlags(Result.ContainerFlags, EIoContainerFlags::OnDemand))
		{
			OnDemandStats.TocCount += Result.TocEntryCount;
			OnDemandStats.TocSize += Result.TocSize;
			OnDemandStats.UncompressedContainerSize += Result.UncompressedContainerSize;
			OnDemandStats.CompressedContainerSize += Result.CompressedContainerSize;
		}

		TotalStats.TocCount += Result.TocEntryCount;
		TotalStats.TocSize += Result.TocSize;
		TotalStats.UncompressedContainerSize += Result.UncompressedContainerSize;
		TotalStats.CompressedContainerSize += Result.CompressedContainerSize;
		TotalStats.PaddingSize += Result.PaddingSize;
		TotalStats.PatchInPlaceSizeBeforeLayout += Result.PatchInPlaceSizeBeforeLayout;
		TotalStats.PatchInPlacePinnedSizeBeforeDefrag += Result.PatchInPlacePinnedSizeBeforeDefrag;
		TotalStats.PatchInPlacePinnedSizeAfterDefrag += Result.PatchInPlacePinnedSizeAfterDefrag;
		TotalStats.PatchInPlaceLocalityChange += Result.PatchInPlaceLocalityChange;
	}

	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(SummaryBorderLength, '-'));

	if (OnDemandStats.TocCount > 0)
	{
		UE_LOGF(LogIoStore, Display, "%-*ls %10ls %15llu %15.2lf %20.2lf %20.2lf", ColumnWidth,
			TEXT("Total On Demand"),
			TEXT(""),
			OnDemandStats.TocCount,
			(double)OnDemandStats.TocSize / 1024.0,
			(double)OnDemandStats.UncompressedContainerSize / 1024.0 / 1024.0,
			(double)OnDemandStats.CompressedContainerSize / 1024.0 / 1024.0);
	}

	FString PatchInPlaceTotal = TEXT("");
	if (GeneralIoWriterSettings.bUsePatchInPlaceLayout)
	{
		const int64 TotalOverhead = (int64)TotalStats.CompressedContainerSize - TotalStats.PatchInPlaceSizeBeforeLayout;
		const int64 DefragUnpinned = (int64)TotalStats.PatchInPlacePinnedSizeAfterDefrag - TotalStats.PatchInPlacePinnedSizeBeforeDefrag;

		PatchInPlaceTotal = FString::Printf(TEXT(" %20.2lf %20.2lf %20.2lf %20.2lf"),
			(double)TotalStats.PatchInPlacePinnedSizeAfterDefrag / 1024.0 / 1024.0,
			(double)DefragUnpinned / 1024.0 / 1024.0,
			(double)TotalStats.PatchInPlaceLocalityChange / 1024.0 / 1024.0,
			(double)TotalOverhead / 1024.0 / 1024.0
		);
	}

	UE_LOGF(LogIoStore, Display, "%-*ls %10ls %15llu %15.2lf %20.2lf %20.2lf%ls", ColumnWidth,
		TEXT("Total"),
		TEXT(""),
		TotalStats.TocCount,
		(double)TotalStats.TocSize / 1024.0,
		(double)TotalStats.UncompressedContainerSize / 1024.0 / 1024.0,
		(double)TotalStats.CompressedContainerSize / 1024.0 / 1024.0,
		*PatchInPlaceTotal);

	{
		uint64 TotalDeduplicatedChunks = 0;
		uint64 TotalDeduplicatedRedirects = 0;
		uint64 TotalDeduplicatedSavedBytes = 0;
		for (const FIoStoreWriterResult& Result : Results)
		{
			TotalDeduplicatedSavedBytes += Result.DeduplicatedSavedBytes;
			if (const FDetectDuplicatesStats* DedupEntry = PerWriterDedupStats.Find(Result.ContainerId))
			{
				TotalDeduplicatedChunks    += DedupEntry->DeduplicatedChunks;
				TotalDeduplicatedRedirects += DedupEntry->Redirects;
			}
		}
		if (TotalDeduplicatedChunks > 0 || TotalDeduplicatedRedirects > 0)
		{
			UE_LOGF(LogIoStore, Display, "Dedup total: %llu chunks, %llu pkg redirects, %.2lf MiB saved",
				TotalDeduplicatedChunks, TotalDeduplicatedRedirects, (double)TotalDeduplicatedSavedBytes / 1024.0 / 1024.0);
		}
	}

	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "** Flags: (C)ompressed / (E)ncrypted / (S)igned) / (I)ndexed) / (O)nDemand **");
	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "Compression block padding: %8.2lf MiB", (double)TotalStats.PaddingSize / 1024.0 / 1024.0);
	UE_LOGF(LogIoStore, Display, "");

	UE_LOGF(LogIoStore, Display, "Container Directory Index");
	UE_LOGF(LogIoStore, Display, "==========================");
	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "%-*ls %15ls", ColumnWidth, TEXT("Container"), TEXT("Size (KiB)"));
	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(ColumnWidth + 1 +15, '-')); // +1 to account for spaces between columns

	for (const FIoStoreWriterResult& Result : Results)
	{
		UE_LOGF(LogIoStore, Display, "%-*ls %15.2lf", ColumnWidth, *Result.ContainerName, double(Result.DirectoryIndexSize) / 1024.0);
	}

	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "Container Patch Report");
	UE_LOGF(LogIoStore, Display, "========================");
	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "%-*ls %16ls %16ls %16ls %16ls %16ls", ColumnWidth, TEXT("Container"), TEXT("Total #"), TEXT("Modified #"), TEXT("Added #"), TEXT("Modified (MiB)"), TEXT("Added (MiB)"));
	
	const int32 PatchReportBorderLength = ColumnWidth + 5 + 16 + 16 + 16 + 16 + 16; // +5 to account for spaces between columns
	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(PatchReportBorderLength, '-'));

	for (const FIoStoreWriterResult& Result : Results)
	{
		UE_LOGF(LogIoStore, Display, "%-*ls %16lld %16lld %16lld %16.2lf %16.2lf", ColumnWidth, *Result.ContainerName, Result.TocEntryCount, Result.ModifiedChunksCount, Result.AddedChunksCount, Result.ModifiedChunksSize / 1024.0 / 1024.0, Result.AddedChunksSize / 1024.0 / 1024.0);
	}

	if (GeneralIoWriterSettings.bUsePatchInPlaceLayout && GeneralIoWriterSettings.PatchInPlaceMaximumTotalGrowth > 0)
	{
		const int64 TotalOverhead = (int64)TotalStats.CompressedContainerSize - TotalStats.PatchInPlaceSizeBeforeLayout;

		if (TotalOverhead > (int64)GeneralIoWriterSettings.PatchInPlaceMaximumTotalGrowth)
		{
			UE_LOGF(LogIoStore, Warning, "Patch-in-place total growth %.2f MiB exceeds limit %.2f MiB. Packaged data is valid.",
				(double)TotalOverhead / 1024.0 / 1024.0,
				(double)GeneralIoWriterSettings.PatchInPlaceMaximumTotalGrowth / 1024.0 / 1024.0);
		}
	}

	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(PatchReportBorderLength, '-'));
}

void LogContainerPackageInfo(const TArray<FContainerTargetSpec*>& ContainerTargets)
{
	uint64 TotalStoreSize = 0;
	uint64 TotalPackageCount = 0;
	uint64 TotalLocalizedPackageCount = 0;

	// Container name length can vary wildly, so find the longest one so that we can correctly align the output grids
	int32 ColumnWidth = 10;
	for (const FContainerTargetSpec* ContainerTarget : ContainerTargets)
	{
		int32 ContainerNamePadding = ((ContainerTarget->Name.ToString().Len() / 10) + 1) * 10;
		ColumnWidth = FMath::Max(ColumnWidth, ContainerNamePadding);
	}

	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "PackageStore");
	UE_LOGF(LogIoStore, Display, "=============");
	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "%-*ls %15ls %15ls %15ls", ColumnWidth,
		TEXT("Container"),
		TEXT("Size (KiB)"),
		TEXT("Packages #"),
		TEXT("Localized #"));

	const int32 BorderLength = ColumnWidth + 3 + 15 + 15 + 15; // +3 to account for spaces between columns
	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(BorderLength, '-'));

	for (const FContainerTargetSpec* ContainerTarget : ContainerTargets)
	{
		uint64 StoreSize = ContainerTarget->Header.StoreEntries.Num();
		uint64 PackageCount = ContainerTarget->Packages.Num();
		uint64 LocalizedPackageCount = ContainerTarget->Header.LocalizedPackages.Num();

		UE_LOGF(LogIoStore, Display, "%-*ls %15.0lf %15llu %15llu", ColumnWidth,
			*ContainerTarget->Name.ToString(),
			(double)StoreSize / 1024.0,
			PackageCount,
			LocalizedPackageCount);

		TotalStoreSize += StoreSize;
		TotalPackageCount += PackageCount;
		TotalLocalizedPackageCount += LocalizedPackageCount;
	}

	UE_LOGF(LogIoStore, Display, "%ls", *FString::ChrN(BorderLength, '-'));
	UE_LOGF(LogIoStore, Display, "%-*ls %15.0lf %15llu %15llu", ColumnWidth,
		TEXT("Total"),
		(double)TotalStoreSize / 1024.0,
		TotalPackageCount,
		TotalLocalizedPackageCount);

	UE_LOGF(LogIoStore, Display, "");
	UE_LOGF(LogIoStore, Display, "");
}

class FIoStoreWriteRequestManager
{
public:
	FIoStoreWriteRequestManager(FPackageStoreOptimizer& InPackageStoreOptimizer, FCookedPackageStore* InPackageStore)
		: PackageStoreOptimizer(InPackageStoreOptimizer)
		, PackageStore(InPackageStore)
	{
		InitiatorThread = Async(EAsyncExecution::Thread, [this]() { InitiatorThreadFunc(); });
		RetirerThread = Async(EAsyncExecution::Thread, [this]() { RetirerThreadFunc(); });

		MaxSourceBufferMemory = 3ull << 30;
		FParse::Value(FCommandLine::Get(), TEXT("MaxSourceBufferMemory="), MaxSourceBufferMemory);

		MaxConcurrentSourceReads = uint32(FMath::Clamp(FPlatformMisc::NumberOfCoresIncludingHyperthreads()/2, 4, 32));
		FParse::Value(FCommandLine::Get(), TEXT("MaxConcurrentSourceReads="), MaxConcurrentSourceReads);

		UE_LOGF(LogIoStore, Display, "Initialized WriteRequestManager with MaxConcurrentSourceReads=%d (%d cores), MaxSourceBufferMemory=%lluMiB",
			MaxConcurrentSourceReads, FPlatformMisc::NumberOfCoresIncludingHyperthreads(), MaxSourceBufferMemory >> 20);
	}

	~FIoStoreWriteRequestManager()
	{
		InitiatorQueue.CompleteAdding();
		RetirerQueue.CompleteAdding();
		InitiatorThread.Wait();
		RetirerThread.Wait();
	}

	void DisableMemoryThrottling()
	{
		MaxSourceBufferMemory = 0;
		UE_LOGF(LogIoStore, Display, "Disabled WriteRequestManager memory throttling, MaxConcurrentSourceReads=%d (%d cores)",
			MaxConcurrentSourceReads, FPlatformMisc::NumberOfCoresIncludingHyperthreads());
	}

	IIoStoreWriteRequest* Read(const FContainerTargetFile& InTargetFile)
	{
		if (InTargetFile.SourceBuffer.IsSet())
		{
			return new FInMemoryWriteRequest(*this, InTargetFile);
		}
		else if (PackageStore->HasZenStoreClient())
		{
			return new FZenWriteRequest(*this, InTargetFile);
		}
		else
		{
			return new FLooseFileWriteRequest(*this, InTargetFile);
		}
	}

private:
	struct FQueueEntry;
	
	class FWriteContainerTargetFileRequest
		: public IIoStoreWriteRequest
	{
		friend class FIoStoreWriteRequestManager;

	public:
		virtual ~FWriteContainerTargetFileRequest()
		{
		}

		virtual uint64 GetOrderHint() override
		{
			return TargetFile.IdealOrder;
		}

		virtual TArrayView<const FFileRegion> GetRegions() override
		{
			return FileRegions;
		}

		virtual const FIoHash* GetChunkHash() override
		{
			return TargetFile.ChunkHash.IsZero() ? nullptr : &TargetFile.ChunkHash;
		}
		
		virtual void PrepareSourceBufferAsync(UE::Tasks::FTaskEvent& InCompletionEvent) override
		{
			CompletionEvent.Emplace(InCompletionEvent);
			Manager.ScheduleLoad(this);
			bLoadScheduled = true;
		}

		virtual const FIoBuffer* GetSourceBuffer() override
		{
			return bSourceBufferValid ? &SourceBuffer : nullptr;
		}
		
		virtual void FreeSourceBuffer() override
		{
			if (bLoadScheduled)
			{
				bLoadScheduled = false;
				bSourceBufferValid = false;
				SourceBuffer = FIoBuffer();
				Manager.OnBufferMemoryFreed(SourceBufferSize);
			}
		}

		virtual uint64 GetSourceBufferSizeEstimate() override
		{
			return SourceBufferSize;
		}

		virtual void LoadSourceBufferAsync() = 0;

	protected:
		FWriteContainerTargetFileRequest(FIoStoreWriteRequestManager& InManager,const FContainerTargetFile& InTargetFile)
			: Manager(InManager)
			, TargetFile(InTargetFile)
			, FileRegions(TargetFile.FileRegions)
			, SourceBufferSize(TargetFile.SourceSize) { }

		void OnSourceBufferLoaded(bool bSuccess, bool bTriggerCompletionEvent)
		{
			TRACE_COUNTER_DECREMENT(IoStoreSourceReadsInflight);
			TRACE_COUNTER_INCREMENT(IoStoreSourceReadsDone);
			bSourceBufferValid = bSuccess;
			QueueEntry->ReleaseRef(Manager);
			if (bTriggerCompletionEvent)
			{
				CompletionEvent.GetValue().Trigger();
			}
		}

		FIoStoreWriteRequestManager& Manager;
		const FContainerTargetFile& TargetFile;
		TArray<FFileRegion> FileRegions;

		// Note -- this is filled with the TargetFile.SourceSize value which is the size of the buffer 
		// used for IO, however it's not necessarily the size of the resulting input to iostore as
		// the buffer can be post-processed after i/o (e.g. CreateOptimizedPackage).
		uint64 SourceBufferSize;
		TOptional<UE::Tasks::FTaskEvent> CompletionEvent;
		FIoBuffer SourceBuffer;
		FQueueEntry* QueueEntry = nullptr;
		bool bLoadScheduled = false;
		bool bSourceBufferValid = false;
	};

	class FInMemoryWriteRequest
		: public FWriteContainerTargetFileRequest
	{
	public:
		FInMemoryWriteRequest(FIoStoreWriteRequestManager& InManager, const FContainerTargetFile& InTargetFile)
			: FWriteContainerTargetFileRequest(InManager, InTargetFile) { }

		virtual void LoadSourceBufferAsync() override
		{
			Manager.MemorySourceReads[(int8)TargetFile.ChunkId.GetChunkType()].IncrementExchange();
			SourceBuffer = TargetFile.SourceBuffer.GetValue();
			Manager.MemorySourceBytes[(int8)TargetFile.ChunkId.GetChunkType()] += SourceBuffer.DataSize();
			OnSourceBufferLoaded(/*bSuccess*/ true, /*bTriggerCompletionEvent*/ true);
		}

		virtual const TCHAR* DebugNameOfRepository() const override
		{
			return TEXT("InMemory");
		}
	};

	// Used when staging from cooked files
	class FLooseFileWriteRequest
		: public FWriteContainerTargetFileRequest
	{
	public:
		FLooseFileWriteRequest(FIoStoreWriteRequestManager& InManager, const FContainerTargetFile& InTargetFile)
			: FWriteContainerTargetFileRequest(InManager, InTargetFile)
			, Package(static_cast<FLegacyCookedPackage*>(InTargetFile.Package))
		{
		}

		virtual void LoadSourceBufferAsync() override
		{
			Manager.LooseFileSourceReads[(int8)TargetFile.ChunkId.GetChunkType()].IncrementExchange();
			SourceBuffer = FIoBuffer(GetSourceBufferSizeEstimate());

			QueueEntry->FileHandle.Reset(
				FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(*TargetFile.NormalizedSourcePath));
			
			QueueEntry->AddRef(); // Must keep it around until we've assigned the ReadRequest pointer
			FAsyncFileCallBack Callback = [this](bool, IAsyncReadRequest* ReadRequest)
			{
				Manager.LooseFileSourceBytes[(int8)TargetFile.ChunkId.GetChunkType()] += SourceBuffer.DataSize();

				if (TargetFile.ChunkType == EContainerChunkType::PackageData)
				{
					SourceBuffer = Manager.PackageStoreOptimizer.CreatePackageBuffer(Package->OptimizedPackage, SourceBuffer);
				}
				else if (TargetFile.ChunkType == EContainerChunkType::OptionalSegmentPackageData)
				{
					check(Package->OptimizedOptionalSegmentPackage);
					SourceBuffer = Manager.PackageStoreOptimizer.CreatePackageBuffer(Package->OptimizedOptionalSegmentPackage, SourceBuffer);
				}
				OnSourceBufferLoaded(/*bSuccess*/ true, /*bTriggerCompletionEvent*/ true);
			};

			QueueEntry->ReadRequest.Reset(
				QueueEntry->FileHandle->ReadRequest(0, SourceBuffer.DataSize(), AIOP_Normal, &Callback, SourceBuffer.Data()));
			QueueEntry->ReleaseRef(Manager);
		}

		virtual const TCHAR* DebugNameOfRepository() const override
		{
			return TEXT("LooseFile");
		}

	private:
		FLegacyCookedPackage* Package;
	};

	class FZenWriteRequest
		: public FWriteContainerTargetFileRequest
	{
	public:
		FZenWriteRequest(FIoStoreWriteRequestManager& InManager,const FContainerTargetFile& InTargetFile)
			: FWriteContainerTargetFileRequest(InManager, InTargetFile) {}

		virtual void LoadSourceBufferAsync() override
		{
			Manager.ZenSourceReads[(int8)TargetFile.ChunkId.GetChunkType()].IncrementExchange();
			UE::Tasks::FTask ReadTask = Manager.PackageStore->ReadChunkAsync(
				TargetFile.ChunkId,
				[this](TIoStatusOr<FIoBuffer> Status)
				{
					const bool bSuccess = Status.IsOk();
					if (bSuccess)
					{
						SourceBuffer = Status.ConsumeValueOrDie();
						Manager.ZenSourceBytes[(int8)TargetFile.ChunkId.GetChunkType()] += SourceBuffer.DataSize();
					}
					OnSourceBufferLoaded(bSuccess, /*bTriggerCompletionEvent*/ false);
				});
			CompletionEvent.GetValue().AddPrerequisites(ReadTask);
			CompletionEvent.GetValue().Trigger();
		}

		virtual const TCHAR* DebugNameOfRepository() const override
		{
			return TEXT("ZenServer");
		}
	};

	struct FQueueEntry
	{
		FQueueEntry* Next = nullptr;
		TUniquePtr<IAsyncReadFileHandle> FileHandle;
		TUniquePtr<IAsyncReadRequest> ReadRequest;
		FWriteContainerTargetFileRequest* WriteRequest = nullptr;

		void AddRef()
		{
			++RefCount;
		}

		void ReleaseRef(FIoStoreWriteRequestManager& Manager)
		{
			if (--RefCount == 0)
			{
				Manager.ScheduleRetire(this);
			}
		}

	private:
		TAtomic<int32> RefCount{ 1 };
	};

	class FQueue
	{
	public:
		FQueue()
			: Event(FPlatformProcess::GetSynchEventFromPool(false))
		{ }

		~FQueue()
		{
			check(Head == nullptr && Tail == nullptr);
			FPlatformProcess::ReturnSynchEventToPool(Event);
		}

		void Enqueue(FQueueEntry* Entry)
		{
			check(!bIsDoneAdding);
			{
				FScopeLock _(&CriticalSection);

				if (!Tail)
				{
					Head = Tail = Entry;
				}
				else
				{
					Tail->Next = Entry;
					Tail = Entry;
				}
				Entry->Next = nullptr;
			}

			Event->Trigger();
		}

		FQueueEntry* DequeueOrWait()
		{
			for (;;)
			{
				{
					FScopeLock _(&CriticalSection);
					if (Head)
					{
						FQueueEntry* Entry = Head;
						Head = Tail = nullptr;
						return Entry;
					}
				}

				if (bIsDoneAdding)
				{
					break;
				}

				Event->Wait();
			}

			return nullptr;
		}

		void CompleteAdding()
		{
			bIsDoneAdding = true;
			Event->Trigger();
		}

	private:
		FCriticalSection CriticalSection;
		FEvent* Event = nullptr;
		FQueueEntry* Head = nullptr;
		FQueueEntry* Tail = nullptr;
		TAtomic<bool> bIsDoneAdding{ false };
	};

	void ScheduleLoad(FWriteContainerTargetFileRequest* WriteRequest)
	{
		FQueueEntry* QueueEntry = new FQueueEntry();
		QueueEntry->WriteRequest = WriteRequest;
		WriteRequest->QueueEntry = QueueEntry;
		InitiatorQueue.Enqueue(QueueEntry);
	}

	void ScheduleRetire(FQueueEntry* QueueEntry)
	{
		--NumConcurrentSourceReads;
		SourceReadCompletedEvent->Trigger();
		RetirerQueue.Enqueue(QueueEntry);
	}

	void Start(FQueueEntry* QueueEntry)
	{
		const uint64 SourceBufferSize = QueueEntry->WriteRequest->GetSourceBufferSizeEstimate();

		while (NumConcurrentSourceReads >= MaxConcurrentSourceReads)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForSourceReads);
			SourceReadCompletedEvent->Wait();
		}

		if (MaxSourceBufferMemory)
		{
			uint64 LocalUsedBufferMemory = UsedBufferMemory.Load();
			while (LocalUsedBufferMemory > 0 && LocalUsedBufferMemory + SourceBufferSize > MaxSourceBufferMemory)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WaitForBufferMemory);
				MemoryAvailableEvent->Wait();
				LocalUsedBufferMemory = UsedBufferMemory.Load();
			}
		}

		UsedBufferMemory.AddExchange(SourceBufferSize);
		TRACE_COUNTER_INCREMENT(IoStoreSourceReadsInflight);
		TRACE_COUNTER_ADD(IoStoreSourceReadsUsedBufferMemory, SourceBufferSize);
		QueueEntry->WriteRequest->LoadSourceBufferAsync();
		++NumConcurrentSourceReads;
	}

	void Retire(FQueueEntry* QueueEntry)
	{
		if (QueueEntry->ReadRequest.IsValid())
		{
			QueueEntry->ReadRequest->WaitCompletion();
			QueueEntry->ReadRequest.Reset();
			QueueEntry->FileHandle.Reset();
		}
		delete QueueEntry;
	}

	void OnBufferMemoryFreed(uint64 Size)
	{
		uint64 OldSize = UsedBufferMemory.SubExchange(Size);
		check(OldSize >= Size);
		TRACE_COUNTER_SUBTRACT(IoStoreSourceReadsUsedBufferMemory, Size);
		MemoryAvailableEvent->Trigger();
	}

	void InitiatorThreadFunc()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SourceReadInitiatorThread);
		for (;;)
		{
			FQueueEntry* QueueEntry = InitiatorQueue.DequeueOrWait();
			if (!QueueEntry)
			{
				return;
			}
			while (QueueEntry)
			{
				FQueueEntry* Next = QueueEntry->Next;
				Start(QueueEntry);
				QueueEntry = Next;
			}
		}
	}

	void RetirerThreadFunc()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SourceReadRetirerThread);
		for (;;)
		{
			FQueueEntry* QueueEntry = RetirerQueue.DequeueOrWait();
			if (!QueueEntry)
			{
				return;
			}
			while (QueueEntry)
			{
				FQueueEntry* Next = QueueEntry->Next;
				Retire(QueueEntry);
				QueueEntry = Next;
			}
		}
	}

	FPackageStoreOptimizer& PackageStoreOptimizer;
	FCookedPackageStore* PackageStore;
	TFuture<void> InitiatorThread;
	TFuture<void> RetirerThread;
	FQueue InitiatorQueue;
	FQueue RetirerQueue;
	uint64 MaxSourceBufferMemory = 0;
	TAtomic<uint64> UsedBufferMemory { 0 };
	int32 MaxConcurrentSourceReads = 0;
	TAtomic<int32> NumConcurrentSourceReads { 0 };
	FEventRef MemoryAvailableEvent;
	FEventRef SourceReadCompletedEvent;

public:
	TAtomic<uint64> ZenSourceReads[(int8)EIoChunkType::MAX] { 0 };
	TAtomic<uint64> ZenSourceBytes[(int8)EIoChunkType::MAX]{ 0 };
	TAtomic<uint64> MemorySourceReads[(int8)EIoChunkType::MAX]{ 0 };
	TAtomic<uint64> MemorySourceBytes[(int8)EIoChunkType::MAX]{ 0 };
	TAtomic<uint64> LooseFileSourceReads[(int8)EIoChunkType::MAX]{ 0 };
	TAtomic<uint64> LooseFileSourceBytes[(int8)EIoChunkType::MAX]{ 0 };
};

static bool WriteUtf8StringView(FUtf8StringView InView, const FString& InFilename)
{
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*InFilename, 0));
	if (!Ar)
	{
		return false;
	}
	UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
	Ar->Serialize(&UTF8BOM, sizeof(UTF8BOM));
	Ar->Serialize((void*)InView.GetData(), InView.Len() * sizeof(UTF8CHAR));
	Ar->Close();
	return true;
}

// When we emit plugin size information, we emit one json for each class of sizes we
// are monitoring.
enum class EPluginGraphSizeClass : uint8
{
	All,
	Texture,
	StaticMesh,
	SoundWave,
	SkeletalMesh,
	Shader,
	Level, 
	Animation,
	Niagara,
	Material,
	Blueprint,
	Geometry,
	Other,
	COUNT
};

static const UTF8CHAR* PluginGraphEntryClassNames[] = 
{
	UTF8TEXT("all"),
	UTF8TEXT("texture"),
	UTF8TEXT("staticmesh"),
	UTF8TEXT("soundwave"),
	UTF8TEXT("skeletalmesh"),
	UTF8TEXT("shader"),
	UTF8TEXT("level"),
	UTF8TEXT("animation"),
	UTF8TEXT("niagara"),
	UTF8TEXT("material"),
	UTF8TEXT("blueprint"),
	UTF8TEXT("geometry"),
	UTF8TEXT("other")
};

static_assert( UE_ARRAY_COUNT(PluginGraphEntryClassNames) == (size_t)EPluginGraphSizeClass::COUNT, "Must have a name for each plugin graph size class!");

struct FPluginGraphEntry
{
	uint16 IndexInEnabledPlugins = 0;

	FString Name;

	TSet<FPluginGraphEntry*> DirectDependencies;
	TSet<FPluginGraphEntry*> TotalDependencies;
	TSet<FPluginGraphEntry*> Roots;

	// Only valid if bIsRoot
	TSet<FPluginGraphEntry*> UniqueDependencies;

	uint32 DirectRefcount = 0;
	bool bIsRoot = false;

	static constexpr uint8 ClassCount = (uint8)EPluginGraphSizeClass::COUNT;
	UE::Cook::FPluginSizeInfo ExclusiveSizes[ClassCount];
	UE::Cook::FPluginSizeInfo InclusiveSizes[ClassCount];
	UE::Cook::FPluginSizeInfo UniqueSizes[ClassCount];

	uint64 ExclusiveCounts[ClassCount] = {};
	uint64 InclusiveCounts[ClassCount] = {};
	uint64 UniqueCounts[ClassCount] = {};
};

struct FPluginGraph
{
	TArray<FPluginGraphEntry> Plugins;

	TMap<FStringView, FPluginGraphEntry*> NameToPlugin;
	TArray<FPluginGraphEntry*> TopologicallySortedPlugins;
	TArray<FPluginGraphEntry*> RootPlugins;

	// Plugins that can't trace a route between a plugin with bIsRoot==true and themselves.
	TSet<FPluginGraphEntry*> UnrootedPlugins;
};


// Rework the hierarchy in to a graph where we have output edges resolved to pointers so we can pass to
// library functions.
static void GeneratePluginGraph(const UE::Cook::FCookMetadataPluginHierarchy& InPluginHierarchy, FPluginGraph& OutPluginGraph)
{
	double GeneratePluginGraphStart = FPlatformTime::Seconds();

	// Allocate up front so our pointer remains stable.
	OutPluginGraph.Plugins.Reserve(InPluginHierarchy.PluginsEnabledAtCook.Num());
	uint16 PluginIndex = 0;
	for (const UE::Cook::FCookMetadataPluginEntry& Plugin : InPluginHierarchy.PluginsEnabledAtCook)
	{
		FPluginGraphEntry& OurEntry = OutPluginGraph.Plugins.AddDefaulted_GetRef();
		OurEntry.IndexInEnabledPlugins = PluginIndex;
		OurEntry.Name = Plugin.Name;

		// Can store pointer since we reserved as a batch..
		OutPluginGraph.NameToPlugin.Add(OurEntry.Name, &OurEntry);
		PluginIndex++;
	}

	OutPluginGraph.RootPlugins.Reserve(InPluginHierarchy.RootPlugins.Num());
	for (uint16 RootIndex : InPluginHierarchy.RootPlugins)
	{
		FPluginGraphEntry* Root = OutPluginGraph.NameToPlugin[InPluginHierarchy.PluginsEnabledAtCook[RootIndex].Name];
		Root->bIsRoot = true;
		OutPluginGraph.RootPlugins.Add(Root);
	}

	for (FPluginGraphEntry& PluginEntry : OutPluginGraph.Plugins)
	{
		const UE::Cook::FCookMetadataPluginEntry& Plugin = InPluginHierarchy.PluginsEnabledAtCook[PluginEntry.IndexInEnabledPlugins];

		for (uint32 DependencyIndex = Plugin.DependencyIndexStart; DependencyIndex < Plugin.DependencyIndexEnd; DependencyIndex++)
		{
			const UE::Cook::FCookMetadataPluginEntry& DependentPlugin = InPluginHierarchy.PluginsEnabledAtCook[InPluginHierarchy.PluginDependencies[DependencyIndex]];
			PluginEntry.DirectDependencies.Add(OutPluginGraph.NameToPlugin[DependentPlugin.Name]);
		}
	}
	
	// From here on out we can operate entirely on our own data - no cook metadata structures - 
	// and we generate the various structures we need.

	// Sort the plugins topologically. This means that when we iterate linearly,
	// we know that when we hit a plugin, we've already processed the dependencies.
	// This takes some memory to track edges but is a depth first search
	// and not anything quadratic or worse.	
	double TopologicalSortStart = FPlatformTime::Seconds();
	{
		OutPluginGraph.TopologicallySortedPlugins.Reserve(OutPluginGraph.Plugins.Num());
		for (FPluginGraphEntry& Plugin : OutPluginGraph.Plugins)
		{
			OutPluginGraph.TopologicallySortedPlugins.Add(&Plugin);
		}

		auto GetElementDependencies = [&OutPluginGraph](const FPluginGraphEntry* PluginEntry) -> const TSet<FPluginGraphEntry*>&
		{
			return OutPluginGraph.NameToPlugin[PluginEntry->Name]->DirectDependencies;
		};

		Algo::TopologicalSort(OutPluginGraph.TopologicallySortedPlugins, GetElementDependencies);
	}


	// Gather the set of all dependencies. This ends up being technically
	// O(N^2) in the worst case. It's highly unlikely our plugin DAG will cause that, but we track the times
	// just so we can keep an eye on it if it ends up taking measurable amounts of time.
	double InclusiveComputeStart = FPlatformTime::Seconds();
	for (FPluginGraphEntry* Plugin : OutPluginGraph.TopologicallySortedPlugins)
	{
		for (FPluginGraphEntry* Dependency : Plugin->DirectDependencies)
		{
			Plugin->TotalDependencies.Add(Dependency);
			Dependency->DirectRefcount++;
		
			// In the worse case this is another O(N) iteration, which makes us overall O(N^2)
			for (FPluginGraphEntry* TotalDependencyEntry : Dependency->TotalDependencies)
			{
				Plugin->TotalDependencies.Add(TotalDependencyEntry);
			}
		}
	}
	double InclusiveComputeEnd = FPlatformTime::Seconds();

	// Generate the unique dependencies for the root plugins. This is the set of dependencies that only
	// belong to the root plugin and not to another. These dependencies could be referred to by another 
	// plugin within the unique set - the only requirement is that there exists no path from _another_
	// root to the dependency.
	for (FPluginGraphEntry* RootPlugin : OutPluginGraph.RootPlugins)
	{
		// Add us as a root entry for all our total dependencies so all plugins know which root they are in.
		for (FPluginGraphEntry* Dependency : RootPlugin->TotalDependencies)
		{
			OutPluginGraph.NameToPlugin[Dependency->Name]->Roots.Add(RootPlugin);
		}

		// Duplicate the TotalDependencies and then remove any dependency that
		// exists for another root.
		RootPlugin->UniqueDependencies = RootPlugin->TotalDependencies;
		for (FPluginGraphEntry* InnerRootPlugin : OutPluginGraph.RootPlugins)
		{
			if (RootPlugin == InnerRootPlugin)
			{
				continue;
			}

			for (FPluginGraphEntry* InnerRootDependency : InnerRootPlugin->TotalDependencies)
			{
				RootPlugin->UniqueDependencies.Remove(InnerRootDependency);
			}
		}
	}

	double UniqueEnd = FPlatformTime::Seconds();

	// Generate the unrooted set. These plugins can not trace a path from _any_ root plugin
	// to themselves. For projects with no root plugins, this will be all plugins.
	{
		for (FPluginGraphEntry* Plugin : OutPluginGraph.TopologicallySortedPlugins)
		{
			OutPluginGraph.UnrootedPlugins.Add(Plugin);
		}

		for (FPluginGraphEntry* Plugin : OutPluginGraph.RootPlugins)
		{
			OutPluginGraph.UnrootedPlugins.Remove(Plugin);
			for (FPluginGraphEntry* Dependency : Plugin->TotalDependencies)
			{
				OutPluginGraph.UnrootedPlugins.Remove(Plugin);
			}
		}
	}
	double GeneratePluginGraphEnd = FPlatformTime::Seconds();

	UE_LOGF(LogIoStore, Log, "Generated plugin graph with %d nodes. Times: %.02f total %.02f setup, %.02f sort %.02f inclusive %.02f unique %.02f unrooted.",
		OutPluginGraph.TopologicallySortedPlugins.Num(),
		GeneratePluginGraphEnd - GeneratePluginGraphStart,
		TopologicalSortStart - GeneratePluginGraphStart,
		InclusiveComputeStart - TopologicalSortStart,
		InclusiveComputeEnd - InclusiveComputeStart,
		UniqueEnd - InclusiveComputeEnd,
		GeneratePluginGraphEnd - UniqueEnd);
}

static void InsertShadersInPluginHierarchy(UE::Cook::FCookMetadataState& InCookMetadata, FPluginGraph& InPluginGraph, FShaderAssociationInfo& InShaderAssociationInfo)
{
	const UE::Cook::FCookMetadataPluginHierarchy& PluginHierarchy = InCookMetadata.GetPluginHierarchy();

	//
	// Create any shader plugins we need. These are pseudo plugins that we create to hold the size information
	// when the packages that reference a plugin cross the plugin boundary. We name them based on their dependencies
	// so it's consistent across builds. These will exist whenever GFPs aren't placed entirely in their own pak chunk.
	//
	// The combinatorics are such that doing this for _all_ plugins isn't tenable. However, for product tracking we
	// actually only care about root GFPs. So instead of gathering all of the plugins entirely, we gather all of the
	// root plugins.
	//
	// The difficulty is that we don't necessarily _have_ any root plugins if the project hasn't defined any, so we 
	// artificially stuff such plugins under "Unrooted".
	//
	// It should be noted that the entire point of root GFPs is to separate the data entirely - so if we have any
	// of these pseudo plugins then there is a content bug as there exists a shared dependency between two "modes".
	//
	TMap<FString, TArray<FShaderAssociationInfo::FShaderChunkInfoKey>> ShaderPseudoPlugins;
	TMap<FString, TArray<FString>> PluginDependenciesOnShaders;

	TSet<FString> ShaderRootPossibles;
	ShaderRootPossibles.Add(TEXT("Unrooted"));

	for (TPair<FShaderAssociationInfo::FShaderChunkInfoKey, FShaderAssociationInfo::FShaderChunkInfo>& ShaderChunkInfo : InShaderAssociationInfo.ShaderChunkInfos)
	{
		// Only Normal shaders can be assigned
		if (ShaderChunkInfo.Value.Type != FShaderAssociationInfo::FShaderChunkInfo::Package)
		{
			continue;
		}

		TSet<FString> ShaderPlugins;
		for (FName PackageName : ShaderChunkInfo.Value.ReferencedByPackages)
		{
			FString Plugin = FPackageName::SplitPackageNameRoot(PackageName, nullptr);
			ShaderPlugins.Add(MoveTemp(Plugin));
		}

		if (ShaderPlugins.Num() == 1)
		{
			// We can assign the size to this plugin when the time comes - no pseudo plugin needed.
			continue;
		}

		bool bAllReferencingPluginsAreRooted = true;
		TSet<FString> ShaderRootPlugins;
		for (FString& ReferencingPluginName : ShaderPlugins)
		{
			bool bReferencingPluginIsRooted = false;
			FPluginGraphEntry** ReferencingPlugin = InPluginGraph.NameToPlugin.Find(ReferencingPluginName);
			if (ReferencingPlugin)
			{
				for (FPluginGraphEntry* RootForReferencingPlugin : (*ReferencingPlugin)->Roots)
				{
					ShaderRootPlugins.Add(RootForReferencingPlugin->Name);
					bReferencingPluginIsRooted = true;
				}
			}
				
			if (bReferencingPluginIsRooted == false)
				bAllReferencingPluginsAreRooted = false;
		}

		if (ShaderRootPlugins.Num() == 0 || bAllReferencingPluginsAreRooted == false)
		{
			// Place in the unrooted list.
			ShaderRootPlugins.Add(TEXT("Unrooted"));
		}

		TStringBuilder<256> PseudoPluginName;
		PseudoPluginName.Append(TEXT("ShaderPlugin"));

		TStringBuilder<256> NameConcatenation;

		// We can't just concat the names because it'll get too long when we use the name as a filename, which is unfortunate. That being said, we
		// only expect this to happen in degenerate cases - so if it's not too long we list the names for convenience
		// otherwise we hash it.
		for (FString& ReferencingPlugin : ShaderRootPlugins)
		{
			NameConcatenation.Append(TEXT("_"));
			NameConcatenation.Append(ReferencingPlugin);
		}

		if (NameConcatenation.Len() > 100) // arbitrary length here just to try and avoid hitting MAX_PATH (260)
		{
			FXxHash64 NameHash = FXxHash64::HashBuffer(NameConcatenation.GetData(), NameConcatenation.Len());
			uint8 HashBytes[8];
			NameHash.ToByteArray(HashBytes);

			PseudoPluginName.Append(TEXT("_"));
			UE::String::BytesToHexLower(MakeArrayView(HashBytes), PseudoPluginName);
		}
		else
		{
			PseudoPluginName.Append(NameConcatenation);
		}

		FString PseudoPluginNameStr = PseudoPluginName.ToString();
		for (FString& ReferencingPlugin : ShaderRootPlugins)
		{
			PluginDependenciesOnShaders.FindOrAdd(ReferencingPlugin).Add(PseudoPluginNameStr);
		}

		ShaderPseudoPlugins.FindOrAdd(PseudoPluginNameStr).Add(ShaderChunkInfo.Key);

		ShaderChunkInfo.Value.SharedPluginName = MoveTemp(PseudoPluginNameStr);
	}

	// We have the list of pseudo plugins we need to make... we need to copy and append to the list
	// of plugins in the cook metadata, after stripping out any previous run's pseudo plugins.
	TArray<UE::Cook::FCookMetadataPluginEntry> PluginEntries = PluginHierarchy.PluginsEnabledAtCook;
	for (int32 PluginIndex = 0; PluginIndex < PluginEntries.Num(); PluginIndex++)
	{
		if (PluginEntries[PluginIndex].Type == UE::Cook::ECookMetadataPluginType::Unassigned)
		{
			UE_LOGF(LogIoStore, Warning, "Found unassigned plugin type in cook metadata! %ls", *PluginEntries[PluginIndex].Name);
		}
		if (PluginEntries[PluginIndex].Type == UE::Cook::ECookMetadataPluginType::ShaderPseudo)
		{
			// We can do a swap because we insert these at the end in a group so there shouldn't
			// by anything else after us.
			PluginEntries.RemoveAtSwap(PluginIndex);
			PluginIndex--;
		}
	}

	for (const TPair< FString, TArray<FShaderAssociationInfo::FShaderChunkInfoKey>>& ShaderPP : ShaderPseudoPlugins)
	{
		UE::Cook::FCookMetadataPluginEntry& ShaderPluginEntry = PluginEntries.AddDefaulted_GetRef();
		ShaderPluginEntry.Name = ShaderPP.Key;
		ShaderPluginEntry.Type = UE::Cook::ECookMetadataPluginType::ShaderPseudo;
	}

	// We have to redo the dependency tree as well to add the shaders as dependencies.
	TMap<FString, int32> PluginNameToIndex;
	int32 CurrentIndex = 0;
	for (UE::Cook::FCookMetadataPluginEntry& Entry : PluginEntries)
	{
		PluginNameToIndex.Add(Entry.Name, CurrentIndex);
		CurrentIndex++;
	}

	if (IntFitsIn<uint16>(PluginEntries.Num()) == false)
	{
		UE_LOGF(LogIoStore, Warning, "Post shared shader plugin count is > 65535 (%d)  - not updating cook metadata!", PluginEntries.Num());
		return;
	}

	TArray<uint16> DependencyList;
	for (UE::Cook::FCookMetadataPluginEntry& Entry : PluginEntries)
	{
		// Add the normal dependencies. Since we didn't reorder anything we can
		// use the old dependency list
		uint32 StartIndex = (uint32)DependencyList.Num();
		for (uint32 DependencyIndex = Entry.DependencyIndexStart; DependencyIndex < Entry.DependencyIndexEnd; DependencyIndex++)
		{
			const UE::Cook::FCookMetadataPluginEntry& DependentPlugin = PluginHierarchy.PluginsEnabledAtCook[PluginHierarchy.PluginDependencies[DependencyIndex]];
			if (DependentPlugin.Type == UE::Cook::ECookMetadataPluginType::ShaderPseudo)
			{
				// If we are rerunning stage then we might already have added shader plugins
				// as dependencies, which we've removed so they won't exist in the lookup
				// (and we want to redo them anyway).
				continue;
			}
			int32* PluginIndex = PluginNameToIndex.Find(DependentPlugin.Name);
			if (PluginIndex)
			{
				DependencyList.Add(*PluginIndex);
			}
			else
			{
				UE_LOGF(LogIoStore, Warning, "Couldn't find plugin %ls when re-adding dependencies.", *DependentPlugin.Name);
			}
		}

		// However we also need to check for shaders
		TArray<FString>* DependenciesOnShader = PluginDependenciesOnShaders.Find(Entry.Name);
		if (DependenciesOnShader)
		{
			for (FString& ShaderPluginName : (*DependenciesOnShader))
			{
				int32* PluginIndex = PluginNameToIndex.Find(ShaderPluginName);
				if (PluginIndex)
				{
					DependencyList.Add(*PluginIndex);
				}
				else
				{
					UE_LOGF(LogIoStore, Warning, "Couldn't find shader pseudo plugin %ls when adding dependencies.", *ShaderPluginName);
				}
			}
		}

		Entry.DependencyIndexStart = StartIndex;
		Entry.DependencyIndexEnd = (uint32)DependencyList.Num();
	}

	// Now blast the old one away and replace.
	UE::Cook::FCookMetadataPluginHierarchy& MutablePluginHierarchy = InCookMetadata.GetMutablePluginHierarchy();
	MutablePluginHierarchy.PluginDependencies = MoveTemp(DependencyList);
	MutablePluginHierarchy.PluginsEnabledAtCook = PluginEntries;

	// Sanity check we assigned plugin types
	for (UE::Cook::FCookMetadataPluginEntry& Entry : MutablePluginHierarchy.PluginsEnabledAtCook)
	{
		if (Entry.Type == UE::Cook::ECookMetadataPluginType::Unassigned)
		{
			UE_LOGF(LogIoStore, Warning, "We caused an unassigned plugin type in shader pseudo plugin generation! %ls", *Entry.Name);
		}
	}
}

/**
*	Use the name of the package to assign sizes so that we can track build size at a per-plugin level,
*	and write out jsons files for each plugin in to the cooked metadata directory.
* 
*	Plugins insert themselves in to the package's path at the top level. Content that is unassigned
*	to a plugin has either /Engine or /Game as it's top level path and will be assigned to a pseudo
*	plugin.
*/
static void UpdatePluginMetadataAndWriteJsons(
	const FString& InAssetRegistryFileName, 
	TMap<FPackageId, TArray<FIoStoreChunkSource, TInlineAllocator<2>>>& PackageToChunks, 
	FAssetRegistryState& AssetRegistry, 
	UE::Cook::FCookMetadataState& CookMetadata, 
	FShaderAssociationInfo* InShaderAssociationInfo)
{
	double WritePluginStart = FPlatformTime::Seconds();

	FPluginGraph PluginGraph;
	GeneratePluginGraph(CookMetadata.GetPluginHierarchy(), PluginGraph);

	if (InShaderAssociationInfo)
	{
		InsertShadersInPluginHierarchy(CookMetadata, PluginGraph, *InShaderAssociationInfo);

		// Generate the graph aggain after we've inserted the new "plugins".
		PluginGraph = FPluginGraph();
		GeneratePluginGraph(CookMetadata.GetPluginHierarchy(), PluginGraph);
	}

	double GeneratePluginGraphEnd = FPlatformTime::Seconds();

	TSet<FString> LoggedPluginNames;

	FTopLevelAssetPath Texture2DPath(TEXT("/Script/Engine.Texture2D"));
	FTopLevelAssetPath Texture2DArrayPath(TEXT("/Script/Engine.Texture2DArray"));
	FTopLevelAssetPath Texture3DPath(TEXT("/Script/Engine.Texture3D"));
	FTopLevelAssetPath TextureCubePath(TEXT("/Script/Engine.TextureCube"));
	FTopLevelAssetPath TextureCubeArrayPath(TEXT("/Script/Engine.TextureCubeArray"));
	FTopLevelAssetPath VirtualTextureBuilderPath(TEXT("/Script/Engine.VirtualTextureBuilder"));
	FTopLevelAssetPath StaticMeshPath(TEXT("/Script/Engine.StaticMesh"));
	FTopLevelAssetPath SoundWavePath(TEXT("/Script/Engine.SoundWave"));
	FTopLevelAssetPath SkeletalMeshPath(TEXT("/Script/Engine.SkeletalMesh"));
	FTopLevelAssetPath LevelPath(TEXT("/Script/Engine.World"));
	FTopLevelAssetPath BlueprintPath(TEXT("/Script/Engine.BlueprintGeneratedClass"));
	FTopLevelAssetPath AnimationSequencePath(TEXT("/Script/Engine.AnimSequence"));
	FTopLevelAssetPath GeometryCollectionPath(TEXT("/Script/GeometryCollectionEngine.GeometryCollection"));
	FTopLevelAssetPath NiagaraSystemPath(TEXT("/Script/Niagara.NiagaraSystem"));
	FTopLevelAssetPath MaterialInstancePath(TEXT("/Script/Engine.MaterialInstanceConstant"));
	
	if (InShaderAssociationInfo)
	{
		//
		// Create a bunch of "Assets" that we can iterate over in the same manner as a normal
		// asset and find its containing "plugin" for the purposes of assigning its size.
		//
		TArray<UE::Cook::FCookMetadataShaderPseudoAsset> ShaderPseudoAssets;

		TMap<FName, TArray<int32>> PackageDependencyMap;

		// All package shaders should now either be able to be assigned to:
		// 1. a single package (i.e. plugin)
		// 2. shared between packages in 1 plugin (i.e. a plugin)
		// 3. shared between packages across plugins (i.e. assignable to a pseudo plugin we created earlier).
		//
		uint64 CrossPluginShaderSize = 0;
		uint64 SinglePluginShaderSize = 0;
		uint64 InlineShaderSize = 0;
		uint64 GlobalShaderSize = 0;
		uint64 OrphanShaderSize = 0;
		for (TPair<FShaderAssociationInfo::FShaderChunkInfoKey, FShaderAssociationInfo::FShaderChunkInfo>& ShaderChunkInfo : InShaderAssociationInfo->ShaderChunkInfos)
		{
			if (ShaderChunkInfo.Value.Type == FShaderAssociationInfo::FShaderChunkInfo::Orphan)
			{
				OrphanShaderSize += ShaderChunkInfo.Value.CompressedSize;
				continue;
			}
			if (ShaderChunkInfo.Value.Type == FShaderAssociationInfo::FShaderChunkInfo::Global)
			{
				GlobalShaderSize += ShaderChunkInfo.Value.CompressedSize;
				continue;
			}

			check(ShaderChunkInfo.Value.Type == FShaderAssociationInfo::FShaderChunkInfo::Package);

			TStringBuilder<128> PackageName;
			if (ShaderChunkInfo.Value.SharedPluginName.Len())
			{
				// We know we belong to this plugin.
				PackageName.Append(ShaderChunkInfo.Value.SharedPluginName);
				CrossPluginShaderSize += ShaderChunkInfo.Value.CompressedSize;
			}
			else
			{
				// We know all the plugin prefixes are the same for all referrers
				if (ShaderChunkInfo.Value.ReferencedByPackages.Num() == 1)
				{
					InlineShaderSize += ShaderChunkInfo.Value.CompressedSize;
				}
				else
				{
					SinglePluginShaderSize += ShaderChunkInfo.Value.CompressedSize;
				}

				FString Plugin = FPackageName::SplitPackageNameRoot(ShaderChunkInfo.Value.ReferencedByPackages[0], nullptr);
				PackageName.Append(MoveTemp(Plugin));
			}

			FPluginGraphEntry** PluginEntryPtr = PluginGraph.NameToPlugin.Find(PackageName.ToString());
			if (PluginEntryPtr)
			{
				PluginEntryPtr[0]->ExclusiveSizes[(uint8)EPluginGraphSizeClass::All][ShaderChunkInfo.Value.SizeType] += ShaderChunkInfo.Value.CompressedSize;
				PluginEntryPtr[0]->ExclusiveSizes[(uint8)EPluginGraphSizeClass::Shader][ShaderChunkInfo.Value.SizeType] += ShaderChunkInfo.Value.CompressedSize;
			}
			else
			{
				FString AllocatedPluginName(PackageName.ToString());
				bool bAlreadyLogged = false;
				LoggedPluginNames.Add(AllocatedPluginName, &bAlreadyLogged);
				if (bAlreadyLogged == false)
				{
					UE_LOGF(LogIoStore, Warning, "Plugin for shader not found: %ls", *AllocatedPluginName);
				}
			}


			// What to name our package? Needs to be unique. We just concat everything so a human
			// can trace where it came from.
			PackageName.Append(TEXT("/ShaderPseudoAsset_"));

			PackageName.Append(ShaderChunkInfo.Key.PakChunkName.ToString());
			PackageName.Append(TEXT("_"));

			// shader hash (iochunkid)
			UE::String::BytesToHexLower(MakeArrayView(ShaderChunkInfo.Key.IoChunkId.GetData(), sizeof(FIoChunkId)), PackageName);

			ShaderPseudoAssets.Add({PackageName.ToString(), ShaderChunkInfo.Value.CompressedSize});

			// Track who depends on this shader by index.
			for (FName ReferencingPackage : ShaderChunkInfo.Value.ReferencedByPackages)
			{
				TArray<int32>& PackageDependencies = PackageDependencyMap.FindOrAdd(ReferencingPackage);
				PackageDependencies.Add(ShaderPseudoAssets.Num() - 1);
			}
		}

		TMap<FName, TPair<int32, int32>> FinalizedDependencyMap;

		// Now convert the package dependency map in to array ranges.
		TArray<int32> DependencyByIndex;
		for (TPair<FName, TArray<int32>>& Dependencies : PackageDependencyMap)
		{
			TPair<int32, int32>& Entry = FinalizedDependencyMap.Add(Dependencies.Key);
			Entry.Key = DependencyByIndex.Num();
			DependencyByIndex.Append(Dependencies.Value);
			Entry.Value = DependencyByIndex.Num();
		}

		// Move the new info over to the cook metadata.
		UE::Cook::FCookMetadataShaderPseudoHierarchy PSH;
		PSH.ShaderAssets = MoveTemp(ShaderPseudoAssets);
		PSH.PackageShaderDependencyMap = MoveTemp(FinalizedDependencyMap);
		PSH.DependencyList = MoveTemp(DependencyByIndex);
		CookMetadata.SetShaderPseudoHieararchy(MoveTemp(PSH));

		double TotalShaderSize = (double)(InlineShaderSize + CrossPluginShaderSize + SinglePluginShaderSize + GlobalShaderSize + OrphanShaderSize);
		UE_LOGF(LogIoStore, Display, "Shader total sizes: %ls single package (%.0f%%), %ls single root GFP (%.0f%%), %ls cross root GFP (%.0f%%), %ls global (%.0f%%), %ls orphan (%.0f%%) - %ls assigned (%.0f%%)",
			*NumberString(InlineShaderSize), 100.0 * InlineShaderSize / TotalShaderSize,
			*NumberString(SinglePluginShaderSize), 100.0 * SinglePluginShaderSize / TotalShaderSize,
			*NumberString(CrossPluginShaderSize), 100.0 * CrossPluginShaderSize / TotalShaderSize,
			*NumberString(GlobalShaderSize), 100.0 * GlobalShaderSize / TotalShaderSize,
			*NumberString(OrphanShaderSize), 100.0 * OrphanShaderSize / TotalShaderSize,
			*NumberString(InlineShaderSize + SinglePluginShaderSize + CrossPluginShaderSize), 100.0 * (InlineShaderSize + SinglePluginShaderSize + CrossPluginShaderSize) / TotalShaderSize
		);
	}

	double AssetPackageMapStart = FPlatformTime::Seconds();
	const TMap<FName, const FAssetPackageData*> AssetPackageMap = AssetRegistry.GetAssetPackageDataMap();
	for (const TPair<FName, const FAssetPackageData*>& AssetPackage : AssetPackageMap)
	{
		if (AssetPackage.Value->DiskSize < 0)
		{
			// No data on disk!
			continue;
		}

		// Grab the most important asset out and use it to track largest asset classes for the plugin.
		// This might be null!
		const FAssetData* AssetData = UE::AssetRegistry::GetMostImportantAsset(
			AssetRegistry.CopyAssetsByPackageName(AssetPackage.Key),
			UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
		
		const TArray<FIoStoreChunkSource, TInlineAllocator<2>>* PackageChunks = PackageToChunks.Find(FPackageId::FromName(AssetPackage.Key));
		if (PackageChunks == nullptr)
		{
			// This happens when the package has been stripped by UAT prior to staging by e.g. PakDenyList.
			continue;
		}

		UE::Cook::FPluginSizeInfo PackageSizes;
		for (const FIoStoreChunkSource& ChunkInfo : *PackageChunks)
		{
			PackageSizes[ChunkInfo.SizeType] += ChunkInfo.ChunkInfo.CompressedSize;
		}

		// Assign the size to the package's plugin.
		{
			FString PluginName = FPackageName::SplitPackageNameRoot(AssetPackage.Key, nullptr);
			FPluginGraphEntry** PluginEntryPtr = PluginGraph.NameToPlugin.Find(PluginName);
			if (PluginEntryPtr)
			{
				FPluginGraphEntry* PluginEntry = *PluginEntryPtr;
				PluginEntry->ExclusiveSizes[(uint8)EPluginGraphSizeClass::All].Add(PackageSizes);

				// If we have asset class info and it's a top contender, track it also.
				if (AssetData != nullptr)
				{
					EPluginGraphSizeClass AssetSizeClass = EPluginGraphSizeClass::Other;
					if (AssetData->AssetClassPath == Texture2DPath ||
						AssetData->AssetClassPath == Texture3DPath ||
						AssetData->AssetClassPath == TextureCubePath ||
						AssetData->AssetClassPath == TextureCubeArrayPath ||
						AssetData->AssetClassPath == Texture2DArrayPath || 
						AssetData->AssetClassPath == VirtualTextureBuilderPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Texture;
					}
					else if (AssetData->AssetClassPath == StaticMeshPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::StaticMesh;
					}
					else if (AssetData->AssetClassPath == SoundWavePath)
					{
						AssetSizeClass = EPluginGraphSizeClass::SoundWave;
					}
					else if (AssetData->AssetClassPath == SkeletalMeshPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::SkeletalMesh;
					}
					else if (AssetData->AssetClassPath == AnimationSequencePath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Animation;
					}
					else if (AssetData->AssetClassPath == NiagaraSystemPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Niagara;
					}
					else if (AssetData->AssetClassPath == MaterialInstancePath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Material;
					}
					else if (AssetData->AssetClassPath == LevelPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Level;
					}
					else if (AssetData->AssetClassPath == BlueprintPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Blueprint;
					}
					else if (AssetData->AssetClassPath == GeometryCollectionPath)
					{
						AssetSizeClass = EPluginGraphSizeClass::Geometry;
					}
					
					// Note that we can't get shaders here so we don't need to handle ::Shader.

					PluginEntry->ExclusiveSizes[(uint8)AssetSizeClass].Add(PackageSizes);
					PluginEntry->ExclusiveCounts[(uint8)AssetSizeClass]++;
				}
			}
			else
			{
				bool bAlreadyLogged = false;
				LoggedPluginNames.Add(MoveTemp(PluginName), &bAlreadyLogged);
				if (bAlreadyLogged == false)
				{
					UE_LOGF(LogIoStore, Display, "Plugin for package not found: %ls", *AssetPackage.Key.ToString());
				}
			}
		}
	}

	// Inclusive is the sum of us plus all our dependencies.
	for (FPluginGraphEntry* PluginEntry : PluginGraph.TopologicallySortedPlugins)
	{
		for (uint8 ClassIndex = 0; ClassIndex < FPluginGraphEntry::ClassCount; ClassIndex++)
		{
			PluginEntry->InclusiveSizes[ClassIndex] = PluginEntry->ExclusiveSizes[ClassIndex];
			PluginEntry->InclusiveCounts[ClassIndex] = PluginEntry->ExclusiveCounts[ClassIndex];
		}

		for (FPluginGraphEntry* Dependency : PluginEntry->TotalDependencies)
		{
			for (uint8 ClassIndex = 0; ClassIndex < FPluginGraphEntry::ClassCount; ClassIndex++)
			{
				PluginEntry->InclusiveSizes[ClassIndex].Add(Dependency->ExclusiveSizes[ClassIndex]);
				PluginEntry->InclusiveCounts[ClassIndex] += Dependency->ExclusiveCounts[ClassIndex];
			}			
		}
	}

	// Now we need to find the unique size for each root plugin. This is the size of dependencies that only
	// belong to the root plugin and not to another. Conceptually this is the "assuming all other roots are
	// installed, this is the size cost to add this plugin to the install".
	for (FPluginGraphEntry* RootPlugin : PluginGraph.RootPlugins)
	{
		for (FPluginGraphEntry* UniqueDependency : RootPlugin->UniqueDependencies)
		{
			for (uint8 ClassIndex = 0; ClassIndex < FPluginGraphEntry::ClassCount; ClassIndex++)
			{
				RootPlugin->UniqueSizes[ClassIndex].Add(UniqueDependency->ExclusiveSizes[ClassIndex]);
				RootPlugin->UniqueCounts[ClassIndex] += UniqueDependency->ExclusiveCounts[ClassIndex];
			}
		}
	}
	
	// Find the total size of all plugins that aren't rooted in the root set.
	UE::Cook::FPluginSizeInfo UnrootedTotal;
	for (FPluginGraphEntry* Plugin : PluginGraph.UnrootedPlugins)
	{		
		UnrootedTotal.Add(Plugin->ExclusiveSizes[(uint8)EPluginGraphSizeClass::All]);
	}

	double WriteBegin = FPlatformTime::Seconds();

	UE::Cook::FCookMetadataPluginHierarchy& MutablePluginHierarchy = CookMetadata.GetMutablePluginHierarchy();

	auto GeneratePluginJson = [&MutablePluginHierarchy](TUtf8StringBuilder<4096>& OutPluginMetadataJson, FStringView InName, const FPluginGraphEntry& InGraphEntry, EPluginGraphSizeClass InSizeClass)
	{
		OutPluginMetadataJson.Reset();

		uint8 SizeClass = (uint8)InSizeClass;

		if (InGraphEntry.InclusiveSizes[SizeClass].TotalSize()== 0)
		{
			// Asset type or its dependencies do not contribute to the record.
			return;
		}
		
		OutPluginMetadataJson << "{\n";
		OutPluginMetadataJson << "\t\"name\":\"" << InName << "\",\n";

		OutPluginMetadataJson << "\t\"schema_version\":4,\n";

		OutPluginMetadataJson << "\t\"is_root_plugin\":" << (InGraphEntry.bIsRoot ? TEXTVIEW("true") : TEXTVIEW("false")) << ",\n";
		OutPluginMetadataJson << "\t\"asset_sizes_class\":\"" << PluginGraphEntryClassNames[SizeClass] << "\",\n";
		OutPluginMetadataJson << "\t\"exclusive_asset_class_count\":" << InGraphEntry.ExclusiveCounts[SizeClass] << ",\n";

		OutPluginMetadataJson << "\t\"exclusive_installed\":" << InGraphEntry.ExclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::Installed] << ",\n";
		OutPluginMetadataJson << "\t\"exclusive_optional\":" << InGraphEntry.ExclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::Optional] << ",\n";
		OutPluginMetadataJson << "\t\"exclusive_ias\":" << InGraphEntry.ExclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::Streaming] << ",\n";
		OutPluginMetadataJson << "\t\"exclusive_optionalsegment\":" << InGraphEntry.ExclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::OptionalSegment] << ",\n";
		
		OutPluginMetadataJson << "\t\"inclusive_installed\":" << InGraphEntry.InclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::Installed] << ",\n";
		OutPluginMetadataJson << "\t\"inclusive_optional\":" << InGraphEntry.InclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::Optional] << ",\n";
		OutPluginMetadataJson << "\t\"inclusive_ias\":" << InGraphEntry.InclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::Streaming] << ",\n";
		OutPluginMetadataJson << "\t\"inclusive_optionalsegment\":" << InGraphEntry.InclusiveSizes[SizeClass][UE::Cook::EPluginSizeTypes::OptionalSegment] << ",\n";

		// this only has values for is_root_plugin == true.
		OutPluginMetadataJson << "\t\"unique_installed\":" << InGraphEntry.UniqueSizes[SizeClass][UE::Cook::EPluginSizeTypes::Installed] << ",\n";
		OutPluginMetadataJson << "\t\"unique_optional\":" << InGraphEntry.UniqueSizes[SizeClass][UE::Cook::EPluginSizeTypes::Optional] << ",\n";
		OutPluginMetadataJson << "\t\"unique_ias\":" << InGraphEntry.UniqueSizes[SizeClass][UE::Cook::EPluginSizeTypes::Streaming] << ",\n";
		OutPluginMetadataJson << "\t\"unique_optionalsegment\":" << InGraphEntry.UniqueSizes[SizeClass][UE::Cook::EPluginSizeTypes::OptionalSegment] << ",\n";

		OutPluginMetadataJson << "\t\"direct_refcount\":" << InGraphEntry.DirectRefcount << ",\n";

		// pass through any custom fields that were added to the cook metadata.
		if (InGraphEntry.IndexInEnabledPlugins != TNumericLimits<uint16>::Max())
		{
			const UE::Cook::FCookMetadataPluginEntry& CookMetadataEntry = MutablePluginHierarchy.PluginsEnabledAtCook[InGraphEntry.IndexInEnabledPlugins];
			for (const TPair<uint8, UE::Cook::FCookMetadataPluginEntry::CustomFieldVariantType>& CustomValue : CookMetadataEntry.CustomFields)
			{
				const FString& FieldName = MutablePluginHierarchy.CustomFieldEntries[CustomValue.Key].Name;
				UE::Cook::ECookMetadataCustomFieldType FieldType = MutablePluginHierarchy.CustomFieldEntries[CustomValue.Key].Type;

				OutPluginMetadataJson << "\t\"" << FieldName;
				
				if (CustomValue.Value.IsType<bool>())
				{
					check(FieldType == UE::Cook::ECookMetadataCustomFieldType::Bool);
					OutPluginMetadataJson << (CustomValue.Value.Get<bool>() ? "\":true,\n" : "\":false,\n");
				}
				else
				{
					check(FieldType == UE::Cook::ECookMetadataCustomFieldType::String);
					OutPluginMetadataJson << "\":\"" << CustomValue.Value.Get<FString>() << "\",\n";
				}

			}
		}

		{
			OutPluginMetadataJson << "\t\"roots\":[";

			int32 RootIndex = 0;
			for (FPluginGraphEntry* Root : InGraphEntry.Roots)
			{
				
				OutPluginMetadataJson << "\"" << Root->Name << "\"";
				if (RootIndex + 1 < InGraphEntry.Roots.Num())
				{
					OutPluginMetadataJson << ",";
				}
				RootIndex++;
			}

			OutPluginMetadataJson << "]\n";
		}



		OutPluginMetadataJson << "}\n";
	};

	//
	// Generate the plugin_summary jsons.	
	//

	// Also write a csv for easier browsing in spreadsheets.
	TUtf8StringBuilder<4096> Csv;
	Csv.Append("name,asset_sizes_class,exclusive_installed,exclusive_optional,exclusive_ias,inclusive_installed,inclusive_optional,inclusive_ias,unique_installed,unique_optional,unique_ias,direct_refcount,total_dependency_count\n");


	// This is so re-staging the same cook is consistent.	
	for (UE::Cook::FCookMetadataPluginEntry& Plugin : MutablePluginHierarchy.PluginsEnabledAtCook)
	{
		Plugin.InclusiveSizes.Zero();
		Plugin.ExclusiveSizes.Zero();
	}
	
	// Instead of writing out a ton of small event files we concat them all. There are a lot in a mature project.
	// This will be megabytes.
	TArray<UTF8CHAR> PluginMetadataFullJson;
	uint32 PluginsAddedToFullJson = 0;
	PluginMetadataFullJson.Append(UTF8TEXTVIEW("{ \"PluginSizeInfos\": ["));

	auto AddPluginJsonToFull = [&PluginMetadataFullJson, &PluginsAddedToFullJson](TUtf8StringBuilder<4096>& InJsonToAdd)
	{
		if (PluginsAddedToFullJson)
		{
			PluginMetadataFullJson.Add(UTF8TEXT(','));
		}
		PluginsAddedToFullJson++;
		PluginMetadataFullJson.Append(InJsonToAdd.GetData(), InJsonToAdd.Len());
	};

	TUtf8StringBuilder<4096> PluginMetadataJson;

	for (UE::Cook::FCookMetadataPluginEntry& Plugin : MutablePluginHierarchy.PluginsEnabledAtCook)
	{
		const FPluginGraphEntry& PluginEntry = *PluginGraph.NameToPlugin[Plugin.Name];
		if (PluginEntry.InclusiveSizes[(uint8)EPluginGraphSizeClass::All].TotalSize() == 0)
		{
			continue;
		}

		Plugin.InclusiveSizes = PluginEntry.InclusiveSizes[(uint8)EPluginGraphSizeClass::All];
		Plugin.ExclusiveSizes = PluginEntry.ExclusiveSizes[(uint8)EPluginGraphSizeClass::All];

		for (uint8 ClassIndex = 0; ClassIndex < FPluginGraphEntry::ClassCount; ClassIndex++)
		{

			GeneratePluginJson(PluginMetadataJson, Plugin.Name, PluginEntry, (EPluginGraphSizeClass)ClassIndex);

			if (PluginMetadataJson.Len()>0)
			{
				AddPluginJsonToFull(PluginMetadataJson);
				Csv.Appendf("%ls,%s,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%llu,%u,%u\n", *Plugin.Name, PluginGraphEntryClassNames[ClassIndex],
					PluginEntry.ExclusiveSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Installed],
					PluginEntry.ExclusiveSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Optional],
					PluginEntry.ExclusiveSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Streaming],
					PluginEntry.InclusiveSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Installed],
					PluginEntry.InclusiveSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Optional],
					PluginEntry.InclusiveSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Streaming],
					PluginEntry.UniqueSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Installed],
					PluginEntry.UniqueSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Optional],
					PluginEntry.UniqueSizes[ClassIndex][UE::Cook::EPluginSizeTypes::Streaming],
					PluginEntry.DirectRefcount, PluginEntry.TotalDependencies.Num());
			}
		}
	}

	// Also write a json that contains the sizes for the plugins that don't belong to any root plugin,
	// so that size information never gets lost.
	{
		FPluginGraphEntry OrphanedEntry;
		OrphanedEntry.bIsRoot = true;
		OrphanedEntry.DirectRefcount = 0;
		OrphanedEntry.InclusiveSizes[(uint8)EPluginGraphSizeClass::All] = UnrootedTotal;
		GeneratePluginJson(PluginMetadataJson, TEXT("OrphanedPlugins"), OrphanedEntry, EPluginGraphSizeClass::All);
		if (PluginMetadataJson.Len() > 0)
		{
			AddPluginJsonToFull(PluginMetadataJson);
		}

		Csv.Appendf("OrphanedPlugins,all,0,0,0,%llu,%llu,%llu,0,%u\n",
			UnrootedTotal[UE::Cook::EPluginSizeTypes::Installed], UnrootedTotal[UE::Cook::EPluginSizeTypes::Optional], UnrootedTotal[UE::Cook::EPluginSizeTypes::Streaming],
			PluginGraph.UnrootedPlugins.Num());
	}

	PluginMetadataFullJson.Append(UTF8TEXTVIEW("]}"));
	{
		FString JsonFilename = FPaths::GetPath(InAssetRegistryFileName) / TEXT("plugin_size_jsons.json");
		if (WriteUtf8StringView(MakeStringView(PluginMetadataFullJson), JsonFilename) == false)
		{
			UE_LOGF(LogIoStore, Error, "Unable to write plugin json file: %ls", *JsonFilename);
			return;
		}
	}

	{
		FString CsvFilename = FPaths::GetPath(InAssetRegistryFileName) / TEXT("plugin_sizes.csv");
		if (WriteUtf8StringView(Csv.ToView(), CsvFilename) == false)
		{
			UE_LOGF(LogIoStore, Error, "Unable to write plugin csv file: %ls", *CsvFilename);
			return;
		}
	}

	double WritePluginEnd = FPlatformTime::Seconds();
	UE_LOGF(LogIoStore, Display, "Wrote plugin size jsons/csv in %.2f seconds [graph %.2f shaders %.2f sizes %.2f writes %.2f]", 
		WritePluginEnd - WritePluginStart, 
		GeneratePluginGraphEnd - WritePluginStart, 
		AssetPackageMapStart - GeneratePluginGraphEnd,
		WriteBegin - AssetPackageMapStart,
		WritePluginEnd - WriteBegin);
}


static void AddChunkInfoToAssetRegistry(TMap<FPackageId, TArray<FIoStoreChunkSource, TInlineAllocator<2>>>&& PackageToChunks, FAssetRegistryState& AssetRegistry, const FShaderAssociationInfo* ShaderSizeInfo, uint64 InUnassignableShaderCodeBytes, uint64 InAssignableShaderCodeBytes, uint64 TotalCompressedSize)
{
	//
	// The asset registry has the chunks associate with each package, so we can just iterate the
	// packages, look up the chunk info, and then save the tags.
	//
	// The complicated thing is (as usual), trying to determine which asset gets the blame for the
	// data. We use the GetMostImportantAsset function for this.
	//
	const TMap<FName, const FAssetPackageData*> AssetPackageMap = AssetRegistry.GetAssetPackageDataMap();

	uint64 AssetsCompressedSize = 0;
	uint64 UpdatedAssetCount = 0;

	for (const TPair<FName, const FAssetPackageData*>& AssetPackage : AssetPackageMap)
	{
		if (AssetPackage.Value->DiskSize < 0)
		{
			// No data on disk!
			continue;
		}

		FPackageId PackageId = FPackageId::FromName(AssetPackage.Key);

		const FAssetData* AssetData = UE::AssetRegistry::GetMostImportantAsset(
			AssetRegistry.CopyAssetsByPackageName(AssetPackage.Key),
			UE::AssetRegistry::EGetMostImportantAssetFlags::IgnoreSkipClasses);
		if (AssetData == nullptr)
		{
			// e.g. /Script packages.
			continue;
		}

		const TArray<FIoStoreChunkSource, TInlineAllocator<2>>* PackageChunks = PackageToChunks.Find(PackageId);
		if (PackageChunks == nullptr)
		{
			// This happens when the package has been stripped by UAT prior to staging by e.g. PakDenyList.
			continue;
		}

		UE::Cook::FPluginSizeInfo PackageCompressedSize;
		UE::Cook::FPluginSizeInfo PackageSize;
		int32 ChunkCount = 0;
		for (const FIoStoreChunkSource& ChunkInfo : *PackageChunks)
		{
			ChunkCount++;
			PackageSize[ChunkInfo.SizeType] += ChunkInfo.ChunkInfo.Size;
			PackageCompressedSize[ChunkInfo.SizeType] += ChunkInfo.ChunkInfo.CompressedSize;
		}

		FAssetDataTagMap TagsAndValues;
		TagsAndValues.Add(UE::AssetRegistry::Stage_ChunkCountFName, LexToString(ChunkCount));
		TagsAndValues.Add(UE::AssetRegistry::Stage_ChunkSizeFName, LexToString(PackageSize.TotalSize()));
		TagsAndValues.Add(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, LexToString(PackageCompressedSize.TotalSize()));
		TagsAndValues.Add(UE::AssetRegistry::Stage_ChunkInstalledSizeFName, LexToString(PackageCompressedSize[UE::Cook::EPluginSizeTypes::Installed]));
		TagsAndValues.Add(UE::AssetRegistry::Stage_ChunkStreamingSizeFName, LexToString(PackageCompressedSize[UE::Cook::EPluginSizeTypes::Streaming]));
		TagsAndValues.Add(UE::AssetRegistry::Stage_ChunkOptionalSizeFName, LexToString(PackageCompressedSize[UE::Cook::EPluginSizeTypes::Optional]));
		AssetRegistry.AddTagsToAssetData(AssetData->GetSoftObjectPath(), MoveTemp(TagsAndValues));

		// We assign a package's chunks to a single asset, remove it from the list so that
		// at the end we can track how many chunks don't get assigned.
		PackageToChunks.Remove(PackageId);

		UpdatedAssetCount++;
		AssetsCompressedSize += PackageCompressedSize.TotalSize();
	}
	
	// PackageToChunks now has chunks that we never assigned to an asset, and so aren't accounted for.
	uint64 RemainingByType[(uint8)EIoChunkType::MAX] = {};
	for (auto PackageChunks : PackageToChunks)
	{
		for (FIoStoreChunkSource& Info : PackageChunks.Value)
		{
			RemainingByType[(uint8)Info.ChunkInfo.ChunkType] += Info.ChunkInfo.CompressedSize;
		}
	}
	
	// Shaders aren't in the PackageToChunks map, but we want to numbers reported to include them.
	RemainingByType[(uint8)EIoChunkType::ShaderCode] += InUnassignableShaderCodeBytes;
	AssetsCompressedSize += InAssignableShaderCodeBytes;

	double PercentAssets = 1.0f;
	if (TotalCompressedSize != 0)
	{
		PercentAssets = AssetsCompressedSize / (double)TotalCompressedSize;
	}

	UE_LOGF(LogIoStore, Display, "Added chunk metadata to %ls assets.", *FText::AsNumber(UpdatedAssetCount).ToString());
	UE_LOGF(LogIoStore, Display, "Assets represent %ls bytes of %ls chunk bytes (%.1f%%)", *FText::AsNumber(AssetsCompressedSize).ToString(), *FText::AsNumber(TotalCompressedSize).ToString(), 100 * PercentAssets);
	UE_LOGF(LogIoStore, Display, "Remaining data by chunk type:");
	for (uint8 TypeIndex = 0; TypeIndex < (uint8)EIoChunkType::MAX; TypeIndex++)
	{
		if (RemainingByType[TypeIndex] != 0)
		{
			UE_LOGF(LogIoStore, Display, "    %-24ls%ls", *LexToString((EIoChunkType)TypeIndex), *FText::AsNumber(RemainingByType[TypeIndex]).ToString());
		}
	}
}

static bool SaveAssetRegistry(const FString& InAssetRegistryFileName, FAssetRegistryState& InAssetRegistry, uint64* OutDevArHash)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SavingAssetRegistry);
	FLargeMemoryWriter SerializedAssetRegistry;
	if (InAssetRegistry.Save(SerializedAssetRegistry, FAssetRegistrySerializationOptions(UE::AssetRegistry::ESerializationTarget::ForDevelopment)) == false)
	{
		UE_LOGF(LogIoStore, Error, "Failed to serialize asset registry to memory.");
		return false;
	}

	if (OutDevArHash)
	{
		*OutDevArHash = UE::Cook::FCookMetadataState::ComputeHashOfDevelopmentAssetRegistry(MakeMemoryView(SerializedAssetRegistry.GetData(), SerializedAssetRegistry.TotalSize()));
	}

	FString OutputFileName = InAssetRegistryFileName + TEXT(".temp");

	TUniquePtr<FArchive> Writer = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*OutputFileName));
	if (!Writer)
	{
		UE_LOGF(LogIoStore, Error, "Failed to open destination asset registry. (%ls)", *OutputFileName);
		return false;
	}

	Writer->Serialize(SerializedAssetRegistry.GetData(), SerializedAssetRegistry.TotalSize());

	// Always explicitly close to catch errors from flush/close
	Writer->Close();

	if (Writer->IsError() || Writer->IsCriticalError())
	{
		UE_LOGF(LogIoStore, Error, "Failed to write asset registry to disk. (%ls)", *OutputFileName);
		return false;
	}

	// Move our temp file over the original asset registry.
	if (IFileManager::Get().Move(*InAssetRegistryFileName, *OutputFileName) == false)
	{
		// Error already logged by FileManager
		return false;
	}

	UE_LOGF(LogIoStore, Display, "Saved asset registry to disk. (%ls)", *InAssetRegistryFileName);

	return true;
}

struct FIoStoreWriterInfo
{
	UE::Cook::EPluginSizeTypes SizeType;
	FName PakChunkName;
};

static bool DoAssetRegistryWritebackDuringStage(
	EAssetRegistryWritebackMethod InMethod, 
	bool bInWritePluginMetadata,
	FCookedPackageStore* InPackageStore,
	const FString& InCookedDir, 
	bool bInCompressionEnabled,
	TArray<TSharedPtr<IIoStoreWriter>>& InIoStoreWriters, 
	TArray<FIoStoreWriterInfo>& InIoStoreWriterInfos,
	FShaderAssociationInfo& InShaderAssociationInfo
)
{
	// This version called during container creation.
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateAssetRegistryWithSizeInfo);
	UE_LOGF(LogIoStore, Display, "Adding staging metadata to asset registry...");

	// The overwhelming majority of time for the asset registry writeback is loading and saving.
	FString AssetRegistryFileName;
	FAssetRegistryState AssetRegistry;
	FString CookMetadataFileName;
	UE::Cook::FCookMetadataState CookMetadata;
	ECookMetadataFiles FilesNeeded = ECookMetadataFiles::AssetRegistry;

	// We always need the cook metadata in order to update the corresponding hash.
	EnumAddFlags(FilesNeeded, ECookMetadataFiles::CookMetadata);

	if (FindAndLoadMetadataFiles(InPackageStore, InCookedDir, FilesNeeded, AssetRegistry, &AssetRegistryFileName, &CookMetadata, &CookMetadataFileName) == ECookMetadataFiles::None)
	{
		// already logged
		return false;
	}

	//
	// We want to separate out the sizes based on where they go in the end product.
	//
	uint64 UnassignableShaderCodeBytes = 0;
	uint64 AssignableShaderCodeBytes = 0;
	UE::Cook::FPluginSizeInfo ProductSize;
	TMap<FPackageId, TArray<FIoStoreChunkSource, TInlineAllocator<2>>> PackageToChunks;
	{
		int32 IoStoreWriterIndex = 0;
		for (TSharedPtr<IIoStoreWriter> IoStoreWriter : InIoStoreWriters)
		{
			IoStoreWriter->EnumerateChunks(
				[&PackageToChunks, 
				 IoStoreWriterInfo = InIoStoreWriterInfos[IoStoreWriterIndex],
				 &ProductSize, 
				 &InShaderAssociationInfo,
				 &UnassignableShaderCodeBytes,
				 &AssignableShaderCodeBytes
				 ](const FIoStoreTocChunkInfo& ChunkInfo)
			{			
				ProductSize[IoStoreWriterInfo.SizeType] += ChunkInfo.CompressedSize;

				// Shader code chunks don't have the package in their chunk id, so we have to use other data to look
				// it up and find it.
				FPackageId PackageId = PackageIdFromChunkId(ChunkInfo.Id);
				if (ChunkInfo.ChunkType == EIoChunkType::ShaderCode)
				{
					// Update size info for the shader.
					FShaderAssociationInfo::FShaderChunkInfo* ShaderChunkInfo = InShaderAssociationInfo.ShaderChunkInfos.Find({IoStoreWriterInfo.PakChunkName, ChunkInfo.Id});
					ShaderChunkInfo->CompressedSize = ChunkInfo.CompressedSize;
					ShaderChunkInfo->SizeType = IoStoreWriterInfo.SizeType;

					// Shaders don't put their package in their chunk - they are just a hash. We have the list of them
					// already in the shader association so we don't bother adding here. However some can't get assigned to
					// anything and so we track that size for reporting.
					if (ShaderChunkInfo->Type == FShaderAssociationInfo::FShaderChunkInfo::Global ||
						ShaderChunkInfo->Type == FShaderAssociationInfo::FShaderChunkInfo::Orphan)
					{
						UnassignableShaderCodeBytes += ChunkInfo.CompressedSize;
					}
					else
					{
						AssignableShaderCodeBytes += ChunkInfo.CompressedSize;
					}
				}
				else
				{
					PackageToChunks.FindOrAdd(PackageId).Add({ChunkInfo, IoStoreWriterInfo.SizeType});
				}
				return true;
			});

			IoStoreWriterIndex++;
		}
	}

	if (bInWritePluginMetadata)
	{
		UpdatePluginMetadataAndWriteJsons(AssetRegistryFileName, PackageToChunks, AssetRegistry, CookMetadata, &InShaderAssociationInfo);
	}

	AddChunkInfoToAssetRegistry(MoveTemp(PackageToChunks), AssetRegistry, &InShaderAssociationInfo, UnassignableShaderCodeBytes, AssignableShaderCodeBytes, ProductSize.TotalSize());
	uint64 UpdatedDevArHash = 0;
	switch (InMethod)
	{
	case EAssetRegistryWritebackMethod::OriginalFile:
		{
			// Write to an adjacent file and move after
			if (SaveAssetRegistry(AssetRegistryFileName, AssetRegistry, &UpdatedDevArHash) == false)
			{
				return false;
			}

			break;
		}
	case EAssetRegistryWritebackMethod::AdjacentFile:
		{
			if (SaveAssetRegistry(AssetRegistryFileName.Replace(TEXT(".bin"), TEXT("Staged.bin")), AssetRegistry, &UpdatedDevArHash) == false)
			{
				return false;
			}
			break;
		}
	default:
		{
			UE_LOGF(LogIoStore, Error, "Invalid asset registry writeback method (should already be handled!) (%d)", int(InMethod));
			return false;
		}
	}

	// Since we modified the dev ar, we need to save the updated hash in the cook metadata so it can still validate.
	CookMetadata.SetSizesPresent(bInCompressionEnabled ? UE::Cook::ECookMetadataSizesPresent::Compressed : UE::Cook::ECookMetadataSizesPresent::Uncompressed);
	CookMetadata.SetAssociatedDevelopmentAssetRegistryHashPostWriteback(UpdatedDevArHash);

	FArrayWriter SerializedCookMetadata;
	CookMetadata.Serialize(SerializedCookMetadata);

	FString TempFileName = CookMetadataFileName + TEXT(".temp");
	if (FFileHelper::SaveArrayToFile(SerializedCookMetadata, *TempFileName))
	{
		// Move our temp file over the original asset registry.
		if (IFileManager::Get().Move(*CookMetadataFileName, *TempFileName) == false)
		{
			// Error already logged by FileManager
			return false;
		}
	}
	else
	{
		UE_LOGF(LogIoStore, Error, "Failed to save temp file for write updated cook metadata file (%ls", *TempFileName);
		return false;
	}
	
	return true;
}

// modified copy from PakFileUtilities
static FName RemapLocalizationPathIfNeeded(const FString& Path)
{
	static constexpr TCHAR L10NString[] = TEXT("/L10N/");
	static constexpr int32 L10NPrefixLength = sizeof(L10NString) / sizeof(TCHAR) - 1;

	int32 BeginL10NOffset = Path.Find(L10NString, ESearchCase::IgnoreCase);
	if (BeginL10NOffset >= 0)
	{
		int32 EndL10NOffset = BeginL10NOffset + L10NPrefixLength;
		int32 NextSlashIndex = Path.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, EndL10NOffset);
		int32 RegionLength = NextSlashIndex - EndL10NOffset;
		if (RegionLength >= 2)
		{
			FString NonLocalizedPath = Path.Mid(0, BeginL10NOffset) + Path.Mid(NextSlashIndex);
			return FName(NonLocalizedPath);
		}
	}
	return NAME_None;
}

void ProcessRedirects(const FIoStoreArguments& Arguments, const TMap<FPackageId, FCookedPackage*>& PackagesMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessRedirects);

	for (const auto& KV : PackagesMap)
	{
		FCookedPackage* Package = KV.Value;
		FName LocalizedSourcePackageName = RemapLocalizationPathIfNeeded(Package->PackageName.ToString());
		if (!LocalizedSourcePackageName.IsNone())
		{
			Package->SourcePackageName = LocalizedSourcePackageName;
			Package->bIsLocalized = true;
		}
	}

	const bool bIsBuildingDLC = Arguments.IsDLC();
	if (bIsBuildingDLC && Arguments.bRemapPluginContentToGame)
	{
		for (const auto& KV : PackagesMap)
		{
			FCookedPackage* Package = KV.Value;
			const int32 DLCNameLen = Arguments.DLCName.Len() + 1;
			FString PackageNameStr = Package->PackageName.ToString();
			FString RedirectedPackageNameStr = TEXT("/Game");
			RedirectedPackageNameStr.AppendChars(*PackageNameStr + DLCNameLen, PackageNameStr.Len() - DLCNameLen);
			FName RedirectedPackageName = FName(*RedirectedPackageNameStr);
			Package->SourcePackageName = RedirectedPackageName;
		}
	}
}

void CreateContainerHeader(FContainerTargetSpec& ContainerTarget, bool bIsOptional)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateContainerHeader);
	FIoContainerHeader& Header = bIsOptional ? ContainerTarget.OptionalSegmentHeader : ContainerTarget.Header;
	Header.ContainerId = bIsOptional ? ContainerTarget.OptionalSegmentContainerId : ContainerTarget.ContainerId;

	int32 NonOptionalSegmentStoreEntriesCount = 0;
	int32 OptionalSegmentStoreEntriesCount = 0;
	if (bIsOptional)
	{
		for (const FCookedPackage* Package : ContainerTarget.Packages)
		{
			if (Package->PackageStoreEntry.HasOptionalSegment())
			{
				if (Package->PackageStoreEntry.IsAutoOptional())
				{
					// Auto optional packages fully replace the non-optional segment
					++NonOptionalSegmentStoreEntriesCount;
				}
				else
				{
					++OptionalSegmentStoreEntriesCount;
				}
			}
		}
	}
	else
	{
		NonOptionalSegmentStoreEntriesCount = ContainerTarget.Packages.Num();
	}

	struct FStoreEntriesWriter
	{
		const int32 StoreTocSize;
		FLargeMemoryWriter StoreTocArchive = FLargeMemoryWriter(0, true);
		FLargeMemoryWriter StoreDataArchive = FLargeMemoryWriter(0, true);

		void Flush(TArray<uint8>& OutputBuffer)
		{
			check(StoreTocArchive.TotalSize() == StoreTocSize);
			if (StoreTocSize)
			{
				const int32 StoreByteCount = StoreTocArchive.TotalSize() + StoreDataArchive.TotalSize();
				OutputBuffer.AddUninitialized(StoreByteCount);
				FBufferWriter PackageStoreArchive(OutputBuffer.GetData(), StoreByteCount);
				PackageStoreArchive.Serialize(StoreTocArchive.GetData(), StoreTocArchive.TotalSize());
				PackageStoreArchive.Serialize(StoreDataArchive.GetData(), StoreDataArchive.TotalSize());
			}
		}
	};

	FStoreEntriesWriter StoreEntriesWriter
	{
		static_cast<int32>(NonOptionalSegmentStoreEntriesCount * sizeof(FFilePackageStoreEntry))
	};

	FStoreEntriesWriter OptionalSegmentStoreEntriesWriter
	{
		static_cast<int32>(OptionalSegmentStoreEntriesCount * sizeof(FFilePackageStoreEntry))
	};

	auto SerializePackageEntryCArrayHeader = [](FStoreEntriesWriter& Writer, int32 Count)
	{
		const int32 RemainingTocSize = Writer.StoreTocSize - Writer.StoreTocArchive.Tell();
		const int32 OffsetFromThis = RemainingTocSize + Writer.StoreDataArchive.Tell();
		uint32 ArrayNum = Count > 0 ? Count : 0;
		uint32 OffsetToDataFromThis = ArrayNum > 0 ? OffsetFromThis : 0;

		Writer.StoreTocArchive << ArrayNum;
		Writer.StoreTocArchive << OffsetToDataFromThis;
	};

	struct FSoftPackageReferenceWriter
	{
		explicit FSoftPackageReferenceWriter(TSet<FPackageId>&& SoftReferences, int32 NumPackageEntries)
			: SoftPackageReferences(SoftReferences.Array())
			, TotalEntrySize(NumPackageEntries * sizeof(FFilePackageStoreEntrySoftReferences))
		{
			SoftPackageReferences.Sort();
		}

		void Append(TConstArrayView<FPackageId> SoftRefs)
		{
			if (SoftPackageReferences.IsEmpty())
			{
				// Skip serialization when there's no soft references for any package
				check(SoftRefs.IsEmpty());
				return;
			}

			const int64 RemainingEntrySize = TotalEntrySize - EntryAr.Tell();
			const int64 OffsetFromThis = RemainingEntrySize + DataAr.Tell();
			uint32		ArrayNum = static_cast<uint32>(SoftRefs.Num());
			uint32		OffsetToDataFromThis = ArrayNum > 0 ? OffsetFromThis : 0;

			EntryAr << ArrayNum;
			EntryAr << OffsetToDataFromThis;

			for (FPackageId SoftRef : SoftRefs)
			{
				int32 Index = Algo::BinarySearch(SoftPackageReferences, SoftRef);
				check(Index != INDEX_NONE);
				DataAr << Index;
			}
		}

		void Flush(FIoContainerHeaderSoftPackageReferences& OutContainerSoftReferences)
		{
			const int64 TotalSize = EntryAr.TotalSize() + DataAr.TotalSize();
			if (TotalSize == 0)
			{
				check(SoftPackageReferences.IsEmpty());
				OutContainerSoftReferences.Empty();
				return;
			}

			OutContainerSoftReferences.bContainsSoftPackageReferences = true;
			OutContainerSoftReferences.PackageIds = MoveTemp(SoftPackageReferences);
			OutContainerSoftReferences.PackageIndices.AddUninitialized(TotalSize);
			FBufferWriter Ar(OutContainerSoftReferences.PackageIndices.GetData(), TotalSize);
			Ar.Serialize(EntryAr.GetData(), EntryAr.TotalSize());
			Ar.Serialize(DataAr.GetData(), DataAr.TotalSize());
		}

	private:
		TArray<FPackageId>	SoftPackageReferences;
		const int64			TotalEntrySize;
		FLargeMemoryWriter	EntryAr;
		FLargeMemoryWriter	DataAr;
	};

	TArray<const FCookedPackage*> SortedPackages(ContainerTarget.Packages);
	Algo::Sort(SortedPackages, [](const FCookedPackage* A, const FCookedPackage* B)
		{
			return A->GlobalPackageId < B->GlobalPackageId;
		});

	Header.PackageIds.Reserve(NonOptionalSegmentStoreEntriesCount);
	Header.OptionalSegmentPackageIds.Reserve(OptionalSegmentStoreEntriesCount);
	FPackageStoreNameMapBuilder RedirectsNameMapBuilder;
	RedirectsNameMapBuilder.SetNameMapType(FMappedName::EType::Container);
	TSet<FName> AllLocalizedPackages;
	if (bIsOptional)
	{
		for (const FCookedPackage* Package : SortedPackages)
		{
			const FPackageStoreEntryResource& Entry = Package->PackageStoreEntry;
			if (Entry.HasOptionalSegment())
			{
				FStoreEntriesWriter* TargetEntriesWriter;
				if (Entry.IsAutoOptional())
				{
					Header.PackageIds.Add(Package->GlobalPackageId);
					TargetEntriesWriter = &StoreEntriesWriter;
				}
				else
				{
					Header.OptionalSegmentPackageIds.Add(Package->GlobalPackageId);
					TargetEntriesWriter = &OptionalSegmentStoreEntriesWriter;
				}

				// OptionalImportedPackages
				const TArray<FPackageId>& OptionalSegmentImportedPackageIds = Entry.OptionalSegmentImportedPackageIds;
				SerializePackageEntryCArrayHeader(*TargetEntriesWriter, OptionalSegmentImportedPackageIds.Num());
				for (FPackageId OptionalSegmentImportedPackageId : OptionalSegmentImportedPackageIds)
				{
					check(OptionalSegmentImportedPackageId.IsValid());
					TargetEntriesWriter->StoreDataArchive << OptionalSegmentImportedPackageId;
				}

				// ShaderMapHashes is N/A for optional segments
				SerializePackageEntryCArrayHeader(*TargetEntriesWriter, 0);
			}
		}
	}
	else
	{
		TSet<FPackageId> AllSoftPackageReferences;
		for (const FCookedPackage* Package : SortedPackages)
		{
			for (FPackageId SoftRef : Package->PackageStoreEntry.SoftPackageReferences)
			{
				AllSoftPackageReferences.Add(SoftRef);
			}
		}

		FSoftPackageReferenceWriter SoftRefsWriter(MoveTemp(AllSoftPackageReferences), NonOptionalSegmentStoreEntriesCount);

		for (const FCookedPackage* Package : SortedPackages)
		{
			const FPackageStoreEntryResource& Entry = Package->PackageStoreEntry;
			Header.PackageIds.Add(Package->GlobalPackageId);
			if (!Package->SourcePackageName.IsNone())
			{
				RedirectsNameMapBuilder.MarkNameAsReferenced(Package->SourcePackageName);
				FMappedName MappedSourcePackageName = RedirectsNameMapBuilder.MapName(Package->SourcePackageName);
				if (Package->bIsLocalized)
				{
					if (!AllLocalizedPackages.Contains(Package->SourcePackageName))
					{
						Header.LocalizedPackages.Add({ FPackageId::FromName(Package->SourcePackageName), MappedSourcePackageName });
						AllLocalizedPackages.Add(Package->SourcePackageName);
					}
				}
				else
				{
					Header.PackageRedirects.Add({ FPackageId::FromName(Package->SourcePackageName), Package->GlobalPackageId, MappedSourcePackageName });
				}
			}

			// ImportedPackages
			const TArray<FPackageId>& ImportedPackageIds = Entry.ImportedPackageIds;
			SerializePackageEntryCArrayHeader(StoreEntriesWriter, ImportedPackageIds.Num());
			for (FPackageId ImportedPackageId : ImportedPackageIds)
			{
				check(ImportedPackageId.IsValid());
				StoreEntriesWriter.StoreDataArchive << ImportedPackageId;
			}

			// ShaderMapHashes
			const TArray<FShaderHash>& ShaderMapHashes = Package->ShaderMapHashes;
			SerializePackageEntryCArrayHeader(StoreEntriesWriter, ShaderMapHashes.Num());
			for (const FShaderHash& ShaderMapHash : ShaderMapHashes)
			{
				StoreEntriesWriter.StoreDataArchive << const_cast<FShaderHash&>(ShaderMapHash);
			}

			// SoftPackageReferences
			SoftRefsWriter.Append(Entry.SoftPackageReferences);
		}

		SoftRefsWriter.Flush(Header.SoftPackageReferences);
	}

	// Inject dedup-originated package redirects (see DetectDuplicatesForContainer).
	const TArray<FPackageRedirectRequest>& DedupRedirects = bIsOptional
		? ContainerTarget.OptionalSegmentPackageRedirects
		: ContainerTarget.PackageRedirects;
	for (const FPackageRedirectRequest& Req : DedupRedirects)
	{
		RedirectsNameMapBuilder.MarkNameAsReferenced(Req.SourcePackageName);
		FMappedName Mapped = RedirectsNameMapBuilder.MapName(Req.SourcePackageName);
		Header.PackageRedirects.Add({ Req.SourcePackageId, Req.TargetPackageId, Mapped });
	}

	Header.RedirectsNameMap = RedirectsNameMapBuilder.GetNameMap();

	StoreEntriesWriter.Flush(Header.StoreEntries);
	OptionalSegmentStoreEntriesWriter.Flush(Header.OptionalSegmentStoreEntries);
}

static FIoChunkId GetEffectiveTargetFileChunkId(const FContainerTargetFile& TF, bool* bOutGoesToOptionalWriter = nullptr)
{
	switch (TF.ChunkType)
	{
	case EContainerChunkType::OptionalSegmentPackageData:
	{
		if (bOutGoesToOptionalWriter)
		{
			*bOutGoesToOptionalWriter = true;
		}
		if (TF.Package && TF.Package->PackageStoreEntry.IsAutoOptional())
		{
			return CreateIoChunkId(TF.Package->GlobalPackageId.Value(), 0, EIoChunkType::ExportBundleData);
		}
		return TF.ChunkId;
	}
	case EContainerChunkType::OptionalSegmentBulkData:
	{
		if (bOutGoesToOptionalWriter)
		{
			*bOutGoesToOptionalWriter = true;
		}
		if (TF.Package && TF.Package->PackageStoreEntry.IsAutoOptional())
		{
			return CreateBulkDataIoChunkId(TF.Package->GlobalPackageId.Value(), 0, TF.BulkDataCookedIndex.GetValue(), EIoChunkType::BulkData);
		}
		return TF.ChunkId;
	}
	default:
	{
		if (bOutGoesToOptionalWriter)
		{
			*bOutGoesToOptionalWriter = false;
		}
		return TF.ChunkId;
	}
	}
}

static void DetectDuplicatesForContainer(
	FContainerTargetSpec& ContainerTarget,
	FCookedPackageStore& PackageStore,
	IIoStoreWriterReferenceChunkDatabase* ReferenceChunkDatabase,
	TMap<FIoContainerId, FDetectDuplicatesStats>& OutPerWriterStats)
{
	if (!ContainerTarget.IoStoreWriter || ContainerTarget.TargetFiles.IsEmpty())
	{
		return;
	}

	// per-writer stats (main=0, optional=1)
	OutPerWriterStats.FindOrAdd(ContainerTarget.ContainerId);
	if (ContainerTarget.OptionalSegmentIoStoreWriter)
	{
		OutPerWriterStats.FindOrAdd(ContainerTarget.OptionalSegmentContainerId);
	}
	FDetectDuplicatesStats DiscardedOpt;
	FDetectDuplicatesStats* Stats[2] = {
		&OutPerWriterStats.FindChecked(ContainerTarget.ContainerId),
		ContainerTarget.OptionalSegmentIoStoreWriter
			? &OutPerWriterStats.FindChecked(ContainerTarget.OptionalSegmentContainerId)
			: &DiscardedOpt
	};

	TMap<FIoHash, TArray<int32>> Groups[2]; // chunk hash -> targetfile[]
	Groups[0].Reserve(ContainerTarget.TargetFiles.Num());

	// collect chunks into groups of matching source data hash
	for (int32 i = 0; i < ContainerTarget.TargetFiles.Num(); ++i)
	{
		const FContainerTargetFile& TF = ContainerTarget.TargetFiles[i];

		if (TF.ChunkHash.IsZero() || TF.SourceSize == 0)
		{
			// count and report chunks that have size but no hash
			if (TF.SourceSize > 0 && TF.ChunkHash.IsZero())
			{
				// TODO should we hash them ourselves?
				++Stats[0]->SizedButNoHash;
				Stats[0]->SizedButNoHashBytes += TF.SourceSize;
			}

			continue;
		}

		// skip target files with invalid or not-yet-assigned ChunkIds, e.g. ShaderCodeLibrary placeholders
		if (!TF.ChunkId.IsValid() || TF.ChunkId.GetChunkType() == EIoChunkType::Invalid)
		{
			continue;
		}

		bool bGoesToOptional = false;
		GetEffectiveTargetFileChunkId(TF, &bGoesToOptional);
		if (bGoesToOptional && !ContainerTarget.OptionalSegmentIoStoreWriter)
		{
			continue;
		}

		Groups[bGoesToOptional ? 1 : 0].FindOrAdd(TF.ChunkHash).Add(i);
	}

	for (int32 WriterIdx = 0; WriterIdx < 2; ++WriterIdx)
	{
		const bool bGoesToOptional = (WriterIdx == 1);
		FDetectDuplicatesStats& PassStats = *Stats[WriterIdx];
		const FIoContainerId WriterContainerId = bGoesToOptional
			? ContainerTarget.OptionalSegmentContainerId
			: ContainerTarget.ContainerId;

		// for each chunk in a group, select canonical chunk which bytes will stay in the container, deduplicate the rest
		for (TPair<FIoHash, TArray<int32>>& Pair : Groups[WriterIdx])
		{
			TArray<int32>& Candidates = Pair.Value;

			// skip hashes with no duplicates
			if (Candidates.Num() < 2)
			{
				continue;
			}

			// order by chunkid to make it deterministic across builds
			Candidates.Sort([&](int32 A, int32 B)
			{
				return ContainerTarget.TargetFiles[A].ChunkId < ContainerTarget.TargetFiles[B].ChunkId;
			});

			// canonical chunk must be as strict as any deduplicated chunk in independent dimensions:
			// - mmap: if any member is mmap, the canonical must be mmap, non-mmap dups can safely read from an mmap-aligned offset
			//         the reverse would leave mmap dups pointing at an unaligned offset
			// - uncompressed: if any member is bForceUncompressed, the canonical must be stored uncompressed
			const bool bGroupHasMmap = Algo::AnyOf(Candidates, [&](int32 Idx)
			{
				return ContainerTarget.TargetFiles[Idx].ChunkType == EContainerChunkType::MemoryMappedBulkData;
			});
			const bool bGroupHasForceUncompressed = Algo::AnyOf(Candidates, [&](int32 Idx)
			{
				return ContainerTarget.TargetFiles[Idx].bForceUncompressed;
			});
			auto IsEligibleCanonical = [&](int32 Idx)
			{
				const FContainerTargetFile& TF = ContainerTarget.TargetFiles[Idx];
				const bool bTFIsMmap = TF.ChunkType == EContainerChunkType::MemoryMappedBulkData;

				if (bGroupHasMmap && !bTFIsMmap) // must be mmap to be eligible
				{
					return false;
				}

				if (bGroupHasForceUncompressed
					&& !TF.bForceUncompressed
					&& !bTFIsMmap) // a mmaped entry will also be uncompressed
				{
					return false;
				}

				return true;
			};

			int32 CanonicalIdx = INDEX_NONE;

			// prefer a chunk present in the previous release so patch-in-place deltas stay small
			if (ReferenceChunkDatabase)
			{
				for (const int32 Idx : Candidates)
				{
					if (!IsEligibleCanonical(Idx))
					{
						continue;
					}

					const FContainerTargetFile& CandidateTargetFile = ContainerTarget.TargetFiles[Idx];
					const FIoChunkId CandidateEffectiveId = GetEffectiveTargetFileChunkId(CandidateTargetFile);

					int32 CandidateOutBlocks = 0;
					if (ReferenceChunkDatabase->ChunkExists(WriterContainerId, CandidateTargetFile.ChunkHash, CandidateEffectiveId, CandidateOutBlocks))
					{
						CanonicalIdx = Idx;
						++PassStats.PriorReleaseCanonicals;
						break;
					}
				}
			}

			// otherwise pick first eligible
			if (CanonicalIdx == INDEX_NONE)
			{
				for (const int32 Idx : Candidates)
				{
					if (IsEligibleCanonical(Idx))
					{
						CanonicalIdx = Idx;
						break;
					}
				}
			}

			// ignore dedup if no eligible was found
			if (CanonicalIdx == INDEX_NONE)
			{
				continue;
			}

			const FIoChunkId CanonicalEffectiveId = GetEffectiveTargetFileChunkId(ContainerTarget.TargetFiles[CanonicalIdx]);

			// memcmp every candidate against each other to ensure no hash collisions
			FIoBuffer ReferenceBytes;
			bool bGroupVerified = true;
			for (const int32 Idx : Candidates)
			{
				const FContainerTargetFile& CandidateTargetFile = ContainerTarget.TargetFiles[Idx];
				TIoStatusOr<FIoBuffer> CandidateRead = PackageStore.ReadChunk(CandidateTargetFile.ChunkId);
				if (!CandidateRead.IsOk())
				{
					UE_LOGF(LogIoStore, Warning, "DetectDuplicates: failed to read candidate chunk (hash %ls): %ls - skipping whole group", *LexToString(Pair.Key), *CandidateRead.Status().ToString());
					bGroupVerified = false;
					break;
				}

				FIoBuffer CandidateBytes = CandidateRead.ConsumeValueOrDie();
				if (ReferenceBytes.DataSize() == 0)
				{
					ReferenceBytes = CandidateBytes;
					continue;
				}

				if (CandidateBytes.DataSize() != ReferenceBytes.DataSize() ||
					FMemory::Memcmp(ReferenceBytes.Data(), CandidateBytes.Data(), CandidateBytes.DataSize()) != 0)
				{
					UE_LOGF(LogIoStore, Warning, "DetectDuplicates: hash collision on %ls - skipping whole group", *LexToString(Pair.Key));
					bGroupVerified = false;
					break;
				}
			}

			if (!bGroupVerified)
			{
				continue;
			}

			// set DuplicateOfChunkId on deduplicated chunks
			for (const int32 Idx : Candidates)
			{
				if (Idx == CanonicalIdx)
				{
					continue;
				}

				FContainerTargetFile& CandidateTargetFile = ContainerTarget.TargetFiles[Idx];
				CandidateTargetFile.DuplicateOfChunkId = CanonicalEffectiveId;

				PassStats.DeduplicatedChunks++;
			}
		}
	}

#if 0 // this code produces package redirects for optional containers where there shouldn't be any packages, disabled for now
	// per package/target-file signature entry: positional bytes (ChunkIndex + Group + ChunkType) + content hash.
	struct FSigEntry
	{
		uint8 ChunkIdTail[4]; // ChunkId bytes [8..11]: ChunkIndex (2) + Group (1) + Type (1)
		FIoHash ChunkHash;

		bool operator<(const FSigEntry& Other) const
		{
			const int32 IdCmp = FMemory::Memcmp(ChunkIdTail, Other.ChunkIdTail, sizeof(ChunkIdTail));
			if (IdCmp != 0)
			{
				return IdCmp < 0;
			}
			else
			{
				return ChunkHash < Other.ChunkHash;
			}
		}

		bool operator==(const FSigEntry& Other) const
		{
			return FMemory::Memcmp(ChunkIdTail, Other.ChunkIdTail, sizeof(ChunkIdTail)) == 0 && ChunkHash == Other.ChunkHash;
		}
	};

	// emit full-package redirects via signature-based equivalence classes.
	for (int32 WriterIdx = 0; WriterIdx < 2; ++WriterIdx)
	{
		const bool bGoesToOptional = (WriterIdx == 1);
		if (bGoesToOptional && !ContainerTarget.OptionalSegmentIoStoreWriter)
		{
			continue;
		}

		TMap<FCookedPackage*, TArray<FSigEntry>> PackageEntries;
		for (const FContainerTargetFile& TF : ContainerTarget.TargetFiles)
		{
			// look for ChunkId's that belong to a package with name, have size and hash, and are of valid type and of [package_id, bytes] structure
			if (!TF.Package ||
				TF.Package->PackageName.IsNone() ||
				TF.ChunkHash.IsZero() ||
				TF.SourceSize == 0 ||
				!TF.ChunkId.IsValid() ||
				TF.ChunkId.GetChunkType() == EIoChunkType::Invalid ||
				TF.ChunkId.GetChunkType() == EIoChunkType::ExternalFile) // ExternalFile's ChunkId is a filename hash, not PackageId+position
			{
				continue;
			}

			bool bTFGoesToOptional = false;
			const FIoChunkId EffectiveId = GetEffectiveTargetFileChunkId(TF, &bTFGoesToOptional);
			if (bTFGoesToOptional != bGoesToOptional)
			{
				continue;
			}

			// signature construction assumes EffectiveId's first 8 bytes are the owning PackageId, e.g. CreateIoChunkId / CreateBulkDataIoChunkId
			// sanity check that schema is correct
			const uint64 ExpectedPackageId = TF.Package->GlobalPackageId.Value();
			ensureMsgf(
				FMemory::Memcmp(EffectiveId.GetData(), &ExpectedPackageId, sizeof(uint64)) == 0,
				TEXT("EffectiveId prefix does not match owning PackageId for chunk %s"),
				*LexToString(EffectiveId));

			FSigEntry Entry = {};
			FMemory::Memcpy(Entry.ChunkIdTail, EffectiveId.GetData() + 8, sizeof(Entry.ChunkIdTail));
			Entry.ChunkHash = TF.ChunkHash;
			PackageEntries.FindOrAdd(TF.Package).Add(Entry);
		}

		// sort each package's entries then hash to collapse equivalents into the same bucket
		TMap<FIoHash, TArray<FCookedPackage*>> EquivalenceClasses;
		for (TPair<FCookedPackage*, TArray<FSigEntry>>& KV : PackageEntries)
		{
			KV.Value.Sort(); // ensure stable hash between packages

			FIoHashBuilder Builder = {};
			for (const FSigEntry& E : KV.Value)
			{
				Builder.Update(E.ChunkIdTail, sizeof(E.ChunkIdTail));
				Builder.Update(E.ChunkHash.GetBytes(), sizeof(FIoHash::ByteArray));
			}

			EquivalenceClasses.FindOrAdd(Builder.Finalize()).Add(KV.Key);
		}

		// for each class pick lowest PackageId as target to make it deterministic across builds, and redirect the rest 
		for (TPair<FIoHash, TArray<FCookedPackage*>>& Class : EquivalenceClasses)
		{
			TArray<FCookedPackage*>& Packages = Class.Value;
			if (Packages.Num() < 2)
			{
				continue;
			}

			Packages.Sort([](const FCookedPackage& A, const FCookedPackage& B)
			{
				return A.GlobalPackageId < B.GlobalPackageId;
			});

			FCookedPackage* TargetPackage = Packages[0];
			const TArray<FSigEntry>& TargetEntries = PackageEntries.FindChecked(TargetPackage);
			for (int32 i = 1; i < Packages.Num(); ++i)
			{
				FCookedPackage* SourcePackage = Packages[i];

				// EquivalenceClasses hash collision check
				const TArray<FSigEntry>& SourceEntries = PackageEntries.FindChecked(SourcePackage);
				if (!ensureMsgf(SourceEntries == TargetEntries,
						TEXT("DetectDuplicates: signature hash collision between %ls and %ls — skipping redirect"),
						*SourcePackage->PackageName.ToString(), *TargetPackage->PackageName.ToString()))
				{
					continue;
				}

				FPackageRedirectRequest Redirect = {};
				Redirect.SourcePackageId = SourcePackage->GlobalPackageId;
				Redirect.TargetPackageId = TargetPackage->GlobalPackageId;
				Redirect.SourcePackageName = SourcePackage->PackageName;

				if (bGoesToOptional)
				{
					ContainerTarget.OptionalSegmentPackageRedirects.Add(Redirect);
				}
				else
				{
					ContainerTarget.PackageRedirects.Add(Redirect);
				}

				Stats[WriterIdx]->Redirects++;
			}
		}
	}

	// deterministic sorting of package redirects
	auto SortPackageRedirects = [](const FPackageRedirectRequest& A, const FPackageRedirectRequest& B)
	{
		return A.SourcePackageId < B.SourcePackageId;
	};
	ContainerTarget.PackageRedirects.Sort(SortPackageRedirects);
	ContainerTarget.OptionalSegmentPackageRedirects.Sort(SortPackageRedirects);
#endif
}

static int32 CreateTarget(const FIoStoreArguments& Arguments, const FIoStoreWriterSettings& GeneralIoWriterSettings)
{
	IOSTORE_CPU_SCOPE(CreateTarget);
	TGuardValue<int32> GuardAllowUnversionedContentInEditor(GAllowUnversionedContentInEditor, 1);

	TSharedPtr<IIoStoreWriterReferenceChunkDatabase> ChunkDatabase;
	if (Arguments.ReferenceChunkGlobalContainerFileName.Len())
	{
		ChunkDatabase = MakeShared<FIoStoreChunkDatabase>();
		FIoStoreChunkDatabase* ChunkDbPtr = (FIoStoreChunkDatabase*)ChunkDatabase.Get();

		if (ChunkDbPtr->Init(Arguments.ReferenceChunkGlobalContainerFileName, Arguments.ReferenceChunkAdditionalContainersPath, Arguments.ReferenceChunkKeys) == false)
		{
			// broken reference database will likely result in a large patch size, treat it as error
			UE_LOGF(LogIoStore, Error, "Failed to initialize reference chunk store. Pak will continue.");
			ChunkDatabase.Reset();
		}
	}

	TSharedPtr<IIoStoreWriterReferenceChunkDatabase> PatchInPlaceChunkDatabase;
	if (Arguments.PatchInPlaceReferenceContainerPathOverride.Len())
	{
		PatchInPlaceChunkDatabase = MakeShared<FIoStoreChunkDatabase>();
		FIoStoreChunkDatabase* ChunkDbPtr = (FIoStoreChunkDatabase*)PatchInPlaceChunkDatabase.Get();

		if (ChunkDbPtr->Init(Arguments.PatchInPlaceReferenceContainerPathOverride, Arguments.PatchInPlaceReferenceAdditionalContainersPath, Arguments.PatchInPlaceReferenceChunkKeys) == false)
		{
			UE_LOGF(LogIoStore, Warning, "Failed to initialize patch-in-place reference chunk store. Pak will continue.");
			PatchInPlaceChunkDatabase.Reset();
		}
	}

	TArray<FCookedPackage*> Packages;
	FPackageNameMap PackageNameMap;
	FPackageIdMap PackageIdMap;

	FPackageStoreOptimizer PackageStoreOptimizer;
	PackageStoreOptimizer.Initialize(*Arguments.ScriptObjects);

	TArray<FContainerTargetSpec*> ContainerTargets;
	UE_LOGF(LogIoStore, Display, "Creating container targets...");
	{
		IOSTORE_CPU_SCOPE(CreateContainerTargets);
		InitializeContainerTargetsAndPackages(Arguments, Packages, PackageNameMap, PackageIdMap, ContainerTargets);
	}

	TUniquePtr<FIoStoreWriterContext> IoStoreWriterContext;
	{
		IOSTORE_CPU_SCOPE(InitializeIoStoreWriters);
		IoStoreWriterContext.Reset(new FIoStoreWriterContext());
		FIoStatus IoStatus = IoStoreWriterContext->Initialize(GeneralIoWriterSettings);
		check(IoStatus.IsOk());
	}
	TArray<FString> OnDemandContainers;
	TArray<TSharedPtr<IIoStoreWriter>> IoStoreWriters;
	TArray<FIoStoreWriterInfo> IoStoreWriterInfos;
	TSharedPtr<IIoStoreWriter> GlobalIoStoreWriter;
	{
		IOSTORE_CPU_SCOPE(InitializeWriters);
		if (!Arguments.IsDLC())
		{
			IOSTORE_CPU_SCOPE(InitializeGlobalWriter);
			FIoContainerSettings GlobalContainerSettings;
			if (Arguments.bSign)
			{
				GlobalContainerSettings.SigningKey = Arguments.KeyChain.GetSigningKey();
				GlobalContainerSettings.ContainerFlags |= EIoContainerFlags::Signed;
			}
			GlobalIoStoreWriter = IoStoreWriterContext->CreateContainer(*Arguments.GlobalContainerPath, GlobalContainerSettings);
			GlobalIoStoreWriter->SetReferenceChunkDatabase(ChunkDatabase, PatchInPlaceChunkDatabase);
			IoStoreWriters.Add(GlobalIoStoreWriter);
			IoStoreWriterInfos.Add({UE::Cook::EPluginSizeTypes::Installed, "global"});
		}
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			IOSTORE_CPU_SCOPE(InitializeWriter);
			check(ContainerTarget->ContainerId.IsValid());

			if (ContainerTarget->OutputPath.IsEmpty())
			{
				continue;
			}

			if (!ContainerTarget->StageLooseFileRootPath.IsEmpty())
			{
				FLooseFilesWriterSettings WriterSettings;
				WriterSettings.TargetRootPath = ContainerTarget->StageLooseFileRootPath;
				ContainerTarget->IoStoreWriter = MakeLooseFilesIoStoreWriter(WriterSettings);
				IoStoreWriterInfos.Add({UE::Cook::EPluginSizeTypes::Streaming, ContainerTarget->Name}); // LooseFiles currently end up as a streamed source.
				IoStoreWriters.Add(ContainerTarget->IoStoreWriter);
			}
			else
			{
				FIoContainerSettings ContainerSettings;
				ContainerSettings.ContainerId = ContainerTarget->ContainerId;
				ContainerSettings.ContainerFlags = ContainerTarget->ContainerFlags; 
				if (Arguments.bCreateDirectoryIndex)
				{
					ContainerSettings.ContainerFlags |= EIoContainerFlags::Indexed;
				}
				if (EnumHasAnyFlags(ContainerTarget->ContainerFlags, EIoContainerFlags::Encrypted))
				{
					if (const FNamedAESKey* Key = Arguments.KeyChain.GetEncryptionKeys().Find(ContainerTarget->EncryptionKeyGuid))
					{
						ContainerSettings.EncryptionKeyGuid = ContainerTarget->EncryptionKeyGuid;
						ContainerSettings.EncryptionKey = Key->Key;
					}
					else
					{
						UE_LOGF(LogIoStore, Error, "Failed to find encryption key '%ls'", *ContainerTarget->EncryptionKeyGuid.ToString());
						return -1;
					}
				}
				if (EnumHasAnyFlags(ContainerTarget->ContainerFlags, EIoContainerFlags::Signed))
				{
					ContainerSettings.SigningKey = Arguments.KeyChain.GetSigningKey();
					ContainerSettings.ContainerFlags |= EIoContainerFlags::Signed;
				}
				ContainerSettings.bGenerateDiffPatch = ContainerTarget->bGenerateDiffPatch;
				ContainerTarget->IoStoreWriter = IoStoreWriterContext->CreateContainer(*ContainerTarget->OutputPath, ContainerSettings);
				ContainerTarget->IoStoreWriter->EnableDiskLayoutOrdering(ContainerTarget->PatchSourceReaders);
				ContainerTarget->IoStoreWriter->SetReferenceChunkDatabase(ChunkDatabase, PatchInPlaceChunkDatabase);
				IoStoreWriters.Add(ContainerTarget->IoStoreWriter);

				if (EnumHasAnyFlags(ContainerTarget->ContainerFlags, EIoContainerFlags::OnDemand))
				{
					IoStoreWriterInfos.Add({UE::Cook::EPluginSizeTypes::Streaming, ContainerTarget->Name});
				}
				else
				{
					// There's no way to know whether a container is optional without parsing the filename, see
					// EIoStoreWriterType.
					FString BaseFileName = FPaths::GetBaseFilename(ContainerTarget->OutputPath, true);
					// Strip the platform identifier off the pak
					if (int32 DashIndex=0; BaseFileName.FindLastChar(TEXT('-'), DashIndex))
					{
						BaseFileName.LeftInline(DashIndex);
					}
					if (BaseFileName.EndsWith(TEXT("optional")))
					{
						IoStoreWriterInfos.Add({UE::Cook::EPluginSizeTypes::Optional, ContainerTarget->Name});
					}
					else
					{
						IoStoreWriterInfos.Add({UE::Cook::EPluginSizeTypes::Installed, ContainerTarget->Name});
					}
				}

				if (!ContainerTarget->OptionalSegmentOutputPath.IsEmpty())
				{
					ContainerSettings.ContainerId = ContainerTarget->OptionalSegmentContainerId;
					ContainerTarget->OptionalSegmentIoStoreWriter = IoStoreWriterContext->CreateContainer(*ContainerTarget->OptionalSegmentOutputPath, ContainerSettings);
					ContainerSettings.ContainerId = ContainerTarget->ContainerId;

					ContainerTarget->OptionalSegmentIoStoreWriter->SetReferenceChunkDatabase(ChunkDatabase, PatchInPlaceChunkDatabase);
					IoStoreWriters.Add(ContainerTarget->OptionalSegmentIoStoreWriter);
					IoStoreWriterInfos.Add({UE::Cook::EPluginSizeTypes::OptionalSegment, ContainerTarget->Name});
				}
				if (EnumHasAnyFlags(ContainerTarget->ContainerFlags, EIoContainerFlags::OnDemand))
				{
					OnDemandContainers.Add(ContainerTarget->OutputPath);
				}
			}
		}
	}

	const bool bIsLegacyStage = !Arguments.PackageStore->HasZenStoreClient();
	if (bIsLegacyStage)
	{
		ParsePackageAssetsFromFiles(Packages, PackageStoreOptimizer);
		if (Arguments.bFileRegions)
		{
			// The file regions for packages are relative to the start of the uexp file so we need to make them relative to the start of the export bundle chunk instead
			for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
			{
				for (FContainerTargetFile& TargetFile : ContainerTarget->TargetFiles)
				{
					if (TargetFile.ChunkType == EContainerChunkType::PackageData)
					{
						uint64 HeaderSize = static_cast<FLegacyCookedPackage*>(TargetFile.Package)->OptimizedPackage->GetHeaderSize();
						for (FFileRegion& Region : TargetFile.FileRegions)
						{
							Region.Offset += HeaderSize;
						}
					}
					else if (TargetFile.ChunkType == EContainerChunkType::OptionalSegmentPackageData)
					{
						uint64 HeaderSize = static_cast<FLegacyCookedPackage*>(TargetFile.Package)->OptimizedOptionalSegmentPackage->GetHeaderSize();
						for (FFileRegion& Region : TargetFile.FileRegions)
						{
							Region.Offset += HeaderSize;
						}
					}
				}
			}
		}
	}

	UE_LOGF(LogIoStore, Display, "Processing shader libraries, compressing with Oodle %ls, level %d (%ls)", FOodleDataCompression::ECompressorToString(Arguments.ShaderOodleCompressor), (int32)Arguments.ShaderOodleLevel, FOodleDataCompression::ECompressionLevelToString(Arguments.ShaderOodleLevel));
	FIoStoreShaderLibraryProcessor ShaderLibraryProcessor;
	FIoStoreShaderLibraryOutput ShaderLibraryOutput;
	ShaderLibraryProcessor.ProcessShaderLibraries(Arguments, ContainerTargets, ShaderLibraryOutput);
	FShaderAssociationInfo& ShaderAssocInfo = ShaderLibraryOutput.AssocInfo;

	FIoStoreWriteRequestManager WriteRequestManager(PackageStoreOptimizer, Arguments.PackageStore.Get());

	TMap<FIoContainerId, FDetectDuplicatesStats> PerWriterDedupStats;

	if (GeneralIoWriterSettings.bDeduplicateChunks && Arguments.PackageStore.IsValid() && Arguments.PackageStore->HasZenStoreClient())
	{
		IOSTORE_CPU_SCOPE(DetectDuplicates);
		UE_LOGF(LogIoStore, Display, "Detecting duplicate chunks...");

		FCookedPackageStore& PackageStore = *Arguments.PackageStore;
		// use PatchInPlaceChunkDatabase if available
		IIoStoreWriterReferenceChunkDatabase* RefDb = PatchInPlaceChunkDatabase.IsValid() ? PatchInPlaceChunkDatabase.Get() : ChunkDatabase.Get();
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			DetectDuplicatesForContainer(*ContainerTarget, PackageStore, RefDb, PerWriterDedupStats);
		}

		FDetectDuplicatesStats Totals = {};
		for (const TPair<FIoContainerId, FDetectDuplicatesStats>& Entry : PerWriterDedupStats)
		{
			Totals += Entry.Value;
		}

		FString TotalExtra;
		if (Totals.PriorReleaseCanonicals > 0)
		{
			TotalExtra += FString::Printf(TEXT(" [%llu canonicals matched prior release]"), Totals.PriorReleaseCanonicals);
		}
		if (Totals.SizedButNoHash > 0)
		{
			TotalExtra += FString::Printf(TEXT(" [%llu chunks missing hash, %.2f MiB skipped from dedup]"), Totals.SizedButNoHash, Totals.SizedButNoHashBytes / (1024.0 * 1024.0));
		}
		UE_LOGF(LogIoStore, Display, "DetectDuplicates total: %llu chunks, %llu pkg redirects%ls", Totals.DeduplicatedChunks, Totals.Redirects, *TotalExtra);
	}

	auto AppendTargetFileChunk = [&WriteRequestManager](FContainerTargetSpec* ContainerTarget, const FContainerTargetFile& TargetFile)
	{
		FIoWriteOptions WriteOptions;
		WriteOptions.DebugName = *TargetFile.DestinationPath;
		WriteOptions.bForceUncompressed = TargetFile.bForceUncompressed;
		WriteOptions.bIsMemoryMapped = TargetFile.ChunkType == EContainerChunkType::MemoryMappedBulkData;
		WriteOptions.FileName = TargetFile.DestinationPath;
		WriteOptions.DuplicateOf = TargetFile.DuplicateOfChunkId;
		bool bIsOptionalSegmentChunk = false;
		const FIoChunkId ChunkId = GetEffectiveTargetFileChunkId(TargetFile, &bIsOptionalSegmentChunk);

		if (bIsOptionalSegmentChunk)
		{
			if (ContainerTarget->OptionalSegmentIoStoreWriter.IsValid())
			{ 
				ContainerTarget->OptionalSegmentIoStoreWriter->Append(ChunkId, WriteRequestManager.Read(TargetFile), WriteOptions);
			}
			else
			{
				UE_LOGF(LogIoStore, Display, "Attempted to write to optionalSegmentIoStoreWriter, but optional segment was invalid for targetfile %ls", *TargetFile.DestinationPath);
			}
		}
		else
		{
			ContainerTarget->IoStoreWriter->Append(ChunkId, WriteRequestManager.Read(TargetFile), WriteOptions);
		}
	};

	{
		IOSTORE_CPU_SCOPE(AppendChunks);
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			if (ContainerTarget->IoStoreWriter)
			{
				for (FContainerTargetFile& TargetFile : ContainerTarget->TargetFiles)
				{
					AppendTargetFileChunk(ContainerTarget, TargetFile);
				}
			}
		}
	}

	UE_LOGF(LogIoStore, Display, "Processing redirects...");
	ProcessRedirects(Arguments, PackageIdMap);

	UE_LOGF(LogIoStore, Display, "Creating disk layout...");
	FString ClusterCSVPath;
	if (FParse::Value(FCommandLine::Get(), TEXT("ClusterCSV="), ClusterCSVPath))
	{
		IOSTORE_CPU_SCOPE(CreateClusterCSV);
		ClusterStatsCsv.CreateOutputFile(ClusterCSVPath);
	}

	// Map and flip the relationship of the ordering options map
	FOrderingOptionsToContainersMap OrderingOptionsOverrides;
	for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
	{
		const FString* PrevMatchedPrefix = nullptr;
		const FString ContainerName = ContainerTarget->Name.ToString();
		for (const TPair<FString, FIoStoreOrderingOptions>& OverridePair : Arguments.OrderingOptionsOverrides)
		{
			const FString& ContainerPrefix = OverridePair.Key;
			const FIoStoreOrderingOptions& OrderingOptions = OverridePair.Value;
			if (ContainerName.StartsWith(ContainerPrefix, ESearchCase::IgnoreCase))
			{
				if (PrevMatchedPrefix != nullptr)
				{
					UE_LOGF(LogIoStore, Fatal, "Found multiple ordering option overrides ('%ls' vs. '%ls') that match '%ls'.", **PrevMatchedPrefix, *ContainerPrefix, *ContainerName);
				}

				PrevMatchedPrefix = &ContainerPrefix;
				UE_LOGF(LogIoStore, Display,
					"Applying ordering options override for %ls (cluster-by-order: %d, alpha-sort-clusters: %d, fallback-mode: %d)",
					*ContainerName, OrderingOptions.bClusterByOrderFilePriority, OrderingOptions.bAlphaSortClusterPackageLists, EnumToUnderlyingType(OrderingOptions.FallbackOrderMode));
				OrderingOptionsOverrides.FindOrAdd(OrderingOptions).Add(ContainerTarget);
			}
		}
	}

	SortPackagesInLoadOrder(Packages, PackageIdMap);
	CreateDiskLayout(ContainerTargets, Packages, Arguments.OrderMaps, PackageIdMap, Arguments.OrderingOptions, OrderingOptionsOverrides);

	{
		IOSTORE_CPU_SCOPE(AppendContainerHeaderChunks);
		for (FContainerTargetSpec* ContainerTarget : ContainerTargets)
		{
			if (ContainerTarget->IoStoreWriter)
			{
				auto WriteContainerHeaderChunk = [](FIoContainerHeader& Header, IIoStoreWriter* IoStoreWriter, const FIoContainerId& IoStoreWriterId)
				{
					FLargeMemoryWriter HeaderAr(0, true);
					HeaderAr << Header;
					int64 DataSize = HeaderAr.TotalSize();
					FIoBuffer ContainerHeaderBuffer(FIoBuffer::AssumeOwnership, HeaderAr.ReleaseOwnership(), DataSize);

					// The header must have the same ID so that the loading code can find it.
					check(IoStoreWriterId == Header.ContainerId);

					FIoWriteOptions WriteOptions;
					WriteOptions.DebugName = TEXT("ContainerHeader");
					WriteOptions.bForceUncompressed = true;
					IoStoreWriter->Append(
						CreateIoChunkId(Header.ContainerId.Value(), 0, EIoChunkType::ContainerHeader),
						ContainerHeaderBuffer,
						WriteOptions);
				};

				CreateContainerHeader(*ContainerTarget, false);
				WriteContainerHeaderChunk(ContainerTarget->Header, ContainerTarget->IoStoreWriter.Get(), ContainerTarget->ContainerId);

				if (ContainerTarget->OptionalSegmentIoStoreWriter)
				{
					CreateContainerHeader(*ContainerTarget, true);
					WriteContainerHeaderChunk(ContainerTarget->OptionalSegmentHeader, ContainerTarget->OptionalSegmentIoStoreWriter.Get(), ContainerTarget->OptionalSegmentContainerId);
				}
			}

		}
	}

	uint64 InitialLoadSize = 0;
	if (GlobalIoStoreWriter)
	{
		IOSTORE_CPU_SCOPE(WriteScriptObjects);
		FIoBuffer ScriptObjectsBuffer = PackageStoreOptimizer.CreateScriptObjectsBuffer();
		InitialLoadSize = ScriptObjectsBuffer.DataSize();
		FIoWriteOptions WriteOptions;
		WriteOptions.DebugName = TEXT("ScriptObjects");
		GlobalIoStoreWriter->Append(CreateIoChunkId(0, 0, EIoChunkType::ScriptObjects), ScriptObjectsBuffer, WriteOptions);
	}

	// Finalize the layout and disable memory throttling for source reads before IoStoreWriterContext->FinalizeWrites that 
	// has its own memory throttling logic.
	IoStoreWriterContext->FinalizeLayout();
	WriteRequestManager.DisableMemoryThrottling();

	UE_LOGF(LogIoStore, Display, "Writing container(s)...");
	{
		IOSTORE_CPU_SCOPE(WritingContainers);

		TFuture<void> FinalizeTask = Async(EAsyncExecution::Thread, [&IoStoreWriterContext]()
		{
			IoStoreWriterContext->FinalizeWrites();
		});

		bool bPrintProgress = true;
		while (bPrintProgress)
		{
			FinalizeTask.WaitFor(FTimespan::FromSeconds(2.0));
			bPrintProgress = !FinalizeTask.IsReady();

			TStringBuilder<1024> ProgressStringBuilder;

			const FIoStoreWriterContext::FProgress Progress = IoStoreWriterContext->GetProgress();
			if (Progress.SerializedChunksCount >= Progress.TotalChunksCount)
			{
				ProgressStringBuilder.Appendf(TEXT("Writing tocs..."));
			}
			else if (Progress.SerializedChunksCount)
			{
				ProgressStringBuilder.Appendf(TEXT("Writing chunks %llu/%llu..."), Progress.SerializedChunksCount, Progress.TotalChunksCount);
				if (Progress.CompressedChunksCount)
				{
					ProgressStringBuilder.Appendf(TEXT(" [%llu compressed]"), Progress.CompressedChunksCount);
				}
				if (Progress.CompressionDDCHitCount || Progress.CompressionDDCPutCount)
				{
					ProgressStringBuilder.Appendf(TEXT(" [DDC: %llu hits, %llu puts]"),
						Progress.CompressionDDCHitCount, Progress.CompressionDDCPutCount);
				}
				if (Progress.ScheduledCompressionTasksCount)
				{
					ProgressStringBuilder.Appendf(TEXT(" [%llu compression tasks]"), Progress.ScheduledCompressionTasksCount);
				}
				ProgressStringBuilder.Appendf(TEXT(" [Scheduled: %llu MiB]"), Progress.ScheduledCompressionMemoryBytes >> 20);
			}
			else
			{
				ProgressStringBuilder.Appendf(TEXT("Hashing %llu/%llu..."), Progress.HashedChunksCount, Progress.TotalChunksCount);
			}

			const FPlatformMemoryStats MemStats = FPlatformMemory::GetStats();
			ProgressStringBuilder.Appendf(TEXT(" [Physical: %llu MiB, Virtual: %llu MiB]"),
				MemStats.UsedPhysical >> 20, MemStats.UsedVirtual >> 20);

			UE_LOGF(LogIoStore, Display, "%ls", *ProgressStringBuilder);
		}

		// When staging base game, all container metadata is aggregated into a single global.umeta file
		const FString GlobalContainerMetaPath = FPaths::GetExtension(Arguments.GlobalContainerPath).IsEmpty()
			? Arguments.GlobalContainerPath + FIoContainerMetaHeader::FileExtension
			: FPaths::ChangeExtension(Arguments.GlobalContainerPath, FIoContainerMetaHeader::FileExtension);
		IoStoreWriterContext->SaveContainerMeta(GlobalContainerMetaPath);
	}

	if (GeneralIoWriterSettings.bCompressionEnableDDC)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForDDC);
		UE_LOGF(LogIoStore, Display, "");
		UE_LOGF(LogIoStore, Display, "Waiting for DDC...");
		GetDerivedDataCacheRef().WaitForQuiescence(true);

		FIoStoreWriterContext::FProgress Progress = IoStoreWriterContext->GetProgress();
		const uint64 TotalDDCAttempts = Progress.CompressionDDCHitCount + Progress.CompressionDDCMissCount;
		const double DDCHitRate = TotalDDCAttempts > 0 ? double(Progress.CompressionDDCHitCount) / TotalDDCAttempts * 100.0 : 0.0;
		const double DDCPutRate = TotalDDCAttempts > 0 ? double(Progress.CompressionDDCPutCount) / TotalDDCAttempts * 100.0 : 0.0;
		UE_LOGF(LogIoStore, Display, "Compression DDC hits: %llu/%llu (%.2lf%%)",
			Progress.CompressionDDCHitCount, TotalDDCAttempts, DDCHitRate);
		UE_LOGF(LogIoStore, Display, "Compression DDC puts: %llu/%llu (%.2lf%%) [%llu failed]",
			Progress.CompressionDDCPutCount, TotalDDCAttempts, DDCPutRate, Progress.CompressionDDCPutErrorCount);
		UE_LOGF(LogIoStore, Display, "");
	}

	{
		FIoStoreWriterContext::FProgress Progress = IoStoreWriterContext->GetProgress();
		if (Progress.HashDbChunksCount)
		{
			UE_LOGF(LogIoStore, Display, "%ls / %ls hashes were loaded from the hash database, by type:", *NumberString(Progress.HashDbChunksCount), *NumberString(Progress.TotalChunksCount));
		
			for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
			{
				if (Progress.HashDbChunksByType[i])
				{
					UE_LOGF(LogIoStore, Display, "    %-26ls %ls", *LexToString((EIoChunkType)i), *NumberString(Progress.HashDbChunksByType[i]));
				}
			}
		}
		if (Progress.RefDbChunksCount)
		{
			FIoStoreChunkDatabase& TypedChunkDatabase = ((FIoStoreChunkDatabase&)*ChunkDatabase);
			UE_LOGF(LogIoStore, Display, "%ls / %ls chunks for %ls bytes were loaded from the reference chunk database, by type:", *NumberString(Progress.RefDbChunksCount), *NumberString(Progress.TotalChunksCount), *NumberString(TypedChunkDatabase.FulfillBytes));

			for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
			{
				if (Progress.RefDbChunksByType[i])
				{
					UE_LOGF(LogIoStore, Display, "    %-26ls: %ls for %ls bytes", *LexToString((EIoChunkType)i), *NumberString(Progress.RefDbChunksByType[i]), *NumberString(TypedChunkDatabase.FulfillBytesPerChunk[i]));
				}
			}
		}
		if (Progress.CompressedChunksCount)
		{
			UE_LOGF(LogIoStore, Display, "%ls / %ls chunks attempted to compress, by type:", *NumberString(Progress.CompressedChunksCount), *NumberString(Progress.TotalChunksCount));

			for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
			{
				if (Progress.CompressedChunksByType[i] || Progress.BeginCompressChunksByType[i])
				{
					UE_LOGF(LogIoStore, Display, "    %-26ls %ls / %ls", *LexToString((EIoChunkType)i), *NumberString(Progress.CompressedChunksByType[i]), *NumberString(Progress.BeginCompressChunksByType[i]));
				}
			}
		}
		if (Progress.CompressionDDCHitCount)
		{
			UE_LOGF(LogIoStore, Display, "%ls / %ls chunks for %ls bytes were loaded from DDC, by type:",
				*NumberString(Progress.CompressionDDCHitCount),
				*NumberString(Progress.TotalChunksCount),
				*NumberString(Progress.CompressionDDCGetBytes));

			for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
			{
				if (Progress.CompressionDDCHitsByType[i])
				{
					UE_LOGF(LogIoStore, Display, "    %-26ls %ls / %ls", *LexToString((EIoChunkType)i),
						*NumberString(Progress.CompressionDDCHitsByType[i]),
						*NumberString(Progress.BeginCompressChunksByType[i]));
				}
			}
		}
		if (Progress.CompressionDDCPutCount)
		{
			UE_LOGF(LogIoStore, Display, "%ls / %ls chunks for %ls bytes were stored in DDC, by type:",
				*NumberString(Progress.CompressionDDCPutCount),
				*NumberString(Progress.TotalChunksCount),
				*NumberString(Progress.CompressionDDCPutBytes));

			for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
			{
				if (Progress.CompressionDDCPutsByType[i])
				{
					UE_LOGF(LogIoStore, Display, "    %-26ls %ls / %ls", *LexToString((EIoChunkType)i),
						*NumberString(Progress.CompressionDDCPutsByType[i]),
						*NumberString(Progress.BeginCompressChunksByType[i]));
				}
			}
		}
		UE_LOGF(LogIoStore, Display, "Source bytes read:");
		uint64 ZenTotalBytes = 0;
		for (uint64 b : WriteRequestManager.ZenSourceBytes)
		{
			ZenTotalBytes += b;
		}

		UE_LOGF(LogIoStore, Display, "    Zen: %34ls", *NumberString(ZenTotalBytes));
		for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
		{
			if (WriteRequestManager.ZenSourceReads[i])
			{
				UE_LOGF(LogIoStore, Display, "        %-22ls %12ls bytes over %ls reads", *LexToString((EIoChunkType)i), *NumberString(WriteRequestManager.ZenSourceBytes[i].Load()), *NumberString(WriteRequestManager.ZenSourceReads[i].Load()));
			}
		}

		uint64 LooseTotalBytes = 0;
		for (uint64 b : WriteRequestManager.LooseFileSourceBytes)
		{
			LooseTotalBytes += b;
		}
		UE_LOGF(LogIoStore, Display, "    Loose File: %27ls", *NumberString(LooseTotalBytes));
		for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
		{
			if (WriteRequestManager.LooseFileSourceReads[i])
			{
				UE_LOGF(LogIoStore, Display, "        %-22ls %12ls bytes over %ls reads", *LexToString((EIoChunkType)i), *NumberString(WriteRequestManager.LooseFileSourceBytes[i].Load()), *NumberString(WriteRequestManager.LooseFileSourceReads[i].Load()));
			}
		}

		uint64 MemoryTotalBytes = 0;
		for (uint64 b : WriteRequestManager.MemorySourceBytes)
		{
			MemoryTotalBytes += b;
		}
		UE_LOGF(LogIoStore, Display, "    Memory: %31ls", *NumberString(MemoryTotalBytes));
		for (uint8 i = 0; i < (uint8)EIoChunkType::MAX; i++)
		{
			if (WriteRequestManager.MemorySourceReads[i])
			{
				UE_LOGF(LogIoStore, Display, "        %-22ls %12ls bytes over %ls reads", *LexToString((EIoChunkType)i), *NumberString(WriteRequestManager.MemorySourceBytes[i].Load()), *NumberString(WriteRequestManager.MemorySourceReads[i].Load()));
			}
		}
	}

	if (Arguments.WriteBackMetadataToAssetRegistry != EAssetRegistryWritebackMethod::Disabled)
	{
		DoAssetRegistryWritebackDuringStage(
			Arguments.WriteBackMetadataToAssetRegistry, 
			Arguments.bWritePluginSizeSummaryJsons, 
			Arguments.PackageStore.Get(),
			Arguments.CookedDir, 
			GeneralIoWriterSettings.CompressionMethod != NAME_None, 
			IoStoreWriters, 
			IoStoreWriterInfos,
			ShaderAssocInfo);
	}

	TArray<FIoStoreWriterResult> IoStoreWriterResults;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetWriterResults);
		IoStoreWriterResults.Reserve(IoStoreWriters.Num());
		for (TSharedPtr<IIoStoreWriter> IoStoreWriter : IoStoreWriters)
		{
			IoStoreWriterResults.Emplace(IoStoreWriter->GetResult().ConsumeValueOrDie());
		}
	}

	FGraphEventRef WriteCsvFileTask;
	if (Arguments.CsvPath.Len() > 0)
	{
		WriteCsvFileTask = FFunctionGraphTask::CreateAndDispatchWhenReady([&Arguments, &IoStoreWriters, &IoStoreWriterResults, MaxPartitionSize = GeneralIoWriterSettings.MaxPartitionSize]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WriteCsvFiles);

			bool bPerContainerCsvFiles = FPaths::DirectoryExists(Arguments.CsvPath);
			FChunkEntryCsv AllContainersOutCsvFile;
			AllContainersOutCsvFile.SetPartitionSize(MaxPartitionSize);
			FChunkEntryCsv* Out = &AllContainersOutCsvFile;
			if (!bPerContainerCsvFiles)
			{
				// When CsvPath is a filename append .utoc.csv to create a unique single csv for all container files,
				// different from the unique single .pak.csv for all pak files.
				FString CsvFilename = Arguments.CsvPath + TEXT(".utoc.csv");
				AllContainersOutCsvFile.CreateOutputFile(*CsvFilename);
			}

			for (int32 Index = 0; Index < IoStoreWriters.Num(); ++Index)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ListContainer);

				TSharedPtr<IIoStoreWriter> Writer = IoStoreWriters[Index];
				FIoStoreWriterResult& Result = IoStoreWriterResults[Index];

				TArray<FIoStoreTocChunkInfo> Chunks;
				{
					IOSTORE_CPU_SCOPE(EnumerateChunks);
					Chunks.Reserve(Result.TocEntryCount);
					Writer->EnumerateChunks([&Chunks](FIoStoreTocChunkInfo&& ChunkInfo)
					{
						Chunks.Add(MoveTemp(ChunkInfo));
						return true;
					});
				}

				{
					IOSTORE_CPU_SCOPE(SortChunks);
					auto SortKey = [](const FIoStoreTocChunkInfo& ChunkInfo) { return ChunkInfo.OffsetOnDisk; };
					Algo::SortBy(Chunks, SortKey);
				}

				{
					IOSTORE_CPU_SCOPE(WriteCsvFile);
					FChunkEntryCsv PerContainerOutCsvFile;
					PerContainerOutCsvFile.SetPartitionSize(MaxPartitionSize);
					if (bPerContainerCsvFiles)
					{
						// When CsvPath is a dir, then create one unique .utoc.csv per container file
						FString PerContainerCsvPath = Arguments.CsvPath / Result.ContainerName + TEXT(".utoc.csv");
						PerContainerOutCsvFile.CreateOutputFile(*PerContainerCsvPath);
						Out = &PerContainerOutCsvFile;
					}
					for (int32 EntryIndex=0; EntryIndex < Chunks.Num(); ++EntryIndex)
					{
						FIoStoreTocChunkInfo& ChunkInfo = Chunks[EntryIndex];
						FString PackageName;
						FPackageId PackageId;
						if (!ChunkInfo.bHasValidFileName)
						{
							FString FileName = Arguments.PackageStore->GetRelativeFilenameFromChunkId(ChunkInfo.Id);
							if (FileName.Len() > 0)
							{
								ChunkInfo.FileName = MoveTemp(FileName);
							}
							FName PackageFName = Arguments.PackageStore->GetPackageNameFromChunkId(ChunkInfo.Id);
							if (!PackageFName.IsNone())
							{
								PackageName = PackageFName.ToString();
								PackageId = FPackageId::FromName(FName(*PackageName));
							}
						}

						Out->AddChunk(Result.ContainerName, EntryIndex, ChunkInfo, PackageId, PackageName, "");
					}
				}
			}
		}, TStatId(), nullptr, ENamedThreads::AnyNormalThreadHiPriTask);
	}

	IOSTORE_CPU_SCOPE(OutputStats);

	UE_LOGF(LogIoStore, Display, "Calculating stats...");
	uint64 UExpSize = 0;
	uint64 UAssetSize = 0;
	uint64 UBulkSize = 0;
	uint64 HeaderSize = 0;
	uint64 ImportedPackagesCount = 0;
	uint64 NoImportedPackagesCount = 0;
	uint64 NameMapCount = 0;
	
	for (const FCookedPackage* Package : Packages)
	{
		UExpSize += Package->UExpSize;
		UAssetSize += Package->UAssetSize;
		UBulkSize += Package->TotalBulkDataSize;
		if (bIsLegacyStage)
		{
			const FLegacyCookedPackage* LegacyPackage = static_cast<const FLegacyCookedPackage*>(Package);
			NameMapCount += LegacyPackage->OptimizedPackage->GetNameCount();
			HeaderSize += LegacyPackage->OptimizedPackage->GetHeaderSize();
		}
		int32 PackageImportedPackagesCount = Package->PackageStoreEntry.ImportedPackageIds.Num();
		ImportedPackagesCount += PackageImportedPackagesCount;
		NoImportedPackagesCount += PackageImportedPackagesCount == 0;
	}
	
	uint64 GlobalShaderCount = 0;
	uint64 SharedShaderCount = 0;
	uint64 UniqueShaderCount = 0;
	uint64 InlineShaderCount = 0;
	uint64 GlobalShaderSize = 0;
	uint64 SharedShaderSize = 0;
	uint64 UniqueShaderSize = 0;
	uint64 InlineShaderSize = 0;
	for (const FContainerTargetSpec* ContainerTarget : ContainerTargets)
	{
		for (const FShaderInfo* ShaderInfo : ContainerTarget->GlobalShaders)
		{
			++GlobalShaderCount;
			GlobalShaderSize += ShaderInfo->ShaderGroupChunk.CodeIoBuffer.DataSize();
		}
		for (const FShaderInfo* ShaderInfo : ContainerTarget->SharedShaders)
		{
			++SharedShaderCount;
			SharedShaderSize += ShaderInfo->ShaderGroupChunk.CodeIoBuffer.DataSize();
		}
		for (const FShaderInfo* ShaderInfo : ContainerTarget->UniqueShaders)
		{
			++UniqueShaderCount;
			UniqueShaderSize += ShaderInfo->ShaderGroupChunk.CodeIoBuffer.DataSize();
		}
		for (const FShaderInfo* ShaderInfo : ContainerTarget->InlineShaders)
		{
			++InlineShaderCount;
			InlineShaderSize += ShaderInfo->ShaderGroupChunk.CodeIoBuffer.DataSize();
		}
	}

	LogWriterResults(GeneralIoWriterSettings, IoStoreWriterResults, PerWriterDedupStats);
	LogContainerPackageInfo(ContainerTargets);

	UE_LOGF(LogIoStore, Display, "Input:  %8d Packages", Packages.Num());
	UE_LOGF(LogIoStore, Display, "Input:  %ls UExp",   *FormatMemoryString(UExpSize));
	UE_LOGF(LogIoStore, Display, "Input:  %ls UAsset", *FormatMemoryString(UAssetSize));
	UE_LOGF(LogIoStore, Display, "Input:  %ls UBulk",  *FormatMemoryString(UBulkSize));
	UE_LOGF(LogIoStore, Display, "Input:  %ls for %lld Global shaders", *FormatMemoryString(GlobalShaderSize), GlobalShaderCount);
	UE_LOGF(LogIoStore, Display, "Input:  %ls for %lld Shared shaders", *FormatMemoryString(SharedShaderSize), SharedShaderCount);
	UE_LOGF(LogIoStore, Display, "Input:  %ls for %lld Unique shaders", *FormatMemoryString(UniqueShaderSize), UniqueShaderCount);
	UE_LOGF(LogIoStore, Display, "Input:  %ls for %lld Inline shaders", *FormatMemoryString(InlineShaderSize), InlineShaderCount);
	UE_LOGF(LogIoStore, Display, "");

	if (bIsLegacyStage)
	{
		UE_LOGF(LogIoStore, Display, "Output: %8llu Name map entries", NameMapCount);
	}

	UE_LOGF(LogIoStore, Display, "Output: %8llu Imported package entries", ImportedPackagesCount);
	UE_LOGF(LogIoStore, Display, "Output: %8llu Packages without imports", NoImportedPackagesCount);
	UE_LOGF(LogIoStore, Display, "Output: %8lld Public runtime script objects", PackageStoreOptimizer.GetTotalScriptObjectCount());
	
	if (bIsLegacyStage)
	{
		UE_LOGF(LogIoStore, Display, "Output: %ls HeaderData", *FormatMemoryString(HeaderSize));
	}

	UE_LOGF(LogIoStore, Display, "Output: %ls InitialLoadData", *FormatMemoryString(InitialLoadSize));

	if (FStudioTelemetry::Get().IsSessionRunning() == true)
	{
		// Write out the summary to telemetry
		TArray<FAnalyticsEventAttribute> Attributes;

		const int SchemaVersion = 1;

		Attributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
		Attributes.Emplace(TEXT("PackageCount"), Packages.Num());
		Attributes.Emplace(TEXT("UExpSize"), UExpSize);
		Attributes.Emplace(TEXT("UUAssetSize"), UAssetSize);
		Attributes.Emplace(TEXT("UBulkSize"), UBulkSize);
		Attributes.Emplace(TEXT("InitialLoadSize"), InitialLoadSize);
		Attributes.Emplace(TEXT("GlobalShaderSize"), GlobalShaderSize);
		Attributes.Emplace(TEXT("SharedShaderSize"), SharedShaderSize);
		Attributes.Emplace(TEXT("UniqueShaderSize"), UniqueShaderSize);
		Attributes.Emplace(TEXT("ImportedPackagesCount"), ImportedPackagesCount);
		Attributes.Emplace(TEXT("NoImportedPackagesCount"), NoImportedPackagesCount);
		Attributes.Emplace(TEXT("ScriptCount"), PackageStoreOptimizer.GetTotalScriptObjectCount());

		if (bIsLegacyStage)
		{
			Attributes.Emplace(TEXT("HeaderSize"), HeaderSize);
			Attributes.Emplace(TEXT("NameMapCount"), NameMapCount);
		}

		FStudioTelemetry::Get().RecordEvent(TEXT("Core.IoStoreHeader.Summary"), Attributes);
	}
	
	if (ChunkDatabase.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReferenceChunkLogging);
		FIoStoreChunkDatabase* ChunkDbPtr = (FIoStoreChunkDatabase*)ChunkDatabase.Get();
		ChunkDbPtr->WriteCSV(Arguments.ReferenceChunkChangesCSVFileName, Packages);
		ChunkDbPtr->LogSummary(IoStoreWriterResults);	
	}

	UE_LOGF(LogIoStore, Display, "");
	if (Arguments.CsvPath.Len() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForCsvFiles);
		UE_LOGF(LogIoStore, Display, "Writing csv file(s) to: %ls (*.utoc.csv)", *Arguments.CsvPath);
		FTaskGraphInterface::Get().WaitUntilTaskCompletes(WriteCsvFileTask);
	}

	if (uint32 ErrorCount = IoStoreWriterContext->GetErrors())
	{
		// Change this to return 0 or re-run UAT stage run with "-pak -skippak" to
		// force produce a local broken build that will crash upon loading any of the reported failed assets.
		UE_LOGF(LogIoStore, Display, "Failure - %d error(s)", ErrorCount);
		return -2;
	}
	UE_LOGF(LogIoStore, Display, "Success");
	return 0;
}


void FIoStoreChunkDatabase::WriteCSV(const FString& InOutputFileName, const TArray<FCookedPackage*>& InPackages)
{
	if (!bValid ||
		InOutputFileName.Len() == 0)
	{
		return;
	}

	FString ChunkNames[FIoStoreChunkDatabase::IoChunkTypeCount];
	for (uint8 TypeIndex = 0; TypeIndex < FIoStoreChunkDatabase::IoChunkTypeCount; TypeIndex++)
	{
		ChunkNames[TypeIndex] = LexToString((EIoChunkType)TypeIndex);
	}

	//
	// If we are using a reference cache, the assumption is that we are expecting a high hit rate
	// on the chunks - this is supposed to be a current release, so theoretically we should only
	// miss on changed or new data, so it's useful to know where the misses are in case something
	// major happened that maybe shouldn't have.
	//

	TMap<FPackageId, FName> PackageIdToName;
	for (const FCookedPackage* Package : InPackages)
	{
		PackageIdToName.Add(Package->GlobalPackageId, Package->PackageName);
	}

	auto FindPackageForChunk = [&PackageIdToName](const FIoChunkId& InChunkId)
	{
		// See CreateIoChunkId and CreatePackageDataChunkId
		uint64 ChunkValue = *(uint64*)InChunkId.GetData();
		FName* Result = PackageIdToName.Find(FPackageId::FromValue(ChunkValue));
		if (Result)
		{
			return *Result;
		}
		return FName();
	};

	TUniquePtr<FArchive> ChangeListArchive;
	ChangeListArchive.Reset(IFileManager::Get().CreateFileWriter(*InOutputFileName));
	if (ChangeListArchive.IsValid() == false)
	{
		UE_LOGF(LogIoStore, Warning, "Unable to open reference chunk change list file %ls", *InOutputFileName);
		return;
	}

	ChangeListArchive->Logf(TEXT("Container,ChunkType,Class,ChunkId,ChunkHash,ChunkPackage"));

	for (const TPair<FIoContainerId, TUniquePtr<FReaderChunks>>& ReaderIdPair : ChunkDatabase)
	{
		FReaderChunks* Reader = ReaderIdPair.Value.Get();

		for (TPair<FIoChunkId, int32> Chunk : Reader->ChunkIds)
		{
			if (Reader->ChunkChanged[Chunk.Value].load(std::memory_order_relaxed))
			{
				FIoChunkId& Id = Chunk.Key;

				EIoChunkType ChangedType = Id.GetChunkType();

				if (ChangedType == EIoChunkType::ShaderCode ||
					ChangedType == EIoChunkType::ShaderCodeLibrary)
				{
					// These don't have the package name in the chunk id.
					ChangeListArchive->Logf(TEXT("%s,%s,CHANGED,%s,%s,<shader>"), *Reader->ContainerName, *ChunkNames[(uint8)ChangedType], *LexToString(Id), *LexToString(Reader->ChangedChunkHashes[Chunk.Value]));
				}
				else
				{
					ChangeListArchive->Logf(TEXT("%s,%s,CHANGED,%s,%s,%s"), *Reader->ContainerName, *ChunkNames[(uint8)ChangedType], *LexToString(Id), *LexToString(Reader->ChangedChunkHashes[Chunk.Value]), *FindPackageForChunk(Id).ToString());
				}
			}
		}

		for (FRequestedChunkInfo& NewChunk : Reader->NewChunks)
		{
			if (NewChunk.Id.GetChunkType() == EIoChunkType::ShaderCode ||
				NewChunk.Id.GetChunkType() == EIoChunkType::ShaderCodeLibrary)
			{
				// These don't have the package name in the chunk id.
				ChangeListArchive->Logf(TEXT("%s,%s,NEW,%s,%s,<shader>"), *Reader->ContainerName, *ChunkNames[(uint8)NewChunk.Id.GetChunkType()], *LexToString(NewChunk.Id), *LexToString(NewChunk.ChunkHash));
			}
			else
			{
				ChangeListArchive->Logf(TEXT("%s,%s,NEW,%s,%s,%s"), *Reader->ContainerName, *ChunkNames[(uint8)NewChunk.Id.GetChunkType()], *LexToString(NewChunk.Id), *LexToString(NewChunk.ChunkHash), *FindPackageForChunk(NewChunk.Id).ToString());
			}
		}
	}

	for (const TPair<FIoContainerId, TUniquePtr<FMissingContainerInfo>>& MissingContainerPair : MissingContainers)
	{
		FMissingContainerInfo* MissingContainer = MissingContainerPair.Value.Get();
		for (const FRequestedChunkInfo& NoContainerChunk : MissingContainer->Chunks)
		{
			const FString& ChunkName = ChunkNames[(uint8)NoContainerChunk.Id.GetChunkType()];

			if (NoContainerChunk.Id.GetChunkType() == EIoChunkType::ShaderCode ||
				NoContainerChunk.Id.GetChunkType() == EIoChunkType::ShaderCodeLibrary)
			{
				// These don't have the package name in the chunk id.
				ChangeListArchive->Logf(TEXT("%s,%s,NOCONTAINER,%s,%s,<shader>"),
					*MissingContainer->ContainerName, *ChunkName, *LexToString(NoContainerChunk.Id), *LexToString(NoContainerChunk.ChunkHash));
			}
			else
			{
				ChangeListArchive->Logf(TEXT("%s,%s,NOCONTAINER,%s,%s,%s"),
					*MissingContainer->ContainerName, *ChunkName, *LexToString(NoContainerChunk.Id), *LexToString(NoContainerChunk.ChunkHash), *FindPackageForChunk(NoContainerChunk.Id).ToString());
			}
		}
	}
}

bool DumpIoStoreContainerInfo(const TCHAR* InContainerFilename, const FKeyChain& InKeyChain)
{
	TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(InContainerFilename, InKeyChain);
	if (!Reader.IsValid())
	{
		return false;
	}

	UE_LOGF(LogIoStore, Display, "IoStore Container File: %ls", InContainerFilename);
	UE_LOGF(LogIoStore, Display, "    Id: 0x%llX", Reader->GetContainerId().Value());
	UE_LOGF(LogIoStore, Display, "    Version: %d", Reader->GetVersion());
	UE_LOGF(LogIoStore, Display, "    Indexed: %d", EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Indexed));
	UE_LOGF(LogIoStore, Display, "    Signed: %d", EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Signed));
	bool bIsEncrypted = EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Encrypted);
	UE_LOGF(LogIoStore, Display, "    Encrypted: %d", bIsEncrypted);
	if (bIsEncrypted)
	{
		UE_LOGF(LogIoStore, Display, "    EncryptionKeyGuid: %ls", *Reader->GetEncryptionKeyGuid().ToString());
	}
	bool bIsCompressed = EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Compressed);
	UE_LOGF(LogIoStore, Display, "    Compressed: %d", bIsCompressed);
	if (bIsCompressed)
	{
		UE_LOGF(LogIoStore, Display, "    CompressionBlockSize: %u", Reader->GetCompressionBlockSize());
		UE_LOGF(LogIoStore, Display, "    CompressionMethods:");
		for (FName Method : Reader->GetCompressionMethods())
		{
			UE_LOGF(LogPakFile, Display, "        %ls", *Method.ToString());
		}
	}

	return true;
}

int32 CreateContentPatch(const FIoStoreArguments& Arguments, const FIoStoreWriterSettings& GeneralIoWriterSettings)
{
	UE_LOGF(LogIoStore, Display, "Building patch...");
	TUniquePtr<FIoStoreWriterContext> IoStoreWriterContext(new FIoStoreWriterContext());
	FIoStatus IoStatus = IoStoreWriterContext->Initialize(GeneralIoWriterSettings);
	check(IoStatus.IsOk());
	TArray<TSharedPtr<IIoStoreWriter>> IoStoreWriters;
	for (const FContainerSourceSpec& Container : Arguments.Containers)
	{
		TArray<TUniquePtr<FIoStoreReader>> SourceReaders = CreatePatchSourceReaders(Container.PatchSourceContainerFiles, Arguments);
		TUniquePtr<FIoStoreReader> TargetReader = CreateIoStoreReader(*Container.PatchTargetFile, Arguments.KeyChain);
		if (!TargetReader.IsValid())
		{
			UE_LOGF(LogIoStore, Error, "Failed loading target container");
			return -1;
		}

		EIoContainerFlags TargetContainerFlags = TargetReader->GetContainerFlags();

		FIoContainerSettings ContainerSettings;
		if (Arguments.bCreateDirectoryIndex)
		{
			ContainerSettings.ContainerFlags |= EIoContainerFlags::Indexed;
		}

		ContainerSettings.ContainerId = TargetReader->GetContainerId();
		if (Arguments.bSign || EnumHasAnyFlags(TargetContainerFlags, EIoContainerFlags::Signed))
		{
			ContainerSettings.SigningKey =Arguments.KeyChain.GetSigningKey();
			ContainerSettings.ContainerFlags |= EIoContainerFlags::Signed;
		}

		if (EnumHasAnyFlags(TargetContainerFlags, EIoContainerFlags::Encrypted))
		{
			ContainerSettings.ContainerFlags |= EIoContainerFlags::Encrypted;
			const FNamedAESKey* Key = Arguments.KeyChain.GetEncryptionKeys().Find(TargetReader->GetEncryptionKeyGuid());
			if (!Key)
			{
				UE_LOGF(LogIoStore, Error, "Missing encryption key for target container");
				return -1;
			}
			ContainerSettings.EncryptionKeyGuid = Key->Guid;
			ContainerSettings.EncryptionKey = Key->Key;
		}

		TSharedPtr<IIoStoreWriter> IoStoreWriter = IoStoreWriterContext->CreateContainer(*Container.OutputPath, ContainerSettings);
		IoStoreWriters.Add(IoStoreWriter);
		TMap<FIoChunkId, FIoHash> SourceHashByChunkId;
		for (const TUniquePtr<FIoStoreReader>& SourceReader : SourceReaders)
		{
			SourceReader->EnumerateChunks([&SourceHashByChunkId](const FIoStoreTocChunkInfo& ChunkInfo)
			{
				SourceHashByChunkId.Add(ChunkInfo.Id, ChunkInfo.ChunkHash);
				return true;
			});
		}

		TMap<FIoChunkId, FString> ChunkFileNamesMap;
		TargetReader->GetDirectoryIndexReader().IterateDirectoryIndex(FIoDirectoryIndexHandle::RootDirectory(), TEXT(""),
		[&ChunkFileNamesMap, &TargetReader](FStringView Filename, uint32 TocEntryIndex) -> bool
		{
			TIoStatusOr<FIoStoreTocChunkInfo> ChunkInfo = TargetReader->GetChunkInfo(TocEntryIndex);
			if (ChunkInfo.IsOk())
			{
				ChunkFileNamesMap.Add(ChunkInfo.ValueOrDie().Id, FString(Filename));
			}
			return true;
		});

		TargetReader->EnumerateChunks([&TargetReader, &SourceHashByChunkId, &IoStoreWriter, &ChunkFileNamesMap](const FIoStoreTocChunkInfo& ChunkInfo)
		{
			FIoHash* FindSourceHash = SourceHashByChunkId.Find(ChunkInfo.Id);
			if (!FindSourceHash || *FindSourceHash != ChunkInfo.ChunkHash)
			{
				FIoReadOptions ReadOptions;
				TIoStatusOr<FIoBuffer> ChunkBuffer = TargetReader->Read(ChunkInfo.Id, ReadOptions);
				FIoWriteOptions WriteOptions;
				FString* FindFileName = ChunkFileNamesMap.Find(ChunkInfo.Id);
				if (FindFileName)
				{
					WriteOptions.FileName = *FindFileName;
					if (FindSourceHash)
					{
						UE_LOGF(LogIoStore, Display, "Modified: %ls", **FindFileName);
					}
					else
					{
						UE_LOGF(LogIoStore, Display, "Added: %ls", **FindFileName);
					}
				}
				WriteOptions.bIsMemoryMapped = ChunkInfo.bIsMemoryMapped;
				WriteOptions.bForceUncompressed = ChunkInfo.bForceUncompressed; 
				IoStoreWriter->Append(ChunkInfo.Id, ChunkBuffer.ConsumeValueOrDie(), WriteOptions);
			}
			return true;
		});
	}

	IoStoreWriterContext->FinalizeLayout();
	IoStoreWriterContext->FinalizeWrites();
	TArray<FIoStoreWriterResult> Results;
	for (TSharedPtr<IIoStoreWriter> IoStoreWriter : IoStoreWriters)
	{
		Results.Emplace(IoStoreWriter->GetResult().ConsumeValueOrDie());
	}

	LogWriterResults(GeneralIoWriterSettings, Results, TMap<FIoContainerId, FDetectDuplicatesStats>());

	return 0;
}

void Split(FStringView Line, FStringView::ViewType Sep, TArray<FStringView>& OutValues)
{
	OutValues.Reset();

	int32 Start = 0;
	int32 End = 0;
	for (;;)
	{
		End = Line.Find(TEXTVIEW(","), Start);
		if (End < 0)
		{
			break;
		}
		FStringView Value = Line.Mid(Start, End - Start);
		OutValues.Add(Value);
		Start = End + 1;
	}
	FStringView Value = Line.Mid(Start);
	OutValues.Add(Value);
}

static TArray<FString> GetContainersFromPathOrWildcard(const FString& InContainerPathOrWildcard)
{
	TArray<FString> ContainerFilePaths;
	if (IFileManager::Get().FileExists(*InContainerPathOrWildcard))
	{
		ContainerFilePaths.Add(InContainerPathOrWildcard);
	}
	else if (IFileManager::Get().DirectoryExists(*InContainerPathOrWildcard))
	{
		FString Directory = InContainerPathOrWildcard;
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		IFileManager::Get().FindFiles(FoundContainerFiles, *(Directory / TEXT("*.utoc")), true, false);

		for (const FString& Filename : FoundContainerFiles)
		{
			ContainerFilePaths.Emplace(Directory / Filename);
		}
	}
	else
	{
		FString Directory = FPaths::GetPath(InContainerPathOrWildcard);
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		IFileManager::Get().FindFiles(FoundContainerFiles, *InContainerPathOrWildcard, true, false);

		for (const FString& Filename : FoundContainerFiles)
		{
			ContainerFilePaths.Emplace(Directory / Filename);
		}
	}
	return MoveTemp(ContainerFilePaths);
}

static int32 FindAndExtractPackages(
	const FKeyChain& KeyChain,
	const FString& ContainerPathOrWildcard,
	const FString& OutputPath,
	const FString& PackageFilter)
{
	
	TArray<FString> ContainerFilePaths = GetContainersFromPathOrWildcard(ContainerPathOrWildcard);
	if (ContainerFilePaths.Num() == 0)
	{
		UE_LOGF(LogIoStore, Error, "Container '%ls' doesn't exist and no container matches wildcard.", *ContainerPathOrWildcard);
		return 1;
	}

	bool bAllSucceeded = true;
	for (const FString& ContainerFilePath : ContainerFilePaths)
	{
		bool bSucceeded = ExtractFilesFromIoStoreContainer(
			*ContainerFilePath, *OutputPath, KeyChain, &PackageFilter, nullptr, nullptr, nullptr);
		bAllSucceeded = bSucceeded && bAllSucceeded;
	}
	return bAllSucceeded ? 0 : 1;
}

void IterateContainerFilesInternal(const FKeyChain& KeyChain, const TMap<FPackageId, TPair<FName, FName>>& AllChunksInfoMap, const FString& ContainerFilePath, FChunkEntryCsv* Out)
{
	FString TocFileName = FPaths::ChangeExtension(ContainerFilePath, TEXT("utoc"));

	TMap<FGuid, FAES::FAESKey> DecryptionKeys;
	for (const auto& KV : KeyChain.GetEncryptionKeys())
	{
		DecryptionKeys.Add(KV.Key, KV.Value.Key);
	}

	TUniquePtr<IIoStoreTocReader> TocReader;
	{
		IOSTORE_CPU_SCOPE(ReadToc);
		TIoStatusOr<TUniquePtr<IIoStoreTocReader>> ReadResult = IIoStoreTocReader::ReadFromDisk(*TocFileName, DecryptionKeys);
		if (!ReadResult.IsOk())
		{
			UE_LOGF(LogIoStore, Warning, "Failed to read container toc '%ls' for container '%ls': %ls", *TocFileName, *ContainerFilePath, *ReadResult.Status().ToString());
			return;
		}

		TocReader = ReadResult.ConsumeValueOrDie();
	}

	Out->SetPartitionSize(TocReader->GetTocResource().Header.PartitionSize);
	if (!EnumHasAnyFlags(TocReader->GetTocResource().Header.ContainerFlags, EIoContainerFlags::Indexed))
	{
		UE_LOGF(LogIoStore, Display, "No directory index for container '%ls'", *ContainerFilePath);
	}

	UE_LOGF(LogIoStore, Display, "Listing container '%ls'", *ContainerFilePath);

	FString ContainerName = FPaths::GetBaseFilename(ContainerFilePath);
	
	TArray<FIoStoreTocChunkInfo> Chunks;
	{
		IOSTORE_CPU_SCOPE(EnumerateChunks);
		for (int32 ChunkIndex = 0; ChunkIndex < TocReader->GetTocResource().ChunkIds.Num(); ++ChunkIndex)
		{
			Chunks.Add(TocReader->GetTocChunkInfo(ChunkIndex));
		}
	}

	{
		IOSTORE_CPU_SCOPE(SortChunks);
		auto SortKey = [](const FIoStoreTocChunkInfo& ChunkInfo) { return ChunkInfo.OffsetOnDisk; };
		Algo::SortBy(Chunks, SortKey);
	}

	{
		IOSTORE_CPU_SCOPE(WriteCsvFile);
		FString PackageName;
		FString ClassType;
		for (int32 Index = 0; Index < Chunks.Num(); ++Index)
		{
			const FIoStoreTocChunkInfo& ChunkInfo = Chunks[Index];

			FPackageId PackageId = PackageIdFromChunkId(ChunkInfo.Id);
			PackageName.Reset();
			ClassType.Reset();
			if (ChunkInfo.bHasValidFileName && FPackageName::TryConvertFilenameToLongPackageName(ChunkInfo.FileName, PackageName, nullptr))
			{
				PackageId = FPackageId::FromName(FName(*PackageName));
			}

			const TPair<FName, FName>* PackageInfo = AllChunksInfoMap.Find(PackageId);
			if (PackageInfo)
			{
				PackageName = PackageInfo->Get<0>().ToString();
				ClassType = PackageInfo->Get<1>().ToString();
			}

			Out->AddChunk(ContainerName, Index, ChunkInfo, PackageId, PackageName, ClassType);
		}
	}
}

int32 ListContainer(
	const FKeyChain& KeyChain,
	const FString& ContainerPathOrWildcard,
	const FString& CsvPath)
{
	IOSTORE_CPU_SCOPE(ListContainer);	

	FString AllChunksInfoFilename;
	TMap<FPackageId, TPair<FName,FName>> AllChunksInfoMap;
	if (FParse::Value(FCommandLine::Get(), TEXT("AllChunksInfo="), AllChunksInfoFilename))
	{
		TArray<FStringView> Values;
		bool bSkipFirstLine = true;
		auto Visitor = [&AllChunksInfoMap,&Values,&bSkipFirstLine](FStringView Line)
		{
			const int32 PackageNameIndex = 1;
			const int32 ClassTypeIndex = 2;
			if (bSkipFirstLine)
			{
				bSkipFirstLine = false;
				return;
			}
			Split(Line, TEXTVIEW(","), Values);
			FName PackageName(Values[PackageNameIndex]);
			FPackageId PackageId = FPackageId::FromName(PackageName);
			AllChunksInfoMap.Add(PackageId, TPair<FName,FName>(PackageName, FName(Values[ClassTypeIndex])));
		};

		FFileHelper::LoadFileToStringWithLineVisitor(*AllChunksInfoFilename, Visitor);
	}

	TArray<FString> ContainerFilePaths = GetContainersFromPathOrWildcard(ContainerPathOrWildcard);
	if (ContainerFilePaths.Num() == 0)
	{
		UE_LOGF(LogIoStore, Error, "Container '%ls' doesn't exist and no container matches wildcard.", *ContainerPathOrWildcard);
		return -1;
	}
	
	// if CsvPath is a dir, not a file name, then write one csv per container to the dir
	// otherwise, write all contents to one big csv
	if (IFileManager::Get().DirectoryExists(*CsvPath))
	{
		ParallelFor(ContainerFilePaths.Num(), [&ContainerFilePaths, &AllChunksInfoMap, &KeyChain, &CsvPath](int32 Idx)
		{
			const FString& ContainerFilePath = ContainerFilePaths[Idx];

			FChunkEntryCsv PerContainerOutCsvFile;

			FString PerContainerCsvPath = CsvPath / FPaths::GetCleanFilename(ContainerFilePath) + TEXT(".csv");
			PerContainerOutCsvFile.CreateOutputFile(*PerContainerCsvPath);

			IterateContainerFilesInternal(KeyChain, AllChunksInfoMap, ContainerFilePath, &PerContainerOutCsvFile);
		});
	}
	else
	{
		FChunkEntryCsv AllContainersOutCsvFile;
		AllContainersOutCsvFile.CreateOutputFile(*CsvPath);
		for (const FString& ContainerFilePath : ContainerFilePaths)
		{			
			IterateContainerFilesInternal(KeyChain, AllChunksInfoMap, ContainerFilePath, &AllContainersOutCsvFile);
		}
		return 0;
	}

	return 0;
}

bool ListIoStoreContainer(const TCHAR* CmdLine)
{
	FKeyChain KeyChain;
	if (!LoadKeyChain(CmdLine, KeyChain))
	{
		return false;
	}

	FString ContainerPathOrWildcard;
	if (!FParse::Value(FCommandLine::Get(), TEXT("ListContainer="), ContainerPathOrWildcard))
	{
		UE_LOGF(LogIoStore, Error, "Missing argument -ListContainer=<ContainerFileOrWildCard>");
		return false;
	}

	FString CsvPath;
	if (!FParse::Value(FCommandLine::Get(), TEXT("csv="), CsvPath))
	{
		UE_LOGF(LogIoStore, Error, "Missing argument -Csv=<Path>");
		return false;
	}

	return ListContainer(KeyChain, ContainerPathOrWildcard, CsvPath) == 0;
}

bool ListContainerBulkData(
	const FKeyChain& KeyChain,
	const FString& ContainerPathOrWildcard,
	const FString& OutFile)
{
	struct FPackageData
	{
		FPackageId Id;
		FString Filename;
		TArray<FBulkDataMapEntry> BulkDataMap;
	};

	struct FContainerData
	{
		FString Name;
		FString Path;
		TArray<FPackageData> Packages;
	};

	TArray<FContainerData> Containers;
	TArray<FString> ContainerFilePaths = GetContainersFromPathOrWildcard(ContainerPathOrWildcard);

	if (ContainerFilePaths.Num() == 0)
	{
		UE_LOGF(LogIoStore, Error, "Container '%ls' doesn't exist and no container matches wildcard.", *ContainerPathOrWildcard);
		return false;
	}

	for (const FString& ContainerFilePath : ContainerFilePaths)
	{
		FString ContainerName = FPaths::GetBaseFilename(ContainerFilePath);
		if (ContainerName == TEXT("global"))
		{
			continue;
		}

		TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*ContainerFilePath, KeyChain);
		if (!Reader.IsValid())
		{
			UE_LOGF(LogIoStore, Error, "Failed to read container '%ls'", *ContainerFilePath);
			continue;
		}

		TMap<FIoChunkId, FString> FilenameByChunkId;
		Reader->GetDirectoryIndexReader().IterateDirectoryIndex(FIoDirectoryIndexHandle::RootDirectory(), TEXT(""),
			[&FilenameByChunkId, &Reader](FStringView Filename, uint32 TocEntryIndex) -> bool
			{
				TIoStatusOr<FIoStoreTocChunkInfo> ChunkInfo = Reader->GetChunkInfo(TocEntryIndex);
				if (ChunkInfo.IsOk())
				{
					FilenameByChunkId.Add(ChunkInfo.ValueOrDie().Id, FString(Filename));
				}
				return true;
			});

		UE_LOGF(LogIoStore, Display, "Listing bulk data in container '%ls'", *ContainerFilePath);
		FIoChunkId ChunkId = CreateIoChunkId(Reader->GetContainerId().Value(), 0, EIoChunkType::ContainerHeader);
		TIoStatusOr<FIoBuffer> Status = Reader->Read(ChunkId, FIoReadOptions());

		if (!Status.IsOk())
		{
			UE_LOGF(LogIoStore, Display, "Failed to read container header '%ls', reason '%ls'",
				*ContainerFilePath, *Status.Status().ToString());
			continue;
		}

		FIoContainerHeader ContainerHeader;
		{
			FIoBuffer Chunk = Status.ValueOrDie();
			FMemoryReaderView Ar(MakeArrayView(Chunk.Data(), Chunk.GetSize()));
			Ar << ContainerHeader;
		}

		FContainerData& Container = Containers.AddDefaulted_GetRef();
		Container.Path = ContainerFilePath;
		Container.Name = MoveTemp(ContainerName);

		for (const FPackageId& PackageId : ContainerHeader.PackageIds)
		{
			ChunkId = CreatePackageDataChunkId(PackageId);
			Status = Reader->Read(ChunkId, FIoReadOptions());
			if (!Status.IsOk())
			{
				UE_LOGF(LogIoStore, Display, "Failed to package data");
				continue;
			}

			FIoBuffer Chunk = Status.ValueOrDie();
			FZenPackageHeader PkgHeader = FZenPackageHeader::MakeView(Chunk.GetView());
			FPackageData& Pkg = Container.Packages.AddDefaulted_GetRef();
			Pkg.Id = PackageId;
			Pkg.BulkDataMap = PkgHeader.BulkDataMap;
			if (FString* Filename = FilenameByChunkId.Find(ChunkId))
			{
				Pkg.Filename = *Filename;
			}
		}
	}

	const FString Ext = FPaths::GetExtension(OutFile);
	if (Ext == TEXT("json"))
	{
		using FWriter = TSharedPtr<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>>; 
		using FWriterFactory = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>;

		FString Json;
		FWriter Writer = FWriterFactory::Create(&Json);
		Writer->WriteArrayStart();
		
		TStringBuilder<512> Sb;
		for (const FContainerData& Container: Containers)
		{
			Writer->WriteObjectStart();
			Writer->WriteValue(TEXT("Container"), Container.Name);
			Writer->WriteArrayStart(TEXT("Packages"));
			for (const FPackageData& Pkg : Container.Packages)
			{
				if (Pkg.BulkDataMap.IsEmpty())
				{
					continue;
				}

				Writer->WriteObjectStart();
				Writer->WriteValue(TEXT("PackageId"), FString::Printf(TEXT("0x%llX"), Pkg.Id.Value()));
				Writer->WriteValue(TEXT("Filename"), Pkg.Filename);
				Writer->WriteArrayStart(TEXT("BulkData"));
				for (const FBulkDataMapEntry& Entry : Pkg.BulkDataMap)
				{
					Sb.Reset();
					LexToString(static_cast<EBulkDataFlags>(Entry.Flags), Sb);
					Writer->WriteObjectStart();
					Writer->WriteValue(TEXT("Offset"), Entry.SerialOffset);
					Writer->WriteValue(TEXT("Size"), Entry.SerialSize);
					Writer->WriteValue(TEXT("Flags"), Sb.ToString());
					Writer->WriteObjectEnd();
				}
				Writer->WriteArrayEnd();
				Writer->WriteObjectEnd();
			}
			Writer->WriteArrayEnd();
			Writer->WriteObjectEnd();
		}

		Writer->WriteArrayEnd();
		Writer->Close();

		UE_LOGF(LogIoStore, Display, "Saving '%ls'", *OutFile);
		if (!FFileHelper::SaveStringToFile(Json, *OutFile))
		{
			return false;
		}
	}
	else
	{
		TUniquePtr<FArchive> CsvAr(IFileManager::Get().CreateFileWriter(*OutFile));
		CsvAr->Logf(TEXT("Container,Filename,PackageId,Offset,Size,Flags"));

		TStringBuilder<512> Sb;
		for (const FContainerData& Container: Containers)
		{
			for (const FPackageData& Pkg : Container.Packages)
			{
				for (const FBulkDataMapEntry& Entry : Pkg.BulkDataMap)
				{
					Sb.Reset();
					LexToString(static_cast<EBulkDataFlags>(Entry.Flags), Sb);
					CsvAr->Logf(TEXT("%s,%s,0x%s,%lld,%lld,%s"),
						*Container.Name, *Pkg.Filename, *LexToString(Pkg.Id), Entry.SerialOffset, Entry.SerialSize, Sb.ToString());
				}
			}
		}

		UE_LOGF(LogIoStore, Display, "Saving '%ls'", *OutFile);
	}

	return true;
}

bool ListIoStoreContainerBulkData(const TCHAR* CmdLine)
{
	FKeyChain KeyChain;
	if (!LoadKeyChain(CmdLine, KeyChain))
	{
		return false;
	}

	FString ContainerPathOrWildcard;
	if (!FParse::Value(FCommandLine::Get(), TEXT("ListContainerBulkData="), ContainerPathOrWildcard))
	{
		UE_LOGF(LogIoStore, Error, "Missing argument -ListContainerBulkData=<ContainerFileOrWildCard>");
		return false;
	}

	FString OutFile;
	if (!FParse::Value(FCommandLine::Get(), TEXT("Out="), OutFile))
	{
		UE_LOGF(LogIoStore, Error, "Missing argument -Out=<Path.[json|csv]>");
		return false;
	}

	return ListContainerBulkData(KeyChain, ContainerPathOrWildcard, OutFile) == 0;
}

bool ListIoStoreContainerMetadataFile(const FString& ContainerPathOrWildcard, const FString& OutFile)
{
	TArray<FString> ContainerFilePaths = GetContainersFromPathOrWildcard(ContainerPathOrWildcard);

	if (ContainerFilePaths.Num() == 0)
	{
		UE_LOGF(LogIoStore, Error, "Container '%ls' doesn't exist and no container matches wildcard.", *ContainerPathOrWildcard);
		return false;
	}

	TUniquePtr<FArchive> CsvAr;
	if (OutFile.IsEmpty() == false)
	{
		CsvAr.Reset(IFileManager::Get().CreateFileWriter(*OutFile));
		if (CsvAr.IsValid())
		{
			CsvAr->Logf(TEXT("ChunkId,ContainerName,Filename"));
		}
		else
		{
			UE_LOGF(LogIoStore, Error, "Failed to create CSV '%ls'", *OutFile);
			return false;
		}
	}

	for (const FString& ContainerFilePath : ContainerFilePaths)
	{
		FString MetaPath = FPaths::ChangeExtension(ContainerFilePath, FIoContainerMetaHeader::FileExtension);
		TIoStatusOr<FIoContainerMetaReader> MaybeReader = FIoContainerMetaReader::Load(MetaPath);
		if (MaybeReader.IsOk() == false)
		{
			UE_LOGF(LogIoStore, Error, "Failed load metadata file '%ls', reason: %ls", *MetaPath, *MaybeReader.Status().ToString());
			continue;
		}

		FIoContainerMetaReader Reader = MaybeReader.ConsumeValueOrDie();
		Reader.Iterate([&CsvAr](const FIoChunkId& ChunkId, FUtf8StringView ContainerName, FUtf8StringView Filename)
		{
			if (CsvAr.IsValid())
			{
				CsvAr->Logf(TEXT("%s, %s, %s"), *LexToString(ChunkId), *FString(ContainerName), *FString(Filename));
			}
			else
			{
				UE_LOGF(LogIoStore, Display, "%ls, %ls, %ls", *LexToString(ChunkId), *FString(ContainerName), *FString(Filename));
			}

			return true;
		});
	}

	if (CsvAr.IsValid())
	{
		UE_LOGF(LogIoStore, Display, "Saving '%ls'", *OutFile);
		CsvAr.Reset();
	}

	return true;
}

bool ListIoStoreContainerMetadataFile(const TCHAR* CmdLine)
{
	FString ContainerPathOrWildcard;
	if (!FParse::Value(FCommandLine::Get(), TEXT("ListContainerMetadataFile="), ContainerPathOrWildcard))
	{
		UE_LOGF(LogIoStore, Error, "Missing argument -ListContainerMetadataFile=<ContainerFileOrWildCard>");
		return false;
	}

	FString OutFile;
	if (!FParse::Value(FCommandLine::Get(), TEXT("Out="), OutFile))
	{
		OutFile.Empty();
	}

	return ListIoStoreContainerMetadataFile(ContainerPathOrWildcard, OutFile);
}

bool LegacyListIoStoreContainer(
	const TCHAR* InContainerFilename,
	int64 InSizeFilter,
	const FString& InCSVFilename,
	const FKeyChain& InKeyChain)
{
	TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(InContainerFilename, InKeyChain);
	if (!Reader.IsValid())
	{
		return false;
	}

	if (!EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Indexed))
	{
		UE_LOGF(LogIoStore, Fatal, "Missing directory index for container '%ls'", InContainerFilename);
	}

	int32 FileCount = 0;
	int64 FileSize = 0;

	TArray<FString> CompressionMethodNames;
	for (const FName& CompressionMethodName : Reader->GetCompressionMethods())
	{
		CompressionMethodNames.Add(CompressionMethodName.ToString());
	}

	TArray<FIoStoreCompressedBlockInfo> CompressionBlocks;
	TArray<FIoStoreTocCompressedBlockInfo> CompressedBlocks;
	Reader->EnumerateCompressedBlocks([&CompressedBlocks](const FIoStoreTocCompressedBlockInfo& Block)
		{
			CompressedBlocks.Add(Block);
			return true;
		});

	const FIoDirectoryIndexReader& IndexReader = Reader->GetDirectoryIndexReader();
	UE_LOGF(LogIoStore, Display, "Mount point %ls", *IndexReader.GetMountPoint());

	struct FEntry
	{
		FIoChunkId ChunkId;
		FIoHash ChunkHash;
		FString FileName;
		int64 Offset;
		int64 Size;
		int32 CompressionMethodIndex;
	};
	TArray<FEntry> Entries;

	const uint64 CompressionBlockSize = Reader->GetCompressionBlockSize();
	Reader->EnumerateChunks([&Entries, CompressionBlockSize, &CompressedBlocks](const FIoStoreTocChunkInfo& ChunkInfo)
		{
			const int32 FirstBlockIndex = int32(ChunkInfo.Offset / CompressionBlockSize);
			
			FEntry& Entry = Entries.AddDefaulted_GetRef();
			Entry.ChunkId = ChunkInfo.Id;
			Entry.ChunkHash = ChunkInfo.ChunkHash;
			Entry.FileName = ChunkInfo.FileName;
			Entry.Offset = CompressedBlocks[FirstBlockIndex].Offset;
			Entry.Size = ChunkInfo.CompressedSize;
			Entry.CompressionMethodIndex = CompressedBlocks[FirstBlockIndex].CompressionMethodIndex;
			return true;
		});

	struct FOffsetSort
	{
		bool operator()(const FEntry& A, const FEntry& B) const
		{
			return A.Offset < B.Offset;
		}
	};
	Entries.Sort(FOffsetSort());

	FileCount = Entries.Num();

	if (InCSVFilename.Len() > 0)
	{
		TArray<FString> Lines;
		Lines.Empty(Entries.Num() + 2);
		Lines.Add(TEXT("Filename, Offset, Size, Hash, Deleted, Compressed, CompressionMethod"));
		for (const FEntry& Entry : Entries)
		{
			bool bWasCompressed = Entry.CompressionMethodIndex != 0;
			Lines.Add(FString::Printf(
				TEXT("%s, %lld, %lld, %s, %s, %s, %d"),
				*Entry.FileName,
				Entry.Offset,
				Entry.Size,
				*LexToString(Entry.ChunkHash),
				TEXT("false"),
				bWasCompressed ? TEXT("true") : TEXT("false"),
				Entry.CompressionMethodIndex));
		}

		if (FFileHelper::SaveStringArrayToFile(Lines, *InCSVFilename) == false)
		{
			UE_LOGF(LogIoStore, Display, "Failed to save CSV file %ls", *InCSVFilename);
		}
		else
		{
			UE_LOGF(LogIoStore, Display, "Saved CSV file to %ls", *InCSVFilename);
		}
	}

	for (const FEntry& Entry : Entries)
	{
		if (Entry.Size >= InSizeFilter)
		{
			UE_LOGF(LogIoStore, Display, "\"%ls\" offset: %lld, size: %lld bytes, hash: %ls, compression: %ls.",
				*Entry.FileName,
				Entry.Offset,
				Entry.Size,
				*LexToString(Entry.ChunkHash),
				*CompressionMethodNames[Entry.CompressionMethodIndex]);
		}
		FileSize += Entry.Size;
	}
	UE_LOGF(LogIoStore, Display, "%d files (%lld bytes).", FileCount, FileSize);

	return true;
}

int32 ProfileReadSpeed(const TCHAR* InCommandLine, const FKeyChain& InKeyChain)
{
	FString ContainerPath;
	if (FParse::Value(InCommandLine, TEXT("Container="), ContainerPath) == false)
	{
		UE_LOGF(LogIoStore, Display, "");
		UE_LOGF(LogIoStore, Display, "ProfileReadSpeed");
		UE_LOGF(LogIoStore, Display, "");
		UE_LOGF(LogIoStore, Display, "Reads the given utoc file using given a read method. This uses FIoStoreReader, which is not");
		UE_LOGF(LogIoStore, Display, "the system the runtime uses to load and stream iostore containers! It's for utility/debug use only.");
		UE_LOGF(LogIoStore, Display, "");
		UE_LOGF(LogIoStore, Display, "Arguments:");
		UE_LOGF(LogIoStore, Display, "");
		UE_LOGF(LogIoStore, Display, "    -Container=path/to/utoc                      [required] The .utoc file to read.");
		UE_LOGF(LogIoStore, Display, "    -ReadType={Read, ReadAsync, ReadCompressed}  What read function to use on FIoStoreReader. Default: Read");
		UE_LOGF(LogIoStore, Display, "    -cryptokeys=path/to/crypto.json              [required if encrypted] The keys to decrypt the container.");
		UE_LOGF(LogIoStore, Display, "    -MaxJobCount=#                               The number of outstanding read tasks to maintain. Default: 512.");
		UE_LOGF(LogIoStore, Display, "    -Validate                                    Whether to hash the reads and verify they match. Invalid for ReadCompressed. Default: disabled");
		return 1;
	}

	TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*ContainerPath, InKeyChain);
	if (Reader.IsValid() == false)
	{
		return 1; // Already logged
	}

	TArray<FIoChunkId> Chunks;
	Reader->EnumerateChunks([&Chunks](const FIoStoreTocChunkInfo& ChunkInfo)
	{
		Chunks.Add(ChunkInfo.Id);
		return true;
	});

	enum class EReadType
	{
		Read,
		ReadAsync,
		ReadCompressed
	};

	auto ReadTypeToString = [](EReadType InReadType)
	{
		switch (InReadType)
		{
		case EReadType::Read: return TEXT("Read");
		case EReadType::ReadAsync: return TEXT("ReadAsync");
		case EReadType::ReadCompressed: return TEXT("ReadCompressed");
		default: return TEXT("INVALID");
		}
	};

	EReadType ReadType = EReadType::Read;
	int32 MaxOutstandingJobs = 512;
	bool bValidate = false;

	bValidate = FParse::Param(InCommandLine, TEXT("Validate"));
	FParse::Value(InCommandLine, TEXT("MaxJobCount="), MaxOutstandingJobs);

	FString ReadTypeRaw;
	if (FParse::Value(InCommandLine, TEXT("ReadType="), ReadTypeRaw))
	{
		if (ReadTypeRaw.Compare(TEXT("Read"), ESearchCase::IgnoreCase) == 0)
		{
			ReadType = EReadType::Read;
		}
		else if (ReadTypeRaw.Compare(TEXT("ReadAsync"), ESearchCase::IgnoreCase) == 0)
		{
			ReadType = EReadType::ReadAsync;
		}
		else if (ReadTypeRaw.Compare(TEXT("ReadCompressed"), ESearchCase::IgnoreCase) == 0)
		{
			ReadType = EReadType::ReadCompressed;
		}
		else
		{
			UE_LOGF(LogIoStore, Error, "Invalid -ReadType provided: %ls. Valid are {Read, ReadAsync, ReadCompressed}", *ReadTypeRaw);
			return 1;
		}
	}

	if (MaxOutstandingJobs <= 0)
	{
		UE_LOGF(LogIoStore, Error, "Invalid -MaxJobCount provided: %d. Specify a positive integer", MaxOutstandingJobs);
		return 1;
	}

	if (ReadType == EReadType::ReadCompressed && 
		bValidate)
	{
		UE_LOGF(LogIoStore, Error, "Can't validate ReadCompressed as the data is not decompressed and thus can't be hashed");
		return 1;
	}

	UE_LOGF(LogIoStore, Display, "MaxJobCount:            %ls", *FText::AsNumber(MaxOutstandingJobs).ToString());
	UE_LOGF(LogIoStore, Display, "ReadType:               %ls", ReadTypeToString(ReadType));
	UE_LOGF(LogIoStore, Display, "Validation:             %ls", bValidate ? TEXT("Enabled") : TEXT("Disabled"));
	UE_LOGF(LogIoStore, Display, "Container Encrypted:    %ls", EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Encrypted) ? TEXT("Yes") : TEXT("No"));
	
	// We need a resettable event, so we can't use any of the task system events.
	FEvent* JustGotSpaceEvent = FPlatformProcess::GetSynchEventFromPool();
	UE::Tasks::FTaskEvent CompletedEvent(TEXT("ProfileReadDone"));

	std::atomic_int32_t OutstandingJobs = 0;
	std::atomic_int32_t TotalJobsRemaining = Chunks.Num();
	std::atomic_int64_t BytesRead = 0;

	double StartTime = FPlatformTime::Seconds();
	UE_LOGF(LogIoStore, Display, "Dispatching %ls chunk reads (%ls max at one time)", *FText::AsNumber(Chunks.Num()).ToString(), *FText::AsNumber(MaxOutstandingJobs).ToString());

	for (FIoChunkId& Id : Chunks)
	{
		for (;;)
		{
			int32 CurrentOutstanding = OutstandingJobs.load();
			if (CurrentOutstanding == MaxOutstandingJobs)
			{
				// Wait for one to complete.
				JustGotSpaceEvent->Wait();
				continue;
			}
			else if (CurrentOutstanding > MaxOutstandingJobs)
			{
				UE_LOGF(LogIoStore, Warning, "Synch error -- too many jobs oustanding %d", CurrentOutstanding);
			}
			break;
		}

		OutstandingJobs++;
		UE::Tasks::Launch(TEXT("IoStoreUtil::ReadJob"), [Id = Id, &OutstandingJobs, &JustGotSpaceEvent, &TotalJobsRemaining, &BytesRead, &Reader, &CompletedEvent, MaxOutstandingJobs, ReadType, bValidate]()
		{
			FIoHash ReadHash;
			bool bHashValid = false;

			switch (ReadType)
			{
			case EReadType::ReadCompressed:
				{
					FIoStoreCompressedReadResult Result = Reader->ReadCompressed(Id, FIoReadOptions()).ValueOrDie();
					BytesRead += Result.IoBuffer.GetSize();
					break;
				}
			case EReadType::Read:
				{
					FIoBuffer Result = Reader->Read(Id, FIoReadOptions()).ValueOrDie();
					BytesRead += Result.GetSize();

					if (bValidate)
					{
						ReadHash = FIoHash::HashBuffer(Result.GetData(), Result.GetSize());
						bHashValid = true;
					}

					break;
				}
			case EReadType::ReadAsync:
				{
					UE::Tasks::TTask<TIoStatusOr<FIoBuffer>> Tasks = Reader->ReadAsync(Id, FIoReadOptions());
					Tasks.Wait();
					FIoBuffer Result = Tasks.GetResult().ValueOrDie();

					BytesRead += Result.GetSize();

					if (bValidate)
					{
						ReadHash = FIoHash::HashBuffer(Result.GetData(), Result.GetSize());
						bHashValid = true;
					}
					break;
				}
			}

			if (bHashValid)
			{
				FIoHash CheckAgainstHash = Reader->GetChunkInfo(Id).ValueOrDie().ChunkHash;
				if (ReadHash != CheckAgainstHash)
				{
					UE_LOGF(LogIoStore, Warning, "Read hash mismatch: Chunk %ls", *LexToString(Id));
				}
			}

			if (OutstandingJobs.fetch_add(-1) == MaxOutstandingJobs)
			{
				// We are the first to make space in our limit, so release the dispatch thread to add more.
				JustGotSpaceEvent->Trigger();
			}

			int32 JobsRemaining = TotalJobsRemaining.fetch_add(-1);
			if ((JobsRemaining % 1000) == 1)
			{
				UE_LOGF(LogIoStore, Display, "Jobs Remaining: %d", JobsRemaining - 1);
			}

			// Were we the last job issued?
			if (JobsRemaining == 1)
			{
				CompletedEvent.Trigger();
			}
		});
	}

	{
		double WaitStartTime = FPlatformTime::Seconds();
		CompletedEvent.Wait();
		UE_LOGF(LogIoStore, Display, "Waited %.1f seconds", FPlatformTime::Seconds() - WaitStartTime);
	}

	FPlatformProcess::ReturnSynchEventToPool(JustGotSpaceEvent);

	double TotalTime = FPlatformTime::Seconds() - StartTime;

	int64 BytesPerSecond = int64(BytesRead.load() / TotalTime);

	UE_LOGF(LogIoStore, Display, "%ls bytes in %.1f seconds; %ls bytes per second", *FText::AsNumber((int64)BytesRead.load()).ToString(), TotalTime, *FText::AsNumber(BytesPerSecond).ToString());
	return 0;
}

namespace DescribeUtils
{
	struct FPackageDesc;

	struct FPackageRedirect
	{
		FPackageDesc* Source = nullptr;
		FPackageDesc* Target = nullptr;
	};

	struct FContainerDesc
	{
		FName Name;
		FIoContainerId Id;
		FGuid EncryptionKeyGuid;
		TArray<FPackageDesc*> LocalizedPackages;
		TArray<FPackageRedirect> PackageRedirects;
		bool bCompressed;
		bool bSigned;
		bool bEncrypted;
		bool bIndexed;
	};

	struct FPackageLocation
	{
		FContainerDesc* Container = nullptr;
		uint64 Offset = -1;
	};

	struct FExportDesc
	{
		FPackageDesc* Package = nullptr;
		FName Name;
		FSoftObjectPath FullName;
		uint64 PublicExportHash;
		FPackageObjectIndex OuterIndex;
		FPackageObjectIndex ClassIndex;
		FPackageObjectIndex SuperIndex;
		FPackageObjectIndex TemplateIndex;
		uint64 SerialOffset = 0;
		uint64 SerialSize = 0;
		FSHAHash ExportHash;
	};

	struct FExportBundleEntryDesc
	{
		FExportBundleEntry::EExportCommandType CommandType = FExportBundleEntry::ExportCommandType_Count;
		int32 LocalExportIndex = -1;
		FExportDesc* Export = nullptr;
	};

	struct FImportDesc
	{
		FSoftObjectPath Name;
		FPackageObjectIndex GlobalImportIndex;
		FExportDesc* Export = nullptr;
	};

	struct FScriptObjectDesc
	{
		FName Name;
		FSoftObjectPath FullName;
		FPackageObjectIndex GlobalImportIndex;
		FPackageObjectIndex OuterIndex;
	};

	struct FPackageDesc
	{
		FPackageId PackageId;
		FName PackageName;
		uint32 PackageFlags = 0;
		int32 NameCount = -1;
		TArray<FPackageLocation, TInlineAllocator<1>> Locations;
		TArray<FPackageId> ImportedPackageIds;
		TArray<uint64> ImportedPublicExportHashes;
		TArray<FImportDesc> Imports;
		TArray<FExportDesc> Exports;
		TArray<FExportBundleEntryDesc> ExportBundleEntries;
	};

	// Info loaded about a set of containers for the purposes of dumping to text in Describe or exploring some other way for debugging 
	struct FContainerPackageInfo
	{
		TArray<FContainerDesc*> Containers;
		TArray<FPackageDesc*> Packages;
		TMap<FPackageObjectIndex, FScriptObjectDesc> ScriptObjectByGlobalIdMap;
		TMap<FPublicExportKey, FExportDesc*> ExportByKeyMap;

		FContainerPackageInfo() = default;
		FContainerPackageInfo(
			TArray<FContainerDesc*> InContainers,
			TArray<FPackageDesc*> InPackages,
			TMap<FPackageObjectIndex, FScriptObjectDesc> InScriptObjectByGlobalIdMap,
			TMap<FPublicExportKey, FExportDesc*> InExportByKeyMap)
			: Containers(MoveTemp(InContainers))
			, Packages(MoveTemp(InPackages))
			, ScriptObjectByGlobalIdMap(MoveTemp(InScriptObjectByGlobalIdMap))
			, ExportByKeyMap(MoveTemp(InExportByKeyMap))
		{}

		FContainerPackageInfo(const FContainerPackageInfo&) = delete;
		FContainerPackageInfo(FContainerPackageInfo&&) = default;
		FContainerPackageInfo& operator=(const FContainerPackageInfo&) = delete;
		FContainerPackageInfo& operator=(FContainerPackageInfo&&) = default;
		~FContainerPackageInfo()
		{
			for (FPackageDesc* PackageDesc : Packages)
			{
				delete PackageDesc;
			}
			for (FContainerDesc* ContainerDesc : Containers)
			{
				delete ContainerDesc;
			}
		}

		FString PackageObjectIndexToString(const FPackageDesc* Package, const FPackageObjectIndex& PackageObjectIndex, bool bIncludeName)
		{
			if (PackageObjectIndex.IsNull())
			{
				return TEXT("<null>");
			}
			else if (PackageObjectIndex.IsPackageImport())
			{
				FPublicExportKey Key = FPublicExportKey::FromPackageImport(PackageObjectIndex, Package->ImportedPackageIds, Package->ImportedPublicExportHashes);
				FExportDesc* ExportDesc = ExportByKeyMap.FindRef(Key);
				if (ExportDesc && bIncludeName)
				{
					return FString::Printf(TEXT("0x%" UINT64_X_FMT " '%s'"), PackageObjectIndex.Value(), *ExportDesc->FullName.ToString());
				}
				else
				{
					return FString::Printf(TEXT("0x%" UINT64_X_FMT), PackageObjectIndex.Value());
				}
			}
			else if (PackageObjectIndex.IsScriptImport())
			{
				const FScriptObjectDesc* ScriptObjectDesc = ScriptObjectByGlobalIdMap.Find(PackageObjectIndex);
				if (ScriptObjectDesc && bIncludeName)
				{
					return FString::Printf(TEXT("0x%" UINT64_X_FMT " '%s'"), PackageObjectIndex.Value(), *ScriptObjectDesc->FullName.ToString());
				}
				else
				{
					return FString::Printf(TEXT("0x%" UINT64_X_FMT), PackageObjectIndex.Value());
				}
			}
			else if (PackageObjectIndex.IsExport())
			{
				return FString::Printf(TEXT("%" UINT64_X_FMT), PackageObjectIndex.Value());
			}
			else
			{
				return FString::Printf(TEXT("0x%" UINT64_X_FMT), PackageObjectIndex.Value());
			}
		}
	};

	// Try and read all packages inside the containers and the links between them for debugging/analysis
	TOptional<FContainerPackageInfo> TryGetContainerPackageInfo(
		const FString& GlobalContainerPath,
		const FKeyChain& KeyChain,
		bool bIncludeExportHashes
	)
	{
		if (!IFileManager::Get().FileExists(*GlobalContainerPath))
		{
			UE_LOGF(LogIoStore, Error, "Global container '%ls' doesn't exist.", *GlobalContainerPath);
			return {};
		}

		TUniquePtr<FIoStoreReader> GlobalReader = CreateIoStoreReader(*GlobalContainerPath, KeyChain);
		if (!GlobalReader.IsValid())
		{
			UE_LOGF(LogIoStore, Warning, "Failed reading global container '%ls'", *GlobalContainerPath);
			return {};
		}

		UE_LOGF(LogIoStore, Display, "Loading script imports...");

		TIoStatusOr<FIoBuffer> ScriptObjectsBuffer = GlobalReader->Read(CreateIoChunkId(0, 0, EIoChunkType::ScriptObjects), FIoReadOptions());
		if (!ScriptObjectsBuffer.IsOk())
		{
			UE_LOGF(LogIoStore, Warning, "Failed reading initial load meta chunk from global container '%ls'", *GlobalContainerPath);
			return {};
		}

		TMap<FPackageObjectIndex, FScriptObjectDesc> ScriptObjectByGlobalIdMap;
		FLargeMemoryReader ScriptObjectsArchive(ScriptObjectsBuffer.ValueOrDie().Data(), ScriptObjectsBuffer.ValueOrDie().DataSize());
		TArray<FDisplayNameEntryId> GlobalNameMap = LoadNameBatch(ScriptObjectsArchive);
		int32 NumScriptObjects = 0;
		ScriptObjectsArchive << NumScriptObjects;
		const FScriptObjectEntry* ScriptObjectEntries = reinterpret_cast<const FScriptObjectEntry*>(ScriptObjectsBuffer.ValueOrDie().Data() + ScriptObjectsArchive.Tell());
		for (int32 ScriptObjectIndex = 0; ScriptObjectIndex < NumScriptObjects; ++ScriptObjectIndex)
		{
			const FScriptObjectEntry& ScriptObjectEntry = ScriptObjectEntries[ScriptObjectIndex];
			FMappedName MappedName = ScriptObjectEntry.Mapped;
			check(MappedName.IsGlobal());
			FScriptObjectDesc& ScriptObjectDesc = ScriptObjectByGlobalIdMap.Add(ScriptObjectEntry.GlobalIndex);
			ScriptObjectDesc.Name = GlobalNameMap[MappedName.GetIndex()].ToName(MappedName.GetNumber());
			ScriptObjectDesc.GlobalImportIndex = ScriptObjectEntry.GlobalIndex;
			ScriptObjectDesc.OuterIndex = ScriptObjectEntry.OuterIndex;
		}
		for (auto& KV : ScriptObjectByGlobalIdMap)
		{
			FScriptObjectDesc& ScriptObjectDesc = KV.Get<1>();
			if (ScriptObjectDesc.FullName.IsNull())
			{
				TArray<FScriptObjectDesc*> ScriptObjectStack;
				FScriptObjectDesc* Current = &ScriptObjectDesc;
				FString FullName;
				while (Current)
				{
					if (!Current->FullName.IsNull())
					{
						FullName = Current->FullName.ToString();
						break;
					}
					ScriptObjectStack.Push(Current);
					Current = ScriptObjectByGlobalIdMap.Find(Current->OuterIndex);
				}
				while (ScriptObjectStack.Num() > 0)
				{
					Current = ScriptObjectStack.Pop();
					FullName /= Current->Name.ToString();
					Current->FullName = FSoftObjectPath(FullName);
				}
			}
		}

		FString Directory = FPaths::GetPath(GlobalContainerPath);
		FPaths::NormalizeDirectoryName(Directory);

		TArray<FString> FoundContainerFiles;
		IFileManager::Get().FindFiles(FoundContainerFiles, *(Directory / TEXT("*.utoc")), true, false);
		TArray<FString> ContainerFilePaths;
		for (const FString& Filename : FoundContainerFiles)
		{
			ContainerFilePaths.Emplace(Directory / Filename);
		}

		UE_LOGF(LogIoStore, Display, "Loading containers...");

		TArray<TUniquePtr<FIoStoreReader>> Readers;

		struct FLoadContainerHeaderJob
		{
			FName ContainerName;
			FContainerDesc* ContainerDesc = nullptr;
			TArray<FPackageDesc*> Packages;
			FIoStoreReader* Reader = nullptr;
			TArray<FIoContainerHeaderLocalizedPackage> RawLocalizedPackages;
			TArray<FIoContainerHeaderPackageRedirect> RawPackageRedirects;
		};

		TArray<FLoadContainerHeaderJob> LoadContainerHeaderJobs;

		for (const FString& ContainerFilePath : ContainerFilePaths)
		{
			TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*ContainerFilePath, KeyChain);
			if (!Reader.IsValid())
			{
				UE_LOGF(LogIoStore, Warning, "Failed to read container '%ls'", *ContainerFilePath);
				continue;
			}

			FLoadContainerHeaderJob& LoadContainerHeaderJob = LoadContainerHeaderJobs.AddDefaulted_GetRef();
			LoadContainerHeaderJob.Reader = Reader.Get();
			LoadContainerHeaderJob.ContainerName = FName(FPaths::GetBaseFilename(ContainerFilePath));
			
			Readers.Emplace(MoveTemp(Reader));
		}
		
		TAtomic<int32> TotalPackageCount{ 0 };
		ParallelFor(LoadContainerHeaderJobs.Num(), [&LoadContainerHeaderJobs, &TotalPackageCount](int32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadContainerHeader);

			FLoadContainerHeaderJob& Job = LoadContainerHeaderJobs[Index];

			FContainerDesc* ContainerDesc = new FContainerDesc();
			ContainerDesc->Name = Job.ContainerName;
			ContainerDesc->Id = Job.Reader->GetContainerId();
			ContainerDesc->EncryptionKeyGuid = Job.Reader->GetEncryptionKeyGuid();
			EIoContainerFlags Flags = Job.Reader->GetContainerFlags();
			ContainerDesc->bCompressed = bool(Flags & EIoContainerFlags::Compressed);
			ContainerDesc->bEncrypted = bool(Flags & EIoContainerFlags::Encrypted);
			ContainerDesc->bSigned = bool(Flags & EIoContainerFlags::Signed);
			ContainerDesc->bIndexed = bool(Flags & EIoContainerFlags::Indexed);
			Job.ContainerDesc = ContainerDesc;

			TIoStatusOr<FIoBuffer> IoBuffer = Job.Reader->Read(CreateIoChunkId(Job.Reader->GetContainerId().Value(), 0, EIoChunkType::ContainerHeader), FIoReadOptions());
			if (IoBuffer.IsOk())
			{
				FMemoryReaderView Ar(MakeArrayView(IoBuffer.ValueOrDie().Data(), IoBuffer.ValueOrDie().DataSize()));
				FIoContainerHeader ContainerHeader;
				Ar << ContainerHeader;

				Job.RawLocalizedPackages = ContainerHeader.LocalizedPackages;
				Job.RawPackageRedirects = ContainerHeader.PackageRedirects;

				TArrayView<FFilePackageStoreEntry> StoreEntries(reinterpret_cast<FFilePackageStoreEntry*>(ContainerHeader.StoreEntries.GetData()), ContainerHeader.PackageIds.Num());

				int32 PackageIndex = 0;
				Job.Packages.Reserve(StoreEntries.Num());
				for (FFilePackageStoreEntry& ContainerEntry : StoreEntries)
				{
					const FPackageId& PackageId = ContainerHeader.PackageIds[PackageIndex++];
					FPackageDesc* PackageDesc = new FPackageDesc();
					PackageDesc->PackageId = PackageId;
					PackageDesc->ImportedPackageIds = TArrayView<FPackageId>(ContainerEntry.ImportedPackages.Data(), ContainerEntry.ImportedPackages.Num());
					Job.Packages.Add(PackageDesc);
					++TotalPackageCount;
				}
			}
		}, EParallelForFlags::Unbalanced);

		struct FLoadPackageSummaryJob
		{
			FPackageDesc* PackageDesc = nullptr;
			FIoChunkId ChunkId;
			TArray<FLoadContainerHeaderJob*, TInlineAllocator<1>> Containers;
		};

		TArray<FLoadPackageSummaryJob> LoadPackageSummaryJobs;

		TArray<FContainerDesc*> Containers;
		TArray<FPackageDesc*> Packages;
		TMap<FPackageId, FPackageDesc*> PackageByIdMap;
		TMap<FPackageId, FLoadPackageSummaryJob*> PackageJobByIdMap;
		Containers.Reserve(LoadContainerHeaderJobs.Num());
		Packages.Reserve(TotalPackageCount);
		PackageByIdMap.Reserve(TotalPackageCount);
		PackageJobByIdMap.Reserve(TotalPackageCount);
		LoadPackageSummaryJobs.Reserve(TotalPackageCount);
		for (FLoadContainerHeaderJob& LoadContainerHeaderJob : LoadContainerHeaderJobs)
		{
			Containers.Add(LoadContainerHeaderJob.ContainerDesc);
			for (FPackageDesc* PackageDesc : LoadContainerHeaderJob.Packages)
			{
				FLoadPackageSummaryJob*& UniquePackageJob = PackageJobByIdMap.FindOrAdd(PackageDesc->PackageId);
				if (!UniquePackageJob)
				{
					Packages.Add(PackageDesc);
					PackageByIdMap.Add(PackageDesc->PackageId, PackageDesc);
					FLoadPackageSummaryJob& LoadPackageSummaryJob = LoadPackageSummaryJobs.AddDefaulted_GetRef();
					LoadPackageSummaryJob.PackageDesc = PackageDesc;
					LoadPackageSummaryJob.ChunkId = CreateIoChunkId(PackageDesc->PackageId.Value(), 0, EIoChunkType::ExportBundleData);
					UniquePackageJob = &LoadPackageSummaryJob;
				}
				UniquePackageJob->Containers.Add(&LoadContainerHeaderJob);
			}
		}
		for (FLoadContainerHeaderJob& LoadContainerHeaderJob : LoadContainerHeaderJobs)
		{
			for (const auto& RedirectPair : LoadContainerHeaderJob.RawPackageRedirects)
			{
				FPackageRedirect& PackageRedirect = LoadContainerHeaderJob.ContainerDesc->PackageRedirects.AddDefaulted_GetRef();
				PackageRedirect.Source = PackageByIdMap.FindRef(RedirectPair.SourcePackageId);
				PackageRedirect.Target = PackageByIdMap.FindRef(RedirectPair.TargetPackageId);
			}
			for (const auto& LocalizedPackage : LoadContainerHeaderJob.RawLocalizedPackages)
			{
				LoadContainerHeaderJob.ContainerDesc->LocalizedPackages.Add(PackageByIdMap.FindRef(LocalizedPackage.SourcePackageId));
			}
		}
		
		ParallelFor(LoadPackageSummaryJobs.Num(), [&LoadPackageSummaryJobs, bIncludeExportHashes](int32 Index)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LoadPackageSummary);

			FLoadPackageSummaryJob& Job = LoadPackageSummaryJobs[Index];
			for (FLoadContainerHeaderJob* LoadContainerHeaderJob : Job.Containers)
			{
				TIoStatusOr<FIoStoreTocChunkInfo> ChunkInfo = LoadContainerHeaderJob->Reader->GetChunkInfo(Job.ChunkId);
				check(ChunkInfo.IsOk());
				FPackageLocation& Location = Job.PackageDesc->Locations.AddDefaulted_GetRef();
				Location.Container = LoadContainerHeaderJob->ContainerDesc;
				Location.Offset = ChunkInfo.ValueOrDie().Offset;
			}

			FIoStoreReader* Reader = Job.Containers[0]->Reader;
			FIoReadOptions ReadOptions;
			if (!bIncludeExportHashes)
			{
				ReadOptions.SetRange(0, 16 << 10);
			}
			TIoStatusOr<FIoBuffer> IoBuffer = Reader->Read(Job.ChunkId, ReadOptions);
			check(IoBuffer.IsOk());
			const uint8* PackageSummaryData = IoBuffer.ValueOrDie().Data();
			const FZenPackageSummary* PackageSummary = reinterpret_cast<const FZenPackageSummary*>(PackageSummaryData);
			if (PackageSummary->HeaderSize > IoBuffer.ValueOrDie().DataSize())
			{
				ReadOptions.SetRange(0, PackageSummary->HeaderSize);
				IoBuffer = Reader->Read(Job.ChunkId, ReadOptions);
				PackageSummaryData = IoBuffer.ValueOrDie().Data();
				PackageSummary = reinterpret_cast<const FZenPackageSummary*>(PackageSummaryData);
			}

			TArrayView<const uint8> HeaderDataView(PackageSummaryData + sizeof(FZenPackageSummary), PackageSummary->HeaderSize - sizeof(FZenPackageSummary));
			FMemoryReaderView HeaderDataReader(HeaderDataView);

			FZenPackageVersioningInfo VersioningInfo;
			if (PackageSummary->bHasVersioningInfo)
			{
				HeaderDataReader << VersioningInfo;
			}

			FZenPackageCellOffsets CellOffsets;
			if (!PackageSummary->bHasVersioningInfo || VersioningInfo.PackageVersion >= EUnrealEngineObjectUE5Version::VERSE_CELLS)
			{
				HeaderDataReader << CellOffsets.CellImportMapOffset;
				HeaderDataReader << CellOffsets.CellExportMapOffset;
			}
			else
			{
				CellOffsets.CellImportMapOffset = PackageSummary->ExportBundleEntriesOffset;
				CellOffsets.CellExportMapOffset = PackageSummary->ExportBundleEntriesOffset;
			}

			TArray<FDisplayNameEntryId> PackageNameMap;
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(LoadNameBatch);
				PackageNameMap = LoadNameBatch(HeaderDataReader);
			}

			Job.PackageDesc->PackageName = PackageNameMap[PackageSummary->Name.GetIndex()].ToName(PackageSummary->Name.GetNumber());
			Job.PackageDesc->PackageFlags = PackageSummary->PackageFlags;
			Job.PackageDesc->NameCount = PackageNameMap.Num();
			
			Job.PackageDesc->ImportedPublicExportHashes = MakeArrayView<const uint64>(reinterpret_cast<const uint64*>(PackageSummaryData + PackageSummary->ImportedPublicExportHashesOffset), (PackageSummary->ImportMapOffset - PackageSummary->ImportedPublicExportHashesOffset) / sizeof(uint64));

			const FPackageObjectIndex* ImportMap = reinterpret_cast<const FPackageObjectIndex*>(PackageSummaryData + PackageSummary->ImportMapOffset);
			Job.PackageDesc->Imports.SetNum((PackageSummary->ExportMapOffset - PackageSummary->ImportMapOffset) / sizeof(FPackageObjectIndex));
			for (int32 ImportIndex = 0; ImportIndex < Job.PackageDesc->Imports.Num(); ++ImportIndex)
			{
				FImportDesc& ImportDesc = Job.PackageDesc->Imports[ImportIndex];
				ImportDesc.GlobalImportIndex = ImportMap[ImportIndex];
			}

			const FExportMapEntry* ExportMap = reinterpret_cast<const FExportMapEntry*>(PackageSummaryData + PackageSummary->ExportMapOffset);
			Job.PackageDesc->Exports.SetNum((CellOffsets.CellImportMapOffset - PackageSummary->ExportMapOffset) / sizeof(FExportMapEntry));
			for (int32 ExportIndex = 0; ExportIndex < Job.PackageDesc->Exports.Num(); ++ExportIndex)
			{
				const FExportMapEntry& ExportMapEntry = ExportMap[ExportIndex];
				FExportDesc& ExportDesc = Job.PackageDesc->Exports[ExportIndex];
				ExportDesc.Package = Job.PackageDesc;
				ExportDesc.Name = PackageNameMap[ExportMapEntry.ObjectName.GetIndex()].ToName(ExportMapEntry.ObjectName.GetNumber());
				ExportDesc.OuterIndex = ExportMapEntry.OuterIndex;
				ExportDesc.ClassIndex = ExportMapEntry.ClassIndex;
				ExportDesc.SuperIndex = ExportMapEntry.SuperIndex;
				ExportDesc.TemplateIndex = ExportMapEntry.TemplateIndex;
				ExportDesc.PublicExportHash = ExportMapEntry.PublicExportHash;
				ExportDesc.SerialOffset = PackageSummary->HeaderSize + ExportMapEntry.CookedSerialOffset;
				ExportDesc.SerialSize = ExportMapEntry.CookedSerialSize;
			}

			const FExportBundleEntry* ExportBundleEntries = reinterpret_cast<const FExportBundleEntry*>(PackageSummaryData + PackageSummary->ExportBundleEntriesOffset);
			const FExportBundleEntry* BundleEntry = ExportBundleEntries;
			int32 ExportBundleEntriesCount = Job.PackageDesc->Exports.Num() * 2;
			const FExportBundleEntry* BundleEntryEnd = BundleEntry + ExportBundleEntriesCount;
			Job.PackageDesc->ExportBundleEntries.Reserve(ExportBundleEntriesCount);
			while (BundleEntry < BundleEntryEnd)
			{
				FExportBundleEntryDesc& EntryDesc = Job.PackageDesc->ExportBundleEntries.AddDefaulted_GetRef();
				EntryDesc.CommandType = FExportBundleEntry::EExportCommandType(BundleEntry->CommandType);
				EntryDesc.LocalExportIndex = BundleEntry->LocalExportIndex;
				EntryDesc.Export = &Job.PackageDesc->Exports[BundleEntry->LocalExportIndex];
				if (BundleEntry->CommandType == FExportBundleEntry::ExportCommandType_Serialize)
				{
					if (bIncludeExportHashes)
					{
						check(EntryDesc.Export->SerialOffset + EntryDesc.Export->SerialSize <= IoBuffer.ValueOrDie().DataSize());
						FSHA1::HashBuffer(IoBuffer.ValueOrDie().Data() + EntryDesc.Export->SerialOffset, EntryDesc.Export->SerialSize, EntryDesc.Export->ExportHash.Hash);
					}
				}
				++BundleEntry;
			}
		}, EParallelForFlags::Unbalanced);

		UE_LOGF(LogIoStore, Display, "Connecting imports and exports...");
		TMap<FPublicExportKey, FExportDesc*> ExportByKeyMap;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConnectImportsAndExports);

			for (FPackageDesc* PackageDesc : Packages)
			{
				for (FExportDesc& ExportDesc : PackageDesc->Exports)
				{
					if (ExportDesc.PublicExportHash)
					{
						FPublicExportKey Key = FPublicExportKey::MakeKey(PackageDesc->PackageId, ExportDesc.PublicExportHash);
						ExportByKeyMap.Add(Key, &ExportDesc);
					}
				}
			}

			ParallelFor(Packages.Num(), [&Packages](int32 Index)
			{
				FPackageDesc* PackageDesc = Packages[Index];
				for (FExportDesc& ExportDesc : PackageDesc->Exports)
				{
					if (ExportDesc.FullName.IsNull())
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(GenerateExportFullName);

						TArray<FExportDesc*> ExportStack;
						
						FExportDesc* Current = &ExportDesc;
						TStringBuilder<2048> FullNameBuilder;
						TCHAR NameBuffer[FName::StringBufferSize];
						for (;;)
						{
							if (!Current->FullName.IsNull())
							{
								// This can be optimized to avoid the temp string copy
								FullNameBuilder.Append(Current->FullName.ToString());
								break;
							}
							ExportStack.Push(Current);
							if (Current->OuterIndex.IsNull())
							{
								PackageDesc->PackageName.ToString(NameBuffer);
								FullNameBuilder.Append(NameBuffer);
								break;
							}
							Current = &PackageDesc->Exports[Current->OuterIndex.Value()];
						}
						while (ExportStack.Num() > 0)
						{
							Current = ExportStack.Pop(EAllowShrinking::No);
							FullNameBuilder.Append(TEXT("."));
							Current->Name.ToString(NameBuffer);
							FullNameBuilder.Append(NameBuffer);
							Current->FullName = FSoftObjectPath(FullNameBuilder);
						}
					}
				}
			}, EParallelForFlags::Unbalanced);

			for (FPackageDesc* PackageDesc : Packages)
			{
				for (FImportDesc& Import : PackageDesc->Imports)
				{
					if (!Import.GlobalImportIndex.IsNull())
					{
						if (Import.GlobalImportIndex.IsPackageImport())
						{
							FPublicExportKey Key = FPublicExportKey::FromPackageImport(Import.GlobalImportIndex, PackageDesc->ImportedPackageIds, PackageDesc->ImportedPublicExportHashes);
							Import.Export = ExportByKeyMap.FindRef(Key);
							if (!Import.Export)
							{
								UE_LOGF(LogIoStore, Warning, "Missing import: 0x%llX in package 0x%ls '%ls'",
									Import.GlobalImportIndex.Value(), *LexToString(PackageDesc->PackageId), *PackageDesc->PackageName.ToString());
							}
							else
							{
								Import.Name = Import.Export->FullName;
							}
						}
						else
						{
							FScriptObjectDesc* ScriptObjectDesc = ScriptObjectByGlobalIdMap.Find(Import.GlobalImportIndex);
							if (ScriptObjectDesc)
							{
								Import.Name = ScriptObjectDesc->FullName;
							}
							else
							{
								UE_LOGF(LogIoStore, Warning, "Missing Script Object for Import: 0x%llX in package 0x%ls '%ls'", Import.GlobalImportIndex.Value(), *LexToString(PackageDesc->PackageId), *PackageDesc->PackageName.ToString());
							}
						}
					}
				}
			}
		}

		return { 
			FContainerPackageInfo{
				MoveTemp(Containers),
				MoveTemp(Packages),
				MoveTemp(ScriptObjectByGlobalIdMap),
				MoveTemp(ExportByKeyMap),
			} 
		};
	}
}

int32 Describe(
	const FString& GlobalContainerPath,
	const FKeyChain& KeyChain,
	const FString& PackageFilter,
	const FString& OutPath,
	bool bIncludeExportHashes)
{
	using namespace DescribeUtils;
	TOptional<FContainerPackageInfo> MaybeInfo = DescribeUtils::TryGetContainerPackageInfo(GlobalContainerPath, KeyChain, bIncludeExportHashes);
	if (!MaybeInfo.IsSet())
	{
		return -1;
	}
	FContainerPackageInfo& Info = MaybeInfo.GetValue();

	const TArray<FContainerDesc*>& Containers = Info.Containers;
	const TArray<FPackageDesc*>& Packages = Info.Packages;
	const TMap<FPackageObjectIndex, FScriptObjectDesc>& ScriptObjectByGlobalIdMap = Info.ScriptObjectByGlobalIdMap;
	const TMap<FPublicExportKey, FExportDesc*>& ExportByKeyMap = Info.ExportByKeyMap;

	UE_LOGF(LogIoStore, Display, "Collecting output packages...");
	TArray<const FPackageDesc*> OutputPackages;
	TSet<FPackageId> RelevantPackages;
	TSet<FContainerDesc*> RelevantContainers;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CollectOutputPackages);

		if (PackageFilter.IsEmpty())
		{
			OutputPackages.Append(Packages);
		}
		else
		{
			TArray<FString> SplitPackageFilters;
			const TCHAR* Delimiters[] = { TEXT(","), TEXT(" ") };
			PackageFilter.ParseIntoArray(SplitPackageFilters, Delimiters, UE_ARRAY_COUNT(Delimiters), true);

			TArray<FString> PackageNameFilter;
			TSet<FPackageId> PackageIdFilter;
			for (const FString& PackageNameOrId : SplitPackageFilters)
			{
				if (PackageNameOrId.Len() > 0 && FChar::IsDigit(PackageNameOrId[0]))
				{
					uint64 Value;
					LexFromString(Value, *PackageNameOrId);
					PackageIdFilter.Add(*(FPackageId*)(&Value));
				}
				else
				{
					PackageNameFilter.Add(PackageNameOrId);
				}
			}

			TArray<const FPackageDesc*> PackageStack;
			for (const FPackageDesc* PackageDesc : Packages)
			{
				bool bInclude = false;
				if (PackageIdFilter.Contains(PackageDesc->PackageId))
				{
					bInclude = true;
				}
				else
				{
					FString PackageName = PackageDesc->PackageName.ToString();
					for (const FString& Wildcard : PackageNameFilter)
					{
						if (PackageName.MatchesWildcard(Wildcard))
						{
							bInclude = true;
							break;
						}
					}
				}
				if (bInclude)
				{
					PackageStack.Push(PackageDesc);
				}
			}
			TSet<const FPackageDesc*> Visited;
			while (PackageStack.Num() > 0)
			{
				const FPackageDesc* PackageDesc = PackageStack.Pop();
				if (!Visited.Contains(PackageDesc))
				{
					Visited.Add(PackageDesc);
					OutputPackages.Add(PackageDesc);
					RelevantPackages.Add(PackageDesc->PackageId);
					for (const FPackageLocation& Location : PackageDesc->Locations)
					{
						RelevantContainers.Add(Location.Container);
					}
					for (const FImportDesc& Import : PackageDesc->Imports)
					{
						if (Import.Export && Import.Export->Package)
						{
							PackageStack.Push(Import.Export->Package);
						}
					}
				}
			}
		}
	}

	UE_LOGF(LogIoStore, Display, "Generating report...");

	FOutputDevice* OutputOverride = GWarn;
	FString OutputFilename;
	TUniquePtr<FOutputDeviceFile> OutputBuffer;
	if (!OutPath.IsEmpty())
	{
		OutputBuffer = MakeUnique<FOutputDeviceFile>(*OutPath, true);
		OutputBuffer->SetSuppressEventTag(true);
		OutputOverride = OutputBuffer.Get();
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GenerateReport);
		TGuardValue GuardPrintLogTimes(GPrintLogTimes, ELogTimes::None);
		TGuardValue GuardPrintLogCategory(GPrintLogCategory, false);
		TGuardValue GuardPrintLogVerbosity(GPrintLogVerbosity, false);

		auto PackageObjectIndexToString = [&ScriptObjectByGlobalIdMap, &ExportByKeyMap](const FPackageDesc* Package, const FPackageObjectIndex& PackageObjectIndex, bool bIncludeName) -> FString
		{
			if (PackageObjectIndex.IsNull())
			{
				return TEXT("<null>");
			}
			else if (PackageObjectIndex.IsPackageImport())
			{
				FPublicExportKey Key = FPublicExportKey::FromPackageImport(PackageObjectIndex, Package->ImportedPackageIds, Package->ImportedPublicExportHashes);
				FExportDesc* ExportDesc = ExportByKeyMap.FindRef(Key);
				if (ExportDesc && bIncludeName)
				{
					return FString::Printf(TEXT("0x%llX '%s'"), PackageObjectIndex.Value(), *ExportDesc->FullName.ToString());
				}
				else
				{
					return FString::Printf(TEXT("0x%llX"), PackageObjectIndex.Value());
				}
			}
			else if (PackageObjectIndex.IsScriptImport())
			{
				const FScriptObjectDesc* ScriptObjectDesc = ScriptObjectByGlobalIdMap.Find(PackageObjectIndex);
				if (ScriptObjectDesc && bIncludeName)
				{
					return FString::Printf(TEXT("0x%llX '%s'"), PackageObjectIndex.Value(), *ScriptObjectDesc->FullName.ToString());
				}
				else
				{
					return FString::Printf(TEXT("0x%llX"), PackageObjectIndex.Value());
				}
			}
			else if (PackageObjectIndex.IsExport())
			{
				return FString::Printf(TEXT("%lld"), PackageObjectIndex.Value());
			}
			else
			{
				return FString::Printf(TEXT("0x%llX"), PackageObjectIndex.Value());
			}
		};

		for (const FContainerDesc* ContainerDesc : Containers)
		{
			if (RelevantContainers.Num() > 0 && !RelevantContainers.Contains(ContainerDesc))
			{
				continue;
			}

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("********************************************"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("Container '%s' Summary"), *ContainerDesc->Name.ToString());
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t      ContainerId: 0x%llX"), ContainerDesc->Id.Value());
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t       Compressed: %s"), ContainerDesc->bCompressed ? TEXT("Yes") : TEXT("No"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Signed: %s"), ContainerDesc->bSigned ? TEXT("Yes") : TEXT("No"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t          Indexed: %s"), ContainerDesc->bIndexed ? TEXT("Yes") : TEXT("No"));
			if (ContainerDesc->bEncrypted)
			{
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\tEncryptionKeyGuid: %s"), *ContainerDesc->EncryptionKeyGuid.ToString());
			}

			if (ContainerDesc->LocalizedPackages.Num())
			{
				bool bNeedsHeader = true;
				for (const FPackageDesc* LocalizedPackage : ContainerDesc->LocalizedPackages)
				{
					if (RelevantPackages.Num() > 0 && !RelevantPackages.Contains(LocalizedPackage->PackageId))
					{
						continue;
					}
					if (bNeedsHeader)
					{
						OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));
						OutputOverride->Logf(ELogVerbosity::Display, TEXT("Localized Packages"));
						OutputOverride->Logf(ELogVerbosity::Display, TEXT("=========="));
						bNeedsHeader = false;
					}
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t*************************"));
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Source: 0x%s '%s'"), *LexToString(LocalizedPackage->PackageId), *LocalizedPackage->PackageName.ToString());
				}
			}

			if (ContainerDesc->PackageRedirects.Num())
			{
				bool bNeedsHeader = true;
				for (const FPackageRedirect& Redirect : ContainerDesc->PackageRedirects)
				{
					if (RelevantPackages.Num() > 0 && !RelevantPackages.Contains(Redirect.Source->PackageId) && !RelevantPackages.Contains(Redirect.Target->PackageId))
					{
						continue;
					}
					if (bNeedsHeader)
					{
						OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));
						OutputOverride->Logf(ELogVerbosity::Display, TEXT("Package Redirects"));
						OutputOverride->Logf(ELogVerbosity::Display, TEXT("=========="));
						bNeedsHeader = false;
					}
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t*************************"));
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Source: 0x%s '%s'"), *LexToString(Redirect.Source->PackageId), *Redirect.Source->PackageName.ToString());
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Target: 0x%s '%s'"), *LexToString(Redirect.Target->PackageId), *Redirect.Target->PackageName.ToString());
				}
			}
		}

		for (const FPackageDesc* PackageDesc : OutputPackages)
		{
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("********************************************"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("Package '%s' Summary"), *PackageDesc->PackageName.ToString());
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t        PackageId: 0x%s"), *LexToString(PackageDesc->PackageId));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t     PackageFlags: %X"), PackageDesc->PackageFlags);
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t        NameCount: %d"), PackageDesc->NameCount);
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t      ImportCount: %d"), PackageDesc->Imports.Num());
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t      ExportCount: %d"), PackageDesc->Exports.Num());

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("Locations"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("=========="));
			int32 Index = 0;
			for (const FPackageLocation& Location : PackageDesc->Locations)
			{
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t*************************"));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\tLocation %d: '%s'"), Index++, *Location.Container->Name.ToString());
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Offset: %lld"), Location.Offset);
			}

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("Imports"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("=========="));
			Index = 0;
			for (const FImportDesc& Import : PackageDesc->Imports)
			{
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t*************************"));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\tImport %d: '%s'"), Index++, *Import.Name.ToString());
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\tGlobalImportIndex: %s"), *PackageObjectIndexToString(PackageDesc, Import.GlobalImportIndex, false));
			}

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("Exports"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("=========="));
			Index = 0;
			for (const FExportDesc& Export : PackageDesc->Exports)
			{
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t*************************"));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\tExport %d: '%s'"), Index++, *Export.Name.ToString());
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t       OuterIndex: %s"), *PackageObjectIndexToString(PackageDesc, Export.OuterIndex, true));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t       ClassIndex: %s"), *PackageObjectIndexToString(PackageDesc, Export.ClassIndex, true));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t       SuperIndex: %s"), *PackageObjectIndexToString(PackageDesc, Export.SuperIndex, true));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t    TemplateIndex: %s"), *PackageObjectIndexToString(PackageDesc, Export.TemplateIndex, true));
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t PublicExportHash: %llu"), Export.PublicExportHash);
				if (bIncludeExportHashes)
				{
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t   ExportHash: %s"), *Export.ExportHash.ToString());
				}
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Offset: %lld"), Export.SerialOffset);
				OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t             Size: %lld"), Export.SerialSize);

			}

			OutputOverride->Logf(ELogVerbosity::Display, TEXT("--------------------------------------------"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("Export Bundle"));
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("=========="));
			for (const FExportBundleEntryDesc& ExportBundleEntry : PackageDesc->ExportBundleEntries)
			{
				if (ExportBundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Create)
				{
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t           Create: %d '%s'"), ExportBundleEntry.LocalExportIndex, *ExportBundleEntry.Export->Name.ToString());
				}
				else
				{
					OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t\t        Serialize: %d '%s'"), ExportBundleEntry.LocalExportIndex, *ExportBundleEntry.Export->Name.ToString());
				}
			}
		}
	}


	return 0;
}

int32 ValidateCrossContainerRefs(
	const FString& GlobalContainerPath,
	const FKeyChain& KeyChain,
	const FString& ConfigPath,
	const FString& OutPath
	)
{
	TMultiMap<FString, FString> ValidEdges;
	TArray<FString> IgnoreRefsFromAssets, IgnoreRefsToAssets;

	FConfigFile ConfigFile;
	if (!FConfigCacheIni::LoadLocalIniFile(ConfigFile, *ConfigPath, false))
	{
		UE_LOGF(LogIoStore, Error, "Failed to load config file %ls", *ConfigPath);
		return -1;
	}

	if (const FConfigSection* EdgesSection = ConfigFile.FindSection(TEXT("Edges")))
	{
		for (auto It = EdgesSection->CreateConstIterator(); It; ++It)
		{
			ValidEdges.Add(It.Key().ToString(), It.Value().GetValue());
		}
	}
	if (const FConfigSection* DefaultEdgesSection = ConfigFile.FindSection(TEXT("DefaultEdges")))
	{
		for (auto It = DefaultEdgesSection->CreateConstIterator(); It; ++It)
		{
			ValidEdges.Add(FString(), It.Key().ToString());
		}
	}
	if (const FConfigSection* IgnoreSection = ConfigFile.FindSection(TEXT("Ignore")))
	{
		IgnoreSection->MultiFind(TEXT("IgnoreRefsFrom"), IgnoreRefsFromAssets);
		IgnoreSection->MultiFind(TEXT("IgnoreRefsTo"), IgnoreRefsToAssets);
	}

	if (ValidEdges.Num() == 0)
	{
		UE_LOGF(LogIoStore, Error, "No valid edges configured, nothing to validate");
		return -1;
	}

	if (ValidEdges.FindPair(TEXT(""), TEXT("")))
	{
		UE_LOGF(LogIoStore, Error, "Configuration contains all-to-all edge (empty string to empty string), nothing to validate");
		return -1;
	}

	using namespace DescribeUtils;
	TOptional<FContainerPackageInfo> MaybeInfo = DescribeUtils::TryGetContainerPackageInfo(GlobalContainerPath, KeyChain, false);
	if (!MaybeInfo.IsSet())
	{
		return -1;
	}
	FContainerPackageInfo& Info = MaybeInfo.GetValue();

	const TArray<FContainerDesc*>& Containers = Info.Containers;
	TArray<FPackageDesc*>& Packages = Info.Packages;
	const TMap<FPackageObjectIndex, FScriptObjectDesc>& ScriptObjectByGlobalIdMap = Info.ScriptObjectByGlobalIdMap;
	const TMap<FPublicExportKey, FExportDesc*>& ExportByKeyMap = Info.ExportByKeyMap;

	// Expand container prefixes from config into full container names and produce transitive closure
	TMultiMap<const FContainerDesc*, const FContainerDesc*> FinalValidEdges;
	{
		TMultiMap<FString, const FContainerDesc* > ShortNameToContainer;
		for (auto It = ValidEdges.CreateIterator(); It; ++It)
		{
			for (const FContainerDesc* Desc : Containers)
			{
				// Empty strings mean 'all containers'
				if (It.Key().Len() == 0 || Desc->Name.ToString().StartsWith(It.Key()))
				{
					ShortNameToContainer.AddUnique(It.Key(), Desc);
				}
				if (It.Value().Len() == 0 || Desc->Name.ToString().StartsWith(It.Value()))
				{
					ShortNameToContainer.AddUnique(It.Value(), Desc);
				}
			}
		}
		TMultiMap<const FContainerDesc*, const FContainerDesc*> ValidDirectEdges;
		for (const TPair<FString, FString>& Pair : ValidEdges)
		{
			if (Pair.Key.Len() == 0) 
			{ 
				// Do 'all containers' later after handling explicit containers 
				continue;
			}
			for (auto FromIt = ShortNameToContainer.CreateKeyIterator(Pair.Key); FromIt; ++FromIt)
			{
				for (auto ToIt = ShortNameToContainer.CreateKeyIterator(Pair.Value); ToIt; ++ToIt)
				{
					ValidDirectEdges.AddUnique(FromIt.Value(), ToIt.Value());
				}
			}
		}
		
		TSet<const FContainerDesc*> UnassignedFromContainers;
		for (const FContainerDesc* Container : Containers) 
		{
			if (!ValidDirectEdges.Contains(Container)) 
			{
				UnassignedFromContainers.Add(Container);
			}
		}
		
		for (const TPair<FString, FString>& Pair : ValidEdges)
		{
			if (Pair.Key.Len() == 0) 
			{ 
				for (auto FromIt = ShortNameToContainer.CreateKeyIterator(Pair.Key); FromIt; ++FromIt)
				{
					if (!UnassignedFromContainers.Contains(FromIt.Value()))
					{
						continue;
					}

					for (auto ToIt = ShortNameToContainer.CreateKeyIterator(Pair.Value); ToIt; ++ToIt)
					{
						ValidDirectEdges.Add(FromIt.Value(), ToIt.Value());
					}
				}
			}
		}

		// Create a transitive closure (i.e. if it is valid for packages in container A to import packages in B, and B to import packages in C, then it is valid for packages in A to import packages in C)
		for (const FContainerDesc* StartContainer : Containers)
		{
			TSet<const FContainerDesc*> SeenContainers;
			TArray<const FContainerDesc*> Queue;
			Queue.Add(StartContainer);
			SeenContainers.Add(StartContainer);
			while (Queue.Num() > 0)
			{
				const FContainerDesc* ToContainer = Queue.Pop();
				FinalValidEdges.Add(StartContainer, ToContainer);
				for (auto ToIt = ValidDirectEdges.CreateKeyIterator(ToContainer); ToIt; ++ToIt)
				{
					if (!SeenContainers.Contains(ToIt.Value()))
					{
						SeenContainers.Add(ToIt.Value());
						Queue.Add(ToIt.Value());
					}
				}
			}
		}
	}

	TMap<const FContainerDesc*, TSet<TTuple<const FPackageDesc*, const FPackageDesc*>>> Errors;
	Algo::SortBy(Packages, [](FPackageDesc* Desc) { return Desc->PackageName; }, FNameLexicalLess());
	for (const FPackageDesc* Package : Packages)
	{
		bool bSkip = Algo::AnyOf(IgnoreRefsFromAssets, [Package](const FString& IgnoreString)
		{
			FString PackageNameString = Package->PackageName.ToString();
			if (PackageNameString == IgnoreString)
			{
				return true;
			}
			else if (FWildcardString::ContainsWildcards(*IgnoreString) && FWildcardString::IsMatch(*IgnoreString, *PackageNameString))
			{
				return true;
			}
			return false;
		});
		if (bSkip)
		{
			continue;
		}

		bool bNeedsHeader = true;
		for (const FImportDesc& Import : Package->Imports)
		{
			FPackageDesc* ImportPackage = Import.Export ? Import.Export->Package : nullptr;
			if (!ImportPackage) 
			{
				UE_CLOGF(Import.Name.IsValid() && !FPackageName::IsScriptPackage(*Import.Name.ToString()),
					LogIoStore, Error, "Unresolved import of package %ls by package %ls", *Import.Name.ToString(), *Package->PackageName.ToString());
				continue;
			}

			bSkip = Algo::AnyOf(IgnoreRefsFromAssets, [ImportPackage](const FString& IgnoreString)
			{
				FString PackageNameString = ImportPackage->PackageName.ToString();
				if (PackageNameString == IgnoreString)
				{
					return true;
				}
				else if (FWildcardString::ContainsWildcards(*IgnoreString) && FWildcardString::IsMatch(*IgnoreString, *PackageNameString))
				{
					return true;
				}
				return false;
			});
			if (bSkip)
			{
				continue;
			}

			// For each location of the importing paacka
			for (const DescribeUtils::FPackageLocation& Location : Package->Locations)
			{
				bool bValid = Algo::AnyOf(ImportPackage->Locations, [&](const DescribeUtils::FPackageLocation& ImportLocation) {
					return FinalValidEdges.FindPair(Location.Container, ImportLocation.Container) != nullptr;
				});
				if (!bValid)
				{
					Errors.FindOrAdd(Location.Container).Add({ Package, ImportPackage });
				}
			}
		}
	}
	FOutputDevice* OutputOverride = GWarn;
	FString OutputFilename;
	TUniquePtr<FOutputDeviceFile> OutputBuffer;
	if (!OutPath.IsEmpty())
	{
		OutputBuffer = MakeUnique<FOutputDeviceFile>(*OutPath, true);
		OutputBuffer->SetSuppressEventTag(true);
		OutputOverride = OutputBuffer.Get();
	}

	OutputOverride->Logf(ELogVerbosity::Display, TEXT("Invalid cross-container reference report"));
	OutputOverride->Logf(ELogVerbosity::Display, TEXT("Final valid edges: %d"), FinalValidEdges.Num());
	for (const TPair<const FContainerDesc*, const FContainerDesc*>& Pair : FinalValidEdges)
	{
		OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t%s -> %s"), *Pair.Key->Name.ToString(), *Pair.Value->Name.ToString());
	}

	if (Errors.Num() == 0)
	{
		OutputOverride->Logf(ELogVerbosity::Display, TEXT("No errors."));
		return 0;
	}

	for (const TPair<const FContainerDesc*, TSet<TTuple<const FPackageDesc*, const FPackageDesc*>>>& Pair : Errors)
	{
		OutputOverride->Logf(ELogVerbosity::Display, TEXT("%s"), *Pair.Key->Name.ToString());
		for (const TTuple<const FPackageDesc*, const FPackageDesc*>& Error : Pair.Value) 
		{
			FString LocationsString = FString::JoinBy(Error.Value->Locations, TEXT(","), [](const DescribeUtils::FPackageLocation& Location) { return Location.Container->Name.ToString(); });
			OutputOverride->Logf(ELogVerbosity::Display, TEXT("\t%s -> %s (%s)"), *Error.Key->PackageName.ToString(), *Error.Value->PackageName.ToString(), *LocationsString);
		} 
	}
	return 0;
}

enum class EChunkTypeFilter
{
	None,
	PackageData,
	BulkData
};

FString LexToString(EChunkTypeFilter Filter)
{
	switch(Filter)
	{
		case EChunkTypeFilter::PackageData:
			return TEXT("PackageData");
		case EChunkTypeFilter::BulkData:
			return TEXT("BulkData");
		default:
			return TEXT("None");
	}
}

static int32 Diff(
	const FString& SourcePath,
	const FKeyChain& SourceKeyChain,
	const FString& TargetPath,
	const FKeyChain& TargetKeyChain,
	const FString& OutPath,
	EChunkTypeFilter ChunkTypeFilter)
{
	struct FContainerChunkInfo
	{
		FString ContainerName;
		TMap<FIoChunkId, FIoStoreTocChunkInfo> ChunkInfoById;
		int64 UncompressedContainerSize = 0;
		int64 CompressedContainerSize = 0;
	};

	struct FContainerDiff
	{
		TSet<FIoChunkId> Unmodified;
		TSet<FIoChunkId> Modified;
		TSet<FIoChunkId> Added;
		TSet<FIoChunkId> Removed;
		int64 UnmodifiedCompressedSize = 0;
		int64 ModifiedCompressedSize = 0;
		int64 AddedCompressedSize = 0;
		int64 RemovedCompressedSize = 0;
	};

	using FContainers = TMap<FString, FContainerChunkInfo>;

	auto ReadContainers = [ChunkTypeFilter](const FString& Directory, const FKeyChain& KeyChain, FContainers& OutContainers)
	{
		TArray<FString> ContainerFileNames;
		IFileManager::Get().FindFiles(ContainerFileNames, *(Directory / TEXT("*.utoc")), true, false);

		for (const FString& ContainerFileName : ContainerFileNames)
		{
			FString ContainerFilePath = Directory / ContainerFileName;
			UE_LOGF(LogIoStore, Display, "Reading container '%ls'", *ContainerFilePath);

			TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*ContainerFilePath, KeyChain);
			if (!Reader.IsValid())
			{
				UE_LOGF(LogIoStore, Warning, "Failed to read container '%ls'", *ContainerFilePath);
				continue;
			}

			FString ContainerName = FPaths::GetBaseFilename(ContainerFileName);
			FContainerChunkInfo& ContainerChunkInfo = OutContainers.FindOrAdd(ContainerName);
			ContainerChunkInfo.ContainerName = MoveTemp(ContainerName);

			Reader->EnumerateChunks([&ContainerChunkInfo, ChunkTypeFilter](const FIoStoreTocChunkInfo& ChunkInfo)
			{
				const EIoChunkType ChunkType = ChunkInfo.Id.GetChunkType();

				const bool bCompareChunk =
					ChunkTypeFilter == EChunkTypeFilter::None
					|| (ChunkTypeFilter == EChunkTypeFilter::PackageData
							&& ChunkType == EIoChunkType::ExportBundleData)
					|| (ChunkTypeFilter == EChunkTypeFilter::BulkData
							&& (ChunkType == EIoChunkType::BulkData || ChunkType == EIoChunkType::OptionalBulkData || ChunkType == EIoChunkType::MemoryMappedBulkData));

				if (bCompareChunk)
				{
					ContainerChunkInfo.ChunkInfoById.Add(ChunkInfo.Id, ChunkInfo);
					ContainerChunkInfo.UncompressedContainerSize += ChunkInfo.Size;
					ContainerChunkInfo.CompressedContainerSize += ChunkInfo.CompressedSize;
				}

				return true;
			});
		}
	};

	auto ComputeDiff = [](const FContainerChunkInfo& SourceContainer, const FContainerChunkInfo& TargetContainer) -> FContainerDiff 
	{
		check(SourceContainer.ContainerName == TargetContainer.ContainerName);

		FContainerDiff ContainerDiff;

		for (const auto& TargetChunkInfo : TargetContainer.ChunkInfoById)
		{
			if (const FIoStoreTocChunkInfo* SourceChunkInfo = SourceContainer.ChunkInfoById.Find(TargetChunkInfo.Key))
			{
				if (SourceChunkInfo->ChunkHash != TargetChunkInfo.Value.ChunkHash)
				{
					ContainerDiff.Modified.Add(TargetChunkInfo.Key);
					ContainerDiff.ModifiedCompressedSize += TargetChunkInfo.Value.CompressedSize;
				}
				else
				{
					ContainerDiff.Unmodified.Add(TargetChunkInfo.Key);
					ContainerDiff.UnmodifiedCompressedSize += TargetChunkInfo.Value.CompressedSize;
				}
			}
			else
			{
				ContainerDiff.Added.Add(TargetChunkInfo.Key);
				ContainerDiff.AddedCompressedSize += TargetChunkInfo.Value.CompressedSize;
			}
		}

		for (const auto& SourceChunkInfo : SourceContainer.ChunkInfoById)
		{
			if (!TargetContainer.ChunkInfoById.Contains(SourceChunkInfo.Key))
			{
				ContainerDiff.Removed.Add(SourceChunkInfo.Key);
				ContainerDiff.RemovedCompressedSize += SourceChunkInfo.Value.CompressedSize;
			}
		}

		return MoveTemp(ContainerDiff);
	};

	FOutputDevice* OutputDevice = GWarn;
	TUniquePtr<FOutputDeviceFile> FileOutputDevice;

	if (!OutPath.IsEmpty())
	{
		UE_LOGF(LogIoStore, Error, "Redirecting output to: '%ls'", *OutPath);

		FileOutputDevice = MakeUnique<FOutputDeviceFile>(*OutPath, true);
		FileOutputDevice->SetSuppressEventTag(true);
		OutputDevice = FileOutputDevice.Get();
	}

	FContainers SourceContainers, TargetContainers;
	TArray<FString> AddedContainers, ModifiedContainers, RemovedContainers;
	TArray<FContainerDiff> ContainerDiffs;

	UE_LOGF(LogIoStore, Display, "Reading source container(s) from '%ls':", *SourcePath);
	ReadContainers(SourcePath, SourceKeyChain, SourceContainers);

	if (!SourceContainers.Num())
	{
		UE_LOGF(LogIoStore, Error, "Failed to read source container(s) from '%ls':", *SourcePath);
		return -1;
	}

	UE_LOGF(LogIoStore, Display, "Reading target container(s) from '%ls':", *TargetPath);
	ReadContainers(TargetPath, TargetKeyChain, TargetContainers);

	if (!TargetContainers.Num())
	{
		UE_LOGF(LogIoStore, Error, "Failed to read target container(s) from '%ls':", *SourcePath);
		return -1;
	}

	for (const auto& TargetContainer : TargetContainers)
	{
		if (SourceContainers.Contains(TargetContainer.Key))
		{
			ModifiedContainers.Add(TargetContainer.Key);
		}
		else
		{
			AddedContainers.Add(TargetContainer.Key);
		}
	}

	for (const auto& SourceContainer : SourceContainers)
	{
		if (!TargetContainers.Contains(SourceContainer.Key))
		{
			RemovedContainers.Add(SourceContainer.Key);
		}
	}

	for (const FString& ModifiedContainer : ModifiedContainers)
	{
		ContainerDiffs.Emplace(ComputeDiff(*SourceContainers.Find(ModifiedContainer), *TargetContainers.Find(ModifiedContainer)));
	}

	OutputDevice->Logf(ELogVerbosity::Display, TEXT(""));
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("------------------------------ Container Diff Summary ------------------------------"));
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("Source path '%s'"), *SourcePath);
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("Target path '%s'"), *TargetPath);
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("Chunk type filter '%s'"), *LexToString(ChunkTypeFilter));

	OutputDevice->Logf(ELogVerbosity::Display, TEXT(""));
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("Source container file(s):"));
	OutputDevice->Logf(ELogVerbosity::Display, TEXT(""));
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("%-40s %15s %15s"), TEXT("Container"), TEXT("Size (MB)"), TEXT("Chunks"));
	OutputDevice->Logf(ELogVerbosity::Display, TEXT("-------------------------------------------------------------------------"));

	{
		uint64 TotalSourceBytes = 0;
		uint64 TotalSourceChunks = 0;

		for (const auto& NameContainerPair : SourceContainers)
		{
			const FContainerChunkInfo& SourceContainer = NameContainerPair.Value;
			OutputDevice->Logf(ELogVerbosity::Display, TEXT("%-40s %15.2lf %15d"), *SourceContainer.ContainerName, double(SourceContainer.CompressedContainerSize) / 1024.0 / 1024.0, SourceContainer.ChunkInfoById.Num());

			TotalSourceBytes += SourceContainer.CompressedContainerSize;
			TotalSourceChunks += SourceContainer.ChunkInfoById.Num();
		}

		OutputDevice->Logf(ELogVerbosity::Display, TEXT("-------------------------------------------------------------------------"));
		OutputDevice->Logf(ELogVerbosity::Display, TEXT("%-40s %15.2lf %15d"), *FString::Printf(TEXT("Total of %d container file(s)"), SourceContainers.Num()), double(TotalSourceBytes) / 1024.0 / 1024.0, TotalSourceChunks);
	}

	{
		uint64 TotalTargetBytes = 0;
		uint64 TotalTargetChunks = 0;
		uint64 TotalUnmodifiedChunks = 0;
		uint64 TotalUnmodifiedCompressedBytes = 0;
		uint64 TotalModifiedChunks = 0;
		uint64 TotalModifiedCompressedBytes = 0;
		uint64 TotalAddedChunks = 0;
		uint64 TotalAddedCompressedBytes = 0;
		uint64 TotalRemovedChunks = 0;
		uint64 TotalRemovedCompressedBytes = 0;

		if (ModifiedContainers.Num())
		{
			OutputDevice->Logf(ELogVerbosity::Display, TEXT(""));
			OutputDevice->Logf(ELogVerbosity::Display, TEXT("Target container file(s):"));
			OutputDevice->Logf(ELogVerbosity::Display, TEXT(""));
			OutputDevice->Logf(ELogVerbosity::Display, TEXT("%-40s %15s %15s %25s %25s %25s %25s %25s %25s %25s %25s"), TEXT("Container"), TEXT("Size (MB)"), TEXT("Chunks"), TEXT("Unmodified"), TEXT("Unmodified (MB)"), TEXT("Modified"), TEXT("Modified (MB)"), TEXT("Added"), TEXT("Added (MB)"), TEXT("Removed"), TEXT("Removed (MB)"));
			OutputDevice->Logf(ELogVerbosity::Display, TEXT("----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------"));

			for (int32 Idx = 0; Idx < ModifiedContainers.Num(); Idx++)
			{
				const FContainerChunkInfo& SourceContainer = *SourceContainers.Find(ModifiedContainers[Idx]);
				const FContainerChunkInfo& TargetContainer = *TargetContainers.Find(ModifiedContainers[Idx]);
				const FContainerDiff& Diff = ContainerDiffs[Idx];

				const int32 NumChunks = TargetContainer.ChunkInfoById.Num();
				const int32 NumSourceChunks = SourceContainer.ChunkInfoById.Num();

				OutputDevice->Logf(ELogVerbosity::Display, TEXT("%-40s %15s %15d %25s %25s %25s %25s %25s %25s %25s %25s"),
					*TargetContainer.ContainerName,
					*FString::Printf(TEXT("%.2lf"),
						double(TargetContainer.CompressedContainerSize) / 1024.0 / 1024.0),
					NumChunks,
					*FString::Printf(TEXT("%d (%.2lf%%)"),
						Diff.Unmodified.Num(),
						100.0 * (double(Diff.Unmodified.Num()) / double(NumChunks))),
					*FString::Printf(TEXT("%.2lf (%.2lf%%)"),
						double(Diff.UnmodifiedCompressedSize) / 1024.0 / 1024.0,
						100.0 * (Diff.UnmodifiedCompressedSize) / double(TargetContainer.CompressedContainerSize)),
					*FString::Printf(TEXT("%d (%.2lf%%)"),
						Diff.Modified.Num(),
						100.0 * (double(Diff.Modified.Num()) / double(NumChunks))),
					*FString::Printf(TEXT("%.2lf (%.2lf%%)"),
						double(Diff.ModifiedCompressedSize) / 1024.0 / 1024.0,
						100.0 * (Diff.ModifiedCompressedSize) / double(TargetContainer.CompressedContainerSize)),
					*FString::Printf(TEXT("%d (%.2lf%%)"),
						Diff.Added.Num(),
						100.0 * (double(Diff.Added.Num()) / double(NumChunks))),
					*FString::Printf(TEXT("%.2lf (%.2lf%%)"),
						double(Diff.AddedCompressedSize) / 1024.0 / 1024.0,
						100.0 * (Diff.AddedCompressedSize) / double(TargetContainer.CompressedContainerSize)),
					*FString::Printf(TEXT("%d/%d (%.2lf%%)"),
						Diff.Removed.Num(),
						NumSourceChunks,
						100.0 * (double(Diff.Removed.Num()) / double(NumSourceChunks))),
					*FString::Printf(TEXT("%.2lf (%.2lf%%)"),
						double(Diff.RemovedCompressedSize) / 1024.0 / 1024.0,
						100.0 * (Diff.RemovedCompressedSize) / double(SourceContainer.CompressedContainerSize)));

				TotalTargetBytes += TargetContainer.CompressedContainerSize;
				TotalTargetChunks += NumChunks;
				TotalUnmodifiedChunks += Diff.Unmodified.Num();
				TotalUnmodifiedCompressedBytes += Diff.UnmodifiedCompressedSize;
				TotalModifiedChunks += Diff.Modified.Num();
				TotalModifiedCompressedBytes += Diff.ModifiedCompressedSize;
				TotalAddedChunks += Diff.Added.Num();
				TotalAddedCompressedBytes += Diff.AddedCompressedSize;
				TotalRemovedChunks += Diff.Removed.Num();
				TotalRemovedCompressedBytes += Diff.RemovedCompressedSize;
			}
		}

		if (AddedContainers.Num())
		{
			for (const FString& AddedContainer : AddedContainers)
			{
				const FContainerChunkInfo& TargetContainer = *TargetContainers.Find(AddedContainer);
				OutputDevice->Logf(ELogVerbosity::Display, TEXT("+%-39s %15.2lf %15d %25s %25s %25s %25s %25s %25s %25s %25s"),
					*TargetContainer.ContainerName,
					double(TargetContainer.CompressedContainerSize) / 1024.0 / 1024.0,
					TargetContainer.ChunkInfoById.Num(),
					TEXT("-"), TEXT("-"), TEXT("-"), TEXT("-"), TEXT("-"), TEXT("-"), TEXT("-"), TEXT("-"));

				TotalTargetBytes += TargetContainer.CompressedContainerSize;
				TotalTargetChunks += TargetContainer.ChunkInfoById.Num();
			}
		}

		OutputDevice->Logf(ELogVerbosity::Display, TEXT("----------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------"));
		OutputDevice->Logf(ELogVerbosity::Display, TEXT("%-40s %15.2lf %15d %25d %25.2f %25d %25.2f %25d %25.2f %25d %25.2f"),
			*FString::Printf(TEXT("Total of %d container file(s)"), TargetContainers.Num()),
			double(TotalTargetBytes) / 1024.0 / 1024.0,
			TotalTargetChunks,
			TotalUnmodifiedChunks,
			double(TotalUnmodifiedCompressedBytes) / 1024.0 / 1024.0,
			TotalModifiedChunks,
			double(TotalModifiedCompressedBytes) / 1024.0 / 1024.0,
			TotalAddedChunks,
			double(TotalAddedCompressedBytes) / 1024.0 / 1024.0,
			TotalRemovedChunks,
			double(TotalRemovedCompressedBytes) / 1024.0 / 1024.0);
	}

	return 0;
}

bool DiffIoStoreContainer(const TCHAR* CmdLine)
{
	FString SourcePath, TargetPath, OutPath;
	FKeyChain SourceKeyChain, TargetKeyChain;

	if (!FParse::Value(CmdLine, TEXT("DiffContainer="), SourcePath))
	{
		UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -DiffContainer=<Path> -Target=<Path>");
		return false;
	}

	if (!IFileManager::Get().DirectoryExists(*SourcePath))
	{
		UE_LOGF(LogIoStore, Error, "Source directory '%ls' doesn't exist", *SourcePath);
		return false;
	}

	if (!FParse::Value(CmdLine, TEXT("Target="), TargetPath))
	{
		UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -DiffContainer=<Path> -Target=<Path>");
	}

	if (!IFileManager::Get().DirectoryExists(*TargetPath))
	{
		UE_LOGF(LogIoStore, Error, "Target directory '%ls' doesn't exist", *TargetPath);
		return false;
	}

	FParse::Value(CmdLine, TEXT("DumpToFile="), OutPath);

	FString CryptoKeysCacheFilename;
	if (FParse::Value(CmdLine, TEXT("CryptoKeys="), CryptoKeysCacheFilename))
	{
		UE_LOGF(LogIoStore, Display, "Parsing source crypto keys from '%ls'", *CryptoKeysCacheFilename);
		KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, SourceKeyChain);
	}

	if (FParse::Value(CmdLine, TEXT("TargetCryptoKeys="), CryptoKeysCacheFilename))
	{
		UE_LOGF(LogIoStore, Display, "Parsing target crypto keys from '%ls'", *CryptoKeysCacheFilename);
		KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, TargetKeyChain);
	}
	else
	{
		TargetKeyChain = SourceKeyChain;
	}

	EChunkTypeFilter ChunkTypeFilter = EChunkTypeFilter::None;
	if (FParse::Param(CmdLine, TEXT("FilterBulkData")))
	{
		ChunkTypeFilter = EChunkTypeFilter::BulkData;
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("FilterPackageData")))
	{
		ChunkTypeFilter = EChunkTypeFilter::PackageData;
	}

	return Diff(SourcePath, SourceKeyChain, TargetPath, TargetKeyChain, OutPath, ChunkTypeFilter) == 0;
}

bool LegacyDiffIoStoreContainers(const TCHAR* InContainerFilename1, const TCHAR* InContainerFilename2, bool bInLogUniques1, bool bInLogUniques2, const FKeyChain& InKeyChain1, const FKeyChain* InKeyChain2)
{
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);
	UE_LOGF(LogIoStore, Display, "FileEventType, FileName, Size1, Size2");

	TUniquePtr<FIoStoreReader> Reader1 = CreateIoStoreReader(InContainerFilename1, InKeyChain1);
	if (!Reader1.IsValid())
	{
		return false;
	}

	if (!EnumHasAnyFlags(Reader1->GetContainerFlags(), EIoContainerFlags::Indexed))
	{
		UE_LOGF(LogIoStore, Warning, "Missing directory index for container '%ls'", InContainerFilename1);
	}

	TUniquePtr<FIoStoreReader> Reader2 = CreateIoStoreReader(InContainerFilename2, InKeyChain2 ? *InKeyChain2 : InKeyChain1);
	if (!Reader2.IsValid())
	{
		return false;
	}

	if (!EnumHasAnyFlags(Reader2->GetContainerFlags(), EIoContainerFlags::Indexed))
	{
		UE_LOGF(LogIoStore, Warning, "Missing directory index for container '%ls'", InContainerFilename2);
	}

	struct FEntry
	{
		FString FileName;
		FIoHash ChunkHash;
		uint64 Size;
	};

	TMap<FIoChunkId, FEntry> Container1Entries;
	Reader1->EnumerateChunks([&Container1Entries](const FIoStoreTocChunkInfo& ChunkInfo)
		{
			FEntry& Entry = Container1Entries.Add(ChunkInfo.Id);
			Entry.FileName = ChunkInfo.FileName;
			Entry.ChunkHash = ChunkInfo.ChunkHash;
			Entry.Size = ChunkInfo.Size;
			return true;
		});

	int32 NumDifferentContents = 0;
	int32 NumEqualContents = 0;
	int32 NumUniqueContainer1 = 0;
	int32 NumUniqueContainer2 = 0;
	Reader2->EnumerateChunks([&Container1Entries, &NumDifferentContents, &NumEqualContents, bInLogUniques2, &NumUniqueContainer2](const FIoStoreTocChunkInfo& ChunkInfo)
		{
			const FEntry* FindContainer1Entry = Container1Entries.Find(ChunkInfo.Id);
			if (FindContainer1Entry)
			{
				if (FindContainer1Entry->Size != ChunkInfo.Size)
				{
					UE_LOGF(LogIoStore, Display, "FilesizeDifferent, %ls, %llu, %llu", *ChunkInfo.FileName, FindContainer1Entry->Size, ChunkInfo.Size);
					++NumDifferentContents;
				}
				else if (FindContainer1Entry->ChunkHash != ChunkInfo.ChunkHash)
				{
					UE_LOGF(LogIoStore, Display, "ContentsDifferent, %ls, %llu, %llu", *ChunkInfo.FileName, FindContainer1Entry->Size, ChunkInfo.Size);
					++NumDifferentContents;
				}
				else
				{
					++NumEqualContents;
				}
				Container1Entries.Remove(ChunkInfo.Id);
			}
			else
			{
				++NumUniqueContainer2;
				if (bInLogUniques2)
				{
					UE_LOGF(LogIoStore, Display, "UniqueToSecondContainer, %ls, 0, %llu", *ChunkInfo.FileName, ChunkInfo.Size);
				}
			}
			return true;
		});

	for (const auto& KV : Container1Entries)
	{
		const FEntry& Entry = KV.Value;
		++NumUniqueContainer1;
		if (bInLogUniques1)
		{
			UE_LOGF(LogIoStore, Display, "UniqueToFirstContainer, %ls, %llu, 0", *Entry.FileName, Entry.Size);
		}
	}

	UE_LOGF(LogIoStore, Display, "Comparison complete");
	UE_LOGF(LogIoStore, Display, "Unique to first container: %d, Unique to second container: %d, Num Different: %d, NumEqual: %d", NumUniqueContainer1, NumUniqueContainer2, NumDifferentContents, NumEqualContents);
	return true;
}

int32 Staged2Zen(const FString& BuildPath, const FKeyChain& KeyChain, const FString& ProjectName, const ITargetPlatform* TargetPlatform)
{
	FString PlatformName = TargetPlatform->PlatformName();
	FString CookedOutputPath = FPaths::Combine(FPaths::ProjectDir(), TEXT("Saved"), TEXT("Cooked"), PlatformName);
	if (IFileManager::Get().DirectoryExists(*CookedOutputPath))
	{
		UE_LOGF(LogIoStore, Error, "'%ls' already exists", *CookedOutputPath);
		return -1;
	}

	TArray<FString> ContainerFiles;
	IFileManager::Get().FindFilesRecursive(ContainerFiles, *BuildPath, TEXT("*.utoc"), true, false);
	if (ContainerFiles.IsEmpty())
	{
		UE_LOGF(LogIoStore, Error, "No container files found");
		return -1;
	}

	TArray<FString> PakFiles;
	IFileManager::Get().FindFilesRecursive(PakFiles, *BuildPath, TEXT("*.pak"), true, false);
	if (PakFiles.IsEmpty())
	{
		UE_LOGF(LogIoStore, Error, "No pak files found");
		return -1;
	}

	UE_LOGF(LogIoStore, Display, "Extracting files from paks...");
	FPakPlatformFile PakPlatformFile;
	for (const auto& KV : KeyChain.GetEncryptionKeys())
	{
		FCoreDelegates::GetRegisterEncryptionKeyMulticastDelegate().Broadcast(KV.Key, KV.Value.Key);
	}
	PakPlatformFile.Initialize(&FPlatformFileManager::Get().GetPlatformFile(), TEXT(""));
	FString CookedEngineContentPath = FPaths::Combine(CookedOutputPath, TEXT("Engine"), TEXT("Content"));
	IFileManager::Get().MakeDirectory(*CookedEngineContentPath, true);
	FString CookedProjectContentPath = FPaths::Combine(CookedOutputPath, ProjectName, TEXT("Content"));
	IFileManager::Get().MakeDirectory(*CookedProjectContentPath, true);
	FString EngineContentPakPath = TEXT("../../../Engine/Content/");
	FString ProjectContentPakPath = FPaths::Combine(TEXT("../../.."), ProjectName, TEXT("Content"));
	for (const FString& PakFilePath : PakFiles)
	{
		PakPlatformFile.Mount(*PakFilePath, 0);
		TArray<FString> FilesInPak;
		PakPlatformFile.GetPrunedFilenamesInPakFile(PakFilePath, FilesInPak);
		for (const FString& FileInPak : FilesInPak)
		{
			FString FileName = FPaths::GetCleanFilename(FileInPak);
			if (FileName == TEXT("AssetRegistry.bin"))
			{
				FString TargetPath = FPaths::Combine(CookedOutputPath, ProjectName, FileName);
				PakPlatformFile.CopyFile(*TargetPath, *FileInPak);
			}
			else if (FileName.EndsWith(TEXT(".ushaderbytecode")))
			{
				FString TargetPath = FPaths::Combine(CookedProjectContentPath, FileName);
				PakPlatformFile.CopyFile(*TargetPath, *FileInPak);
			}
			else if (FileName.StartsWith("GlobalShaderCache"))
			{
				FString TargetPath = FPaths::Combine(CookedOutputPath, TEXT("Engine"), FileName);
				PakPlatformFile.CopyFile(*TargetPath, *FileInPak);
			}
			else if (FileName.EndsWith(TEXT(".ufont")))
			{
				FString TargetPath;
				if (FileInPak.StartsWith(EngineContentPakPath))
				{
					TargetPath = FPaths::Combine(CookedEngineContentPath, *FileInPak + EngineContentPakPath.Len());
				}
				else if (FileInPak.StartsWith(ProjectContentPakPath))
				{
					TargetPath = FPaths::Combine(CookedProjectContentPath, *FileInPak + ProjectContentPakPath.Len());
				}
				else
				{
					UE_DEBUG_BREAK();
					continue;
				}
				IFileManager::Get().MakeDirectory(*FPaths::GetPath(TargetPath), true);
				PakPlatformFile.CopyFile(*TargetPath, *FileInPak);
			}
		}
	}

	struct FBulkDataInfo
	{
		FString FileName;
		IPackageWriter::FBulkDataInfo::EType BulkDataType;
		TTuple<FIoStoreReader*, FIoChunkId> Chunk;
	};

	struct FPackageInfo
	{
		FName PackageName;
		FString FileName;
		TTuple<FIoStoreReader*, FIoChunkId> Chunk;
		TArray<FBulkDataInfo> BulkData;
		FPackageStoreEntryResource PackageStoreEntry;
	};

	struct FCollectedData
	{
		TSet<FIoChunkId> SeenChunks;
		TMap<FName, FPackageInfo> Packages;
		TMap<FPackageId, FName> PackageIdToName;
		TArray<TTuple<FIoStoreReader*, FIoChunkId>> ContainerHeaderChunks;
	} CollectedData;

	UE_LOGF(LogIoStore, Display, "Collecting chunks...");
	TArray<TUniquePtr<FIoStoreReader>> IoStoreReaders;
	IoStoreReaders.Reserve(ContainerFiles.Num());
	for (const FString& ContainerFilePath : ContainerFiles)
	{
		TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(*ContainerFilePath, KeyChain);
		if (!Reader.IsValid())
		{
			UE_LOGF(LogIoStore, Warning, "Failed to read container '%ls'", *ContainerFilePath);
			continue;
		}

		
		Reader->EnumerateChunks([&Reader, &CollectedData](const FIoStoreTocChunkInfo& ChunkInfo)
			{
				if (CollectedData.SeenChunks.Contains(ChunkInfo.Id))
				{
					return true;
				}
				CollectedData.SeenChunks.Add(ChunkInfo.Id);
				EIoChunkType ChunkType = static_cast<EIoChunkType>(ChunkInfo.Id.GetData()[11]);
				if (ChunkType == EIoChunkType::ExportBundleData ||
					ChunkType == EIoChunkType::BulkData ||
					ChunkType == EIoChunkType::OptionalBulkData ||
					ChunkType == EIoChunkType::MemoryMappedBulkData)
				{
					FString PackageNameStr;
					UE_CLOGF(!ChunkInfo.bHasValidFileName, LogIoStore, Fatal, "Missing file name for package chunk");
					if (FPackageName::TryConvertFilenameToLongPackageName(ChunkInfo.FileName, PackageNameStr, nullptr))
					{
						FName PackageName(PackageNameStr);
						CollectedData.PackageIdToName.Add(FPackageId::FromName(PackageName), PackageName);
						FPackageInfo& PackageInfo = CollectedData.Packages.FindOrAdd(PackageName);
						if (ChunkType == EIoChunkType::ExportBundleData)
						{
							PackageInfo.FileName = ChunkInfo.FileName;
							PackageInfo.PackageName = PackageName;
							PackageInfo.Chunk = MakeTuple(Reader.Get(), ChunkInfo.Id);
						}
						else
						{
							FBulkDataInfo& BulkDataInfo = PackageInfo.BulkData.AddDefaulted_GetRef();
							BulkDataInfo.FileName = ChunkInfo.FileName;
							BulkDataInfo.Chunk = MakeTuple(Reader.Get(), ChunkInfo.Id);
							if (ChunkType == EIoChunkType::OptionalBulkData)
							{
								BulkDataInfo.BulkDataType = IPackageWriter::FBulkDataInfo::Optional;
							}
							else if (ChunkType == EIoChunkType::MemoryMappedBulkData)
							{
								BulkDataInfo.BulkDataType = IPackageWriter::FBulkDataInfo::Mmap;
							}
							else
							{
								BulkDataInfo.BulkDataType = IPackageWriter::FBulkDataInfo::BulkSegment;
							}
						}
					}
					else
					{
						UE_LOGF(LogIoStore, Warning, "Failed to convert file name '%ls' to package name", *ChunkInfo.FileName);
					}
				}
				else if (ChunkType == EIoChunkType::ContainerHeader)
				{
					CollectedData.ContainerHeaderChunks.Emplace(Reader.Get(), ChunkInfo.Id);
				}
				return true;
			});

		IoStoreReaders.Emplace(MoveTemp(Reader));
	}

	UE_LOGF(LogIoStore, Display, "Reading container headers...");
	for (const auto& ContainerHeaderChunk : CollectedData.ContainerHeaderChunks)
	{
		FIoBuffer ContainerHeaderBuffer = ContainerHeaderChunk.Key->Read(ContainerHeaderChunk.Value, FIoReadOptions()).ValueOrDie();
		FMemoryReaderView Ar(MakeArrayView(ContainerHeaderBuffer.Data(), ContainerHeaderBuffer.DataSize()));
		FIoContainerHeader ContainerHeader;
		Ar << ContainerHeader;
		
		const FFilePackageStoreEntry* StoreEntry = reinterpret_cast<const FFilePackageStoreEntry*>(ContainerHeader.StoreEntries.GetData());
		for (const FPackageId& PackageId : ContainerHeader.PackageIds)
		{
			const FName* FindPackageName = CollectedData.PackageIdToName.Find(PackageId);
			if (FindPackageName)
			{
				FPackageInfo* FindPackageInfo = CollectedData.Packages.Find(*FindPackageName);
				check(FindPackageInfo);
				
				FPackageStoreEntryResource& PackageStoreEntryResource = FindPackageInfo->PackageStoreEntry;
				PackageStoreEntryResource.PackageName = *FindPackageName;
				PackageStoreEntryResource.ImportedPackageIds.SetNum(StoreEntry->ImportedPackages.Num());
				FMemory::Memcpy(PackageStoreEntryResource.ImportedPackageIds.GetData(), StoreEntry->ImportedPackages.Data(), sizeof(FPackageId) * StoreEntry->ImportedPackages.Num()); //-V575
			}
			++StoreEntry;
		}
	}

	FString MetaDataOutputPath = FPaths::Combine(CookedOutputPath, ProjectName, TEXT("Metadata"));
	TSharedRef<FZenCookArtifactReader> ZenCookArtifactReader = MakeShared<FZenCookArtifactReader>(CookedOutputPath, MetaDataOutputPath, TargetPlatform);
	TUniquePtr<FZenStoreWriter> ZenStoreWriter = MakeUnique<FZenStoreWriter>(CookedOutputPath, MetaDataOutputPath, TargetPlatform, ZenCookArtifactReader);
	
	ICookedPackageWriter::FCookInfo CookInfo;
	CookInfo.bFullBuild = true;
	ZenStoreWriter->InitializeConnection();
	// Reader InitializeConnection must come after writer InitializeConnection, it uses data written by that function.
	ZenCookArtifactReader->InitializeConnection();
	ZenStoreWriter->Initialize(CookInfo);
	ZenStoreWriter->BeginCook(CookInfo);
	int32 LocalPackageIndex = 0;
	TArray<FPackageInfo> PackagesArray;
	CollectedData.Packages.GenerateValueArray(PackagesArray);
	TAtomic<int32> UploadCount { 0 };
	ParallelFor(PackagesArray.Num(), [&UploadCount, &PackagesArray, &ZenStoreWriter](int32 Index)
	{
		const FPackageInfo& PackageInfo = PackagesArray[Index];

		IPackageWriter::FBeginPackageInfo BeginPackageInfo;
		BeginPackageInfo.PackageName = PackageInfo.PackageName;

		ZenStoreWriter->BeginPackage(BeginPackageInfo);

		IPackageWriter::FPackageInfo PackageStorePackageInfo;
		PackageStorePackageInfo.PackageName = PackageInfo.PackageName;
		PackageStorePackageInfo.LooseFilePath = PackageInfo.FileName;
		PackageStorePackageInfo.ChunkId = PackageInfo.Chunk.Value;

		FIoBuffer PackageDataBuffer = PackageInfo.Chunk.Key->Read(PackageInfo.Chunk.Value, FIoReadOptions()).ValueOrDie();
		ZenStoreWriter->WriteIoStorePackageData(PackageStorePackageInfo, PackageDataBuffer, PackageInfo.PackageStoreEntry, TArray<FFileRegion>());

		for (const FBulkDataInfo& BulkDataInfo : PackageInfo.BulkData)
		{
			IPackageWriter::FBulkDataInfo PackageStoreBulkDataInfo;
			PackageStoreBulkDataInfo.PackageName = PackageInfo.PackageName;
			PackageStoreBulkDataInfo.LooseFilePath = BulkDataInfo.FileName;
			PackageStoreBulkDataInfo.ChunkId = BulkDataInfo.Chunk.Value;
			PackageStoreBulkDataInfo.BulkDataType = BulkDataInfo.BulkDataType;
			FIoBuffer BulkDataBuffer = BulkDataInfo.Chunk.Key->Read(BulkDataInfo.Chunk.Value, FIoReadOptions()).ValueOrDie();
			ZenStoreWriter->WriteBulkData(PackageStoreBulkDataInfo, BulkDataBuffer, TArray<FFileRegion>());
		}
		
		IPackageWriter::FCommitPackageInfo CommitInfo;
		CommitInfo.PackageName = PackageInfo.PackageName;
		CommitInfo.WriteOptions = IPackageWriter::EWriteOptions::Write;
		CommitInfo.Status = IPackageWriter::ECommitStatus::Success;
		ZenStoreWriter->CommitPackage(MoveTemp(CommitInfo));

		int32 LocalUploadCount = UploadCount.IncrementExchange() + 1;
		UE_CLOGF(LocalUploadCount % 1000 == 0, LogIoStore, Display, "Uploading package %d/%d", LocalUploadCount, PackagesArray.Num());
	}, EParallelForFlags::ForceSingleThread); // Single threaded for now to limit memory usage

	UE_LOGF(LogIoStore, Display, "Waiting for uploads to finish...");
	ZenStoreWriter->EndCook(CookInfo);
	return 0;
}

int32 GenerateZenFileSystemManifest(ITargetPlatform* TargetPlatform)
{
	FString OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("Cooked"), TEXT("[Platform]"));
	OutputDirectory = FPaths::ConvertRelativePathToFull(OutputDirectory);
	FPaths::NormalizeDirectoryName(OutputDirectory);
	TUniquePtr<FSandboxPlatformFile> LocalSandboxFile = FSandboxPlatformFile::Create(false);
	LocalSandboxFile->Initialize(&FPlatformFileManager::Get().GetPlatformFile(), *FString::Printf(TEXT("-sandbox=\"%s\""), *OutputDirectory));
	const FString RootPathSandbox = LocalSandboxFile->ConvertToAbsolutePathForExternalAppForWrite(*FPaths::RootDir());
	FString MetadataPathSandbox = LocalSandboxFile->ConvertToAbsolutePathForExternalAppForWrite(*(FPaths::ProjectDir() / TEXT("Metadata")));
	const FString PlatformString = TargetPlatform->PlatformName();
	const FString ResolvedRootPath = RootPathSandbox.Replace(TEXT("[Platform]"), *PlatformString);
	const FString ResolvedMetadataPath = MetadataPathSandbox.Replace(TEXT("[Platform]"), *PlatformString);

	FZenFileSystemManifest ZenFileSystemManifest(*TargetPlatform, ResolvedRootPath);
	ZenFileSystemManifest.Generate(ResolvedMetadataPath);
	ZenFileSystemManifest.Save(*FPaths::Combine(ResolvedMetadataPath, TEXT("zenfs.manifest")));
	return 0;
}

bool ExtractFilesWriter(const FString& SrcFileName, const FString& DestFileName, const FIoChunkId& ChunkId, const uint8* Data, uint64 DataSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WriteFile);
	TUniquePtr<FArchive> FileHandle(IFileManager::Get().CreateFileWriter(*DestFileName));
	if (FileHandle)
	{
		FileHandle->Serialize(const_cast<uint8*>(Data), DataSize);
		UE_CLOGF(FileHandle->IsError(), LogIoStore, Error, "Failed writing to file \"%ls\".", *DestFileName);
		return !FileHandle->IsError();
	}
	else
	{
		UE_LOGF(LogIoStore, Error, "Unable to create file \"%ls\".", *DestFileName);
		return false;
	}
};

bool ExtractFilesFromIoStoreContainer(
	const TCHAR* InContainerFilename,
	const TCHAR* InDestPath,
	const FKeyChain& InKeyChain,
	const FString* InFilter,
	TMap<FString, uint64>* OutOrderMap,
	TArray<FGuid>* OutUsedEncryptionKeys,
	bool* bOutIsSigned)
{
	return ProcessFilesFromIoStoreContainer(InContainerFilename, InDestPath, InKeyChain, InFilter, ExtractFilesWriter, OutOrderMap, OutUsedEncryptionKeys, bOutIsSigned, -1);
}

FString MakeSafeDestEntry(const FString& NormalizedDestRoot, const FString& EntryFileName)
{
	FString DestFilename = FPaths::ConvertRelativePathToFull(NormalizedDestRoot / EntryFileName);
	if (!FPaths::IsUnderDirectory(DestFilename, NormalizedDestRoot))
	{
		FString EntryPath = FPaths::GetPath(EntryFileName);
		uint64 EntryHash = FXxHash64::HashBuffer(*EntryPath, EntryPath.Len() * sizeof(TCHAR)).Hash;
		FString HashDir = FString::Printf(TEXT("%016llx"), EntryHash);
		FString SafeDestFilename = FPaths::ConvertRelativePathToFull(NormalizedDestRoot / HashDir / FPaths::GetCleanFilename(EntryFileName));
		UE_LOG(LogPakFile, Display, TEXT("Fixup extract path for \"%s\": from \"%s\" to \"%s\""),
				*EntryFileName, *DestFilename, *SafeDestFilename);
		DestFilename = SafeDestFilename;
	}
	return DestFilename;
}

bool ProcessFilesFromIoStoreContainer(
	const TCHAR* InContainerFilename,
	const TCHAR* InDestPath,
	const FKeyChain& InKeyChain,
	const FString* InFilter,
	TFunction<bool(const FString&, const FString&, const FIoChunkId&, const uint8*, uint64)> FileProcessFunc,
	TMap<FString, uint64>* OutOrderMap,
	TArray<FGuid>* OutUsedEncryptionKeys,
	bool* bOutIsSigned,
	int32 MaxConcurrentReaders)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ExtractFilesFromIoStoreContainer);

	TUniquePtr<FIoStoreReader> Reader = CreateIoStoreReader(InContainerFilename, InKeyChain);
	if (!Reader.IsValid())
	{
		return false;
	}

	if (!EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Indexed))
	{
		UE_LOGF(LogIoStore, Error, "Missing directory index for container '%ls'", InContainerFilename);
		return false;
	}
	
	if (OutUsedEncryptionKeys)
	{
		OutUsedEncryptionKeys->Add(Reader->GetEncryptionKeyGuid());
	}

	if (bOutIsSigned)
	{
		*bOutIsSigned = EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Signed);
	}

	UE_LOGF(LogIoStore, Display, "Extracting files from IoStore container '%ls'...", InContainerFilename);

	struct FEntry
	{
		FIoChunkId ChunkId;
		FString SourceFileName;
		FString DestFileName;
		uint64 Offset;
		bool bIsCompressed;

		FIoHash ChunkHash;
	};
	TArray<FEntry> Entries;
	const FIoDirectoryIndexReader& IndexReader = Reader->GetDirectoryIndexReader();
	FString NormalizedDestRoot = FPaths::ConvertRelativePathToFull(InDestPath);
	Reader->EnumerateChunks([&Entries, InFilter, &NormalizedDestRoot](const FIoStoreTocChunkInfo& ChunkInfo)
		{
			if (!ChunkInfo.bHasValidFileName)
			{
				return true;
			}

			if (InFilter && (!ChunkInfo.FileName.MatchesWildcard(*InFilter)))
			{
				return true;
			}

			FString DestFileName = ChunkInfo.FileName;
			FPaths::NormalizeFilename(DestFileName);
			DestFileName.ReplaceInline(TEXT("../../../"), TEXT(""));
			DestFileName = MakeSafeDestEntry(NormalizedDestRoot, DestFileName);

			FEntry& Entry = Entries.AddDefaulted_GetRef();
			Entry.ChunkId = ChunkInfo.Id;
			Entry.SourceFileName = ChunkInfo.FileName;
			Entry.DestFileName = DestFileName;
			Entry.Offset = ChunkInfo.Offset;
			Entry.bIsCompressed = ChunkInfo.bIsCompressed;

			Entry.ChunkHash = ChunkInfo.ChunkHash;

			return true;
		});

	
	const bool bContainerIsEncrypted = EnumHasAnyFlags(Reader->GetContainerFlags(), EIoContainerFlags::Encrypted);
	const int32 MaxConcurrentTasks = (MaxConcurrentReaders <= 0) ? Entries.Num() : FMath::Min(MaxConcurrentReaders, Entries.Num());
	int32 ErrorCount = 0;

	for (int32 EntryStartIdx = 0; EntryStartIdx < Entries.Num(); )
	{
		TArray<UE::Tasks::TTask<bool>> ExtractTasks;
		ExtractTasks.Reserve(MaxConcurrentTasks);
		EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;

		const int32 NumTasks = FMath::Min(MaxConcurrentTasks, Entries.Num() - EntryStartIdx);
		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			const FEntry& Entry = Entries[EntryStartIdx + TaskIndex];

			UE::Tasks::TTask<TIoStatusOr<FIoBuffer>> ReadTask = Reader->ReadAsync(Entry.ChunkId, FIoReadOptions());

			// Once the read is done, write out the file.
			ExtractTasks.Emplace(UE::Tasks::Launch(TEXT("IoStore_Extract"),
				[&Entry, &FileProcessFunc, ReadTask]() mutable
				{
					TIoStatusOr<FIoBuffer> ReadChunkResult = ReadTask.GetResult();
					if (!ReadChunkResult.IsOk())
					{
						UE_LOGF(LogIoStore, Error, "Failed reading chunk for file \"%ls\" (%ls).", *Entry.SourceFileName, *ReadChunkResult.Status().ToString());
						return false;
					}

					const uint8* Data = ReadChunkResult.ValueOrDie().Data();
					uint64 DataSize = ReadChunkResult.ValueOrDie().DataSize();
					if (Entry.ChunkId.GetChunkType() == EIoChunkType::ExportBundleData)
					{
						const FZenPackageSummary* PackageSummary = reinterpret_cast<const FZenPackageSummary*>(Data);
						uint64 HeaderDataSize = PackageSummary->HeaderSize;
						check(HeaderDataSize <= DataSize);
						FString DestFileName = FPaths::ChangeExtension(Entry.DestFileName, TEXT(".uheader"));
						if (!FileProcessFunc(Entry.SourceFileName, DestFileName, Entry.ChunkId, Data, HeaderDataSize))
						{
							return false;
						}
						DestFileName = FPaths::ChangeExtension(Entry.DestFileName, TEXT(".uexp"));
						if (!FileProcessFunc(Entry.SourceFileName, DestFileName, Entry.ChunkId, Data + HeaderDataSize, DataSize - HeaderDataSize))
						{
							return false;
						}
					}
					else if (!FileProcessFunc(Entry.SourceFileName, Entry.DestFileName, Entry.ChunkId, Data, DataSize))
					{
						return false;
					}
					return true;
				},
				Prerequisites(ReadTask)));
		}

		for (int32 TaskIndex = 0; TaskIndex < NumTasks; ++TaskIndex)
		{
			if (ExtractTasks[TaskIndex].GetResult())
			{
				const FEntry& Entry = Entries[EntryStartIdx + TaskIndex];
				if (OutOrderMap != nullptr)
				{
					OutOrderMap->Add(IndexReader.GetMountPoint() / Entry.SourceFileName, Entry.Offset);
				}
			}
			else
			{
				++ErrorCount;
			}
		}

		EntryStartIdx += NumTasks;
	}

	UE_LOGF(LogIoStore, Log, "Finished extracting %d chunks (including %d errors).", Entries.Num(), ErrorCount);
	return true;
}

bool SignIoStoreContainer(const TCHAR* InContainerFilename, const FRSAKeyHandle InSigningKey)
{
	FString TocFilePath = FPaths::ChangeExtension(InContainerFilename, TEXT(".utoc"));
	FString TempOutputPath = TocFilePath + ".tmp";
	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	ON_SCOPE_EXIT
	{
		if (Ipf.FileExists(*TempOutputPath))
		{
			Ipf.DeleteFile(*TempOutputPath);
		}
	};

	FIoStoreTocResource TocResource;
	FIoStatus Status = FIoStoreTocResource::Read(*TocFilePath, EIoStoreTocReadOptions::ReadAll, TocResource);
	if (!Status.IsOk())
	{
		UE_LOGF(LogIoStore, Error, "Failed reading container file \"%ls\".", InContainerFilename);
		return false;
	}

	if (TocResource.ChunkBlockSignatures.Num() != TocResource.CompressionBlocks.Num())
	{
		UE_LOGF(LogIoStore, Display, "Container is not signed, calculating block hashes...");
		TocResource.ChunkBlockSignatures.Empty();
		TUniquePtr<FArchive> ContainerFileReader;
		int32 LastPartitionIndex = -1;
		TArray<uint8> BlockBuffer;
		BlockBuffer.SetNum(static_cast<int32>(TocResource.Header.CompressionBlockSize));
		const int32 BlockCount = TocResource.CompressionBlocks.Num();
		FString ContainerBasePath = FPaths::ChangeExtension(InContainerFilename, TEXT(""));
		TStringBuilder<256> UcasFilePath;
		for (int32 BlockIndex = 0; BlockIndex < BlockCount; ++BlockIndex)
		{
			const FIoStoreTocCompressedBlockEntry& CompressionBlockEntry = TocResource.CompressionBlocks[BlockIndex];
			uint64 BlockRawSize = Align(CompressionBlockEntry.GetCompressedSize(), FAES::AESBlockSize);
			check(BlockRawSize <= TocResource.Header.CompressionBlockSize);
			const int32 PartitionIndex = int32(CompressionBlockEntry.GetOffset() / TocResource.Header.PartitionSize);
			const uint64 PartitionRawOffset = CompressionBlockEntry.GetOffset() % TocResource.Header.PartitionSize;
			if (PartitionIndex != LastPartitionIndex)
			{
				UcasFilePath.Reset();
				UcasFilePath.Append(ContainerBasePath);
				if (PartitionIndex > 0)
				{
					UcasFilePath.Append(FString::Printf(TEXT("_s%d"), PartitionIndex));
				}
				UcasFilePath.Append(TEXT(".ucas"));
				IFileHandle* ContainerFileHandle = Ipf.OpenRead(*UcasFilePath, /* allowwrite */ false);
				if (!ContainerFileHandle)
				{
					UE_LOGF(LogIoStore, Error, "Failed opening container file \"%ls\".", *UcasFilePath);
					return false;
				}
				ContainerFileReader.Reset(new FArchiveFileReaderGeneric(ContainerFileHandle, *UcasFilePath, ContainerFileHandle->Size(), 256 << 10));
				LastPartitionIndex = PartitionIndex;
			}
			ContainerFileReader->Seek(PartitionRawOffset);
			ContainerFileReader->Precache(PartitionRawOffset, 0); // Without this buffering won't work due to the first read after a seek always being uncached
			ContainerFileReader->Serialize(BlockBuffer.GetData(), BlockRawSize);
			FSHAHash& BlockHash = TocResource.ChunkBlockSignatures.AddDefaulted_GetRef();
			FSHA1::HashBuffer(BlockBuffer.GetData(), BlockRawSize, BlockHash.Hash);
		}
	}

	FIoContainerSettings ContainerSettings;
	ContainerSettings.ContainerId = TocResource.Header.ContainerId;
	ContainerSettings.ContainerFlags = TocResource.Header.ContainerFlags | EIoContainerFlags::Signed;
	ContainerSettings.EncryptionKeyGuid = TocResource.Header.EncryptionKeyGuid;
	ContainerSettings.SigningKey = InSigningKey;

	TIoStatusOr<uint64> WriteStatus = FIoStoreTocResource::Write(*TempOutputPath, TocResource, TocResource.Header.CompressionBlockSize, TocResource.Header.PartitionSize, ContainerSettings);
	if (!WriteStatus.IsOk())
	{
		UE_LOGF(LogIoStore, Error, "Failed writing new utoc file file \"%ls\".", *TocFilePath);
		return false;
	}

	Ipf.DeleteFile(*TocFilePath);
	Ipf.MoveFile(*TocFilePath, *TempOutputPath);

	return true;
}

static bool ParsePakResponseFile(const TCHAR* FilePath, TArray<FContainerSourceFile>& OutFiles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ParsePakResponseFile);

	TArray<FString> ResponseFileContents;
	if (!FFileHelper::LoadFileToStringArray(ResponseFileContents, FilePath))
	{
		UE_LOGF(LogIoStore, Error, "Failed to read response file '%ls'.", FilePath);
		return false;
	}

	for (const FString& ResponseLine : ResponseFileContents)
	{
		TArray<FString> SourceAndDest;
		TArray<FString> Switches;

		FString NextToken;
		const TCHAR* ResponseLinePtr = *ResponseLine;
		while (FParse::Token(ResponseLinePtr, NextToken, false))
		{
			if ((**NextToken == TCHAR('-')))
			{
				new(Switches) FString(NextToken.Mid(1));
			}
			else
			{
				new(SourceAndDest) FString(NextToken);
			}
		}

		if (SourceAndDest.Num() == 0)
		{
			continue;
		}

		if (SourceAndDest.Num() != 2)
		{
			UE_LOGF(LogIoStore, Error, "Invalid line in response file '%ls'.", *ResponseLine);
			return false;
		}

		FPaths::NormalizeFilename(SourceAndDest[0]);

		FContainerSourceFile& FileEntry = OutFiles.AddDefaulted_GetRef();
		FileEntry.NormalizedPath = MoveTemp(SourceAndDest[0]);
		FileEntry.DestinationPath = MoveTemp(SourceAndDest[1]);

		for (int32 Index = 0; Index < Switches.Num(); ++Index)
		{
			if (Switches[Index] == TEXT("compress"))
			{
				FileEntry.bNeedsCompression = true;
			}
			if (Switches[Index] == TEXT("encrypt"))
			{
				FileEntry.bNeedsEncryption = true;
			}
		}
	}
	return true;
}

static bool ParsePakOrderFile(const TCHAR* FilePath, FFileOrderMap& Map, const FIoStoreArguments& Arguments)
{
	IOSTORE_CPU_SCOPE(ParsePakOrderFile);

	TArray<FString> OrderFileContents;
	if (!FFileHelper::LoadFileToStringArray(OrderFileContents, FilePath))
	{
		UE_LOGF(LogIoStore, Error, "Failed to read order file '%ls'.", FilePath);
		return false;
	}

	Map.Name = FPaths::GetCleanFilename(FilePath);
	UE_LOGF(LogIoStore, Display, "Order file %ls (short name %ls) priority %d", FilePath, *Map.Name, Map.Priority);
	int64 NextOrder = 0;
	for (const FString& OrderLine : OrderFileContents)
	{
		const TCHAR* OrderLinePtr = *OrderLine;
		FString PackageName;

		// Skip comments
		if (FCString::Strncmp(OrderLinePtr, TEXT("#"), 1) == 0 || FCString::Strncmp(OrderLinePtr, TEXT("//"), 2) == 0)
		{
			continue;
		}

		if (!FParse::Token(OrderLinePtr, PackageName, false))
		{
			UE_LOGF(LogIoStore, Error, "Invalid line in order file '%ls'.", *OrderLine);
			return false;
		}

		FName PackageFName;
		if (FPackageName::IsValidTextForLongPackageName(PackageName))
		{
			PackageFName = FName(PackageName);
		}
		else if (PackageName.StartsWith(TEXT("../../../")))
		{
			FString FullFileName = FPaths::Combine(Arguments.CookedDir, PackageName.RightChop(9));
			FPaths::NormalizeFilename(FullFileName);
			PackageFName = Arguments.PackageStore->GetPackageNameFromFileName(FullFileName);
		}

		if (!PackageFName.IsNone() && !Map.PackageNameToOrder.Contains(PackageFName))
		{
			Map.PackageNameToOrder.Emplace(PackageFName, NextOrder++);
		}
	}

	UE_LOGF(LogIoStore, Display, "Order file %ls (short name %ls) contained %d valid entries", FilePath, *Map.Name, Map.PackageNameToOrder.Num());
	return true;
}

class FCookedFileVisitor : public IPlatformFile::FDirectoryStatVisitor
{
	FCookedFileStatMap& CookedFileStatMap;

public:
	FCookedFileVisitor(FCookedFileStatMap& InCookedFileSizes)
		: CookedFileStatMap(InCookedFileSizes)
	{
		
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData)
	{
		if (StatData.bIsDirectory)
		{
			return true;
		}

		CookedFileStatMap.Add(FilenameOrDirectory, StatData.FileSize);

		return true;
	}
};

static bool ParseSizeArgument(const TCHAR* CmdLine, const TCHAR* Argument, uint64& OutSize, uint64 DefaultSize = 0)
{
	FString SizeString;
	if (FParse::Value(CmdLine, Argument, SizeString) && FParse::Value(CmdLine, Argument, OutSize))
	{
		if (SizeString.EndsWith(TEXT("MB")))
		{
			OutSize *= 1024*1024;
		}
		else if (SizeString.EndsWith(TEXT("KB")))
		{
			OutSize *= 1024;
		}
		return true;
	}
	else
	{
		OutSize = DefaultSize;
		return false;
	}
}

static EFallbackOrderMode ParseFallbackOrderMode(const FString& FallbackOrderStr)
{
	if (FallbackOrderStr == TEXT("LoadOrder"))
	{
		return EFallbackOrderMode::LoadOrder;
	}
	else if (FallbackOrderStr == TEXT("AlphabeticalClustered"))
	{
		return EFallbackOrderMode::AlphabeticalClustered;
	}
	else if (FallbackOrderStr == TEXT("Alphabetical"))
	{
		return EFallbackOrderMode::Alphabetical;
	}
	else
	{
		UE_LOGF(LogIoStore, Fatal, "Unrecognized fallback order '%ls'.", *FallbackOrderStr);
		return EFallbackOrderMode::LoadOrder;
	}
}

static bool ParseOrderFileArguments(FIoStoreArguments& Arguments)
{
	IOSTORE_CPU_SCOPE(ParseOrderFileArguments);

	uint64 OrderMapStartIndex = 0;
	FString OrderFileStr;
	if (FParse::Value(FCommandLine::Get(), TEXT("Order="), OrderFileStr, false))
	{
		TArray<int32> OrderFilePriorities;
		TArray<FString> OrderFilePaths;
		OrderFileStr.ParseIntoArray(OrderFilePaths, TEXT(","), true);

		FString LegacyParam;
		if (FParse::Value(FCommandLine::Get(), TEXT("GameOrder="), LegacyParam, false))
		{
			UE_LOGF(LogIoStore, Warning, "-GameOrder= and -CookerOrder= are deprecated in favor of -Order");
			TArray<FString> LegacyPaths;
			LegacyParam.ParseIntoArray(LegacyPaths, TEXT(","), true);
			OrderFilePaths.Append(LegacyPaths);
		}
		if (FParse::Value(FCommandLine::Get(), TEXT("CookerOrder="), LegacyParam, false))
		{
			UE_LOGF(LogIoStore, Warning, "-CookerOrder is ignored by IoStore. -GameOrder= and -CookerOrder= are deprecated in favor of -Order.");
		}

		FString OrderPriorityString;
		if (FParse::Value(FCommandLine::Get(), TEXT("OrderPriority="), OrderPriorityString, false))
		{
			TArray<FString> PriorityStrings;
			OrderPriorityString.ParseIntoArray(PriorityStrings, TEXT(","), true);
			if (PriorityStrings.Num() != OrderFilePaths.Num())
			{
				UE_LOGF(LogIoStore, Error, "Number of parameters to -Order= and -OrderPriority= do not match");
				return false;
			}

			for (const FString& PriorityString : PriorityStrings)
			{
				int32 Priority = FCString::Atoi(*PriorityString);
				OrderFilePriorities.Add(Priority);
			}
		}
		else
		{
			OrderFilePriorities.AddZeroed(OrderFilePaths.Num());
		}

		check(OrderFilePaths.Num() == OrderFilePriorities.Num());

		bool bMerge = false;
		for (int32 i = 0; i < OrderFilePaths.Num(); ++i)
		{
			FString& OrderFile = OrderFilePaths[i];
			int32 Priority = OrderFilePriorities[i];

			FFileOrderMap OrderMap(Priority, i);
			if (!ParsePakOrderFile(*OrderFile, OrderMap, Arguments))
			{
				return false;
			}
			Arguments.OrderMaps.Add(OrderMap);
		}
	}

	Arguments.OrderingOptions.bClusterByOrderFilePriority = !FParse::Param(FCommandLine::Get(), TEXT("DoNotClusterByOrderPriority"));
	Arguments.OrderingOptions.bAlphaSortClusterPackageLists = FParse::Param(FCommandLine::Get(), TEXT("AlphaSortClusterPackageLists"));
	Arguments.OrderingOptions.bPlaceShadersAtEnd = FParse::Param(FCommandLine::Get(), TEXT("PlaceShadersAtEnd"));

	FString FallbackOrderStr;
	if (FParse::Value(FCommandLine::Get(), TEXT("-FallbackOrdering="), FallbackOrderStr))
	{
		Arguments.OrderingOptions.FallbackOrderMode = ParseFallbackOrderMode(FallbackOrderStr);
	}

	// Now apply the per-container-prefix overrides on top of the base configuration
	FString PerContainerAlphaSortClusterPackageListsString;
	if (FParse::Value(FCommandLine::Get(), TEXT("PerContainerAlphaSortClusterPackageLists="), PerContainerAlphaSortClusterPackageListsString, false))
	{
		TArray<FString> ContainerPrefixes;
		PerContainerAlphaSortClusterPackageListsString.ParseIntoArray(ContainerPrefixes, TEXT(","), true);
		for (FString& ContainerPrefix : ContainerPrefixes)
		{
			FIoStoreOrderingOptions& Options = Arguments.OrderingOptionsOverrides.FindOrAdd(MoveTemp(ContainerPrefix), Arguments.OrderingOptions);
			Options.bAlphaSortClusterPackageLists = true;
		}
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("PerContainerFallbackOrdering="), PerContainerAlphaSortClusterPackageListsString, false))
	{
		TArray<FString> ContainerOrderPairStrings;
		PerContainerAlphaSortClusterPackageListsString.ParseIntoArray(ContainerOrderPairStrings, TEXT(","), true);
		for (FString& ContainerOrderPairString : ContainerOrderPairStrings)
		{
			TArray<FString> ContainerOrderPair;
			if (ContainerOrderPairString.ParseIntoArray(ContainerOrderPair, TEXT(":"), true) != 2)
			{
				UE_LOGF(LogIoStore, Fatal, "Failed to parse per-container fallback order pair '%ls'.", *ContainerOrderPairString);
			}

			FString& ContainerPrefix = ContainerOrderPair[0];
			const FString& OverrideFallbackOrderStr = ContainerOrderPair[1];
			FIoStoreOrderingOptions& Options = Arguments.OrderingOptionsOverrides.FindOrAdd(MoveTemp(ContainerPrefix), Arguments.OrderingOptions);
			Options.FallbackOrderMode = ParseFallbackOrderMode(OverrideFallbackOrderStr);
		}
	}
	
	return true;
}

static TMap<FString, uint64> GetMaxIoStorePartitionSizeOverride(TConstArrayView<FString> MaxIoStorePartitionSizeOverrideMB)
{
	TMap<FString, uint64> MaxPartitionSizeOverrideMap;
	for (const FString& ConfigString : MaxIoStorePartitionSizeOverrideMB)
	{
		FStringView ConfigStringView(ConfigString);
		ConfigStringView = ConfigStringView.TrimStartAndEnd();
		if (ConfigStringView.IsEmpty() || ConfigStringView[0] != TEXT('(') || ConfigStringView[ConfigStringView.Len() - 1] != TEXT(')'))
		{
			continue;
		}
		ConfigStringView.RemovePrefix(1);
		ConfigStringView.RemoveSuffix(1);

		TArray<FStringView> ConfigStringArrayNameSize;
		UE::String::ParseTokens(ConfigStringView, TEXTVIEW(","), [&](FStringView Element)
		{
			ConfigStringArrayNameSize.Add(Element);
		});

		if (ConfigStringArrayNameSize.Num() != 2 || ConfigStringArrayNameSize[0].Len() == 0 || ConfigStringArrayNameSize[1].Len() == 0)
		{
			continue;
		}

		const TCHAR* ConfigSizeStringViewEnd = ConfigStringArrayNameSize[1].GetData();
		uint64 ConfigSize = FCString::Strtoui64(ConfigStringArrayNameSize[1].GetData(), (TCHAR**)&ConfigSizeStringViewEnd, 10);

		if (ConfigSizeStringViewEnd == ConfigStringArrayNameSize[1].GetData())
		{
			continue;
		}

		ConfigSize = ConfigSize << 20; // MiB to bytes

		MaxPartitionSizeOverrideMap.Add(FString(ConfigStringArrayNameSize[0]), ConfigSize);
	}

	return MaxPartitionSizeOverrideMap;
}

bool ParseContainerGenerationArguments(FIoStoreArguments& Arguments, FIoStoreWriterSettings& WriterSettings)
{
	IOSTORE_CPU_SCOPE(ParseContainerGenerationArguments);
	if (FParse::Param(FCommandLine::Get(), TEXT("sign")))
	{
		Arguments.bSign = true;
	}

	UE_LOGF(LogIoStore, Display, "Container signing - %ls", Arguments.bSign ? TEXT("ENABLED") : TEXT("DISABLED"));

	Arguments.bCreateDirectoryIndex = !FParse::Param(FCommandLine::Get(), TEXT("NoDirectoryIndex"));
	UE_LOGF(LogIoStore, Display, "Directory index - %ls", Arguments.bCreateDirectoryIndex ? TEXT("ENABLED") : TEXT("DISABLED"));

	WriterSettings.CompressionMethod = DefaultCompressionMethod;
	WriterSettings.CompressionBlockSize = DefaultCompressionBlockSize;

	TArray<FName> CompressionFormats;
	FString DesiredCompressionFormats;
	if (FParse::Value(FCommandLine::Get(), TEXT("-compressionformats="), DesiredCompressionFormats) ||
		FParse::Value(FCommandLine::Get(), TEXT("-compressionformat="), DesiredCompressionFormats))
	{
		TArray<FString> Formats;
		DesiredCompressionFormats.ParseIntoArray(Formats, TEXT(","));
		for (FString& Format : Formats)
		{
			// look until we have a valid format
			FName FormatName = *Format;

			if (FCompression::IsFormatValid(FormatName))
			{
				WriterSettings.CompressionMethod = FormatName;
				break;
			}
		}

		if (WriterSettings.CompressionMethod == NAME_None)
		{
			UE_LOGF(LogIoStore, Warning, "Failed to find desired compression format(s) '%ls'. Using falling back to '%ls'",
				*DesiredCompressionFormats, *DefaultCompressionMethod.ToString());
		}
		else
		{
			UE_LOGF(LogIoStore, Display, "Using compression format '%ls'", *WriterSettings.CompressionMethod.ToString());
		}
	}

	ParseSizeArgument(FCommandLine::Get(), TEXT("-alignformemorymapping="), WriterSettings.MemoryMappingAlignment, DefaultMemoryMappingAlignment);
	ParseSizeArgument(FCommandLine::Get(), TEXT("-compressionblocksize="), WriterSettings.CompressionBlockSize, DefaultCompressionBlockSize);

	WriterSettings.CompressionBlockAlignment = DefaultCompressionBlockAlignment;

	uint64 BlockAlignment = 0;
	if (ParseSizeArgument(FCommandLine::Get(), TEXT("-blocksize="), BlockAlignment))
	{
		WriterSettings.CompressionBlockAlignment = BlockAlignment;
	}

	//
	// If a filename to a global.utoc container is provided, all containers in that directory will have their compressed blocks be
	// made available for the new containers to reuse. This provides two benefits:
	//	1.	Saves compression time for the new blocks, as ssd/nvme io times are significantly faster.
	//	2.	Prevents trivial bit changes in the compressor from causing patch changes down the line, 
	//		allowing worry-free compressor upgrading.
	//
	// This should be a path to your last released containers. If those containers are encrypted, be sure to
	// provide keys via -ReferenceContainerCryptoKeys.
	//
	if (FParse::Value(FCommandLine::Get(), TEXT("-ReferenceContainerGlobalFileName="), Arguments.ReferenceChunkGlobalContainerFileName))
	{
		FString CryptoKeysCacheFilename;
		if (FParse::Value(FCommandLine::Get(), TEXT("-ReferenceContainerCryptoKeys="), CryptoKeysCacheFilename))
		{
			UE_LOGF(LogIoStore, Display, "Parsing reference container crypto keys from a crypto key cache file '%ls'", *CryptoKeysCacheFilename);
			KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, Arguments.ReferenceChunkKeys);
		}

		if (FParse::Value(FCommandLine::Get(), TEXT("-ReferenceContainerChangesCSVFileName="), Arguments.ReferenceChunkChangesCSVFileName))
		{
			UE_LOGF(LogIoStore, Display, "ReferenceCache will write the list of changed/new packages to %ls", *Arguments.ReferenceChunkChangesCSVFileName);
		}

		if (FParse::Value(FCommandLine::Get(), TEXT("-ReferenceContainerAdditionalPath="), Arguments.ReferenceChunkAdditionalContainersPath))
		{
			UE_LOGF(LogIoStore, Display, "ReferenceCache will read additional containers from %ls", *Arguments.ReferenceChunkAdditionalContainersPath);
		}
	}

	// By default, we use uncompressed chunk hashes from cook to avoid reading and hashing chunks unnecessarily.
	// This flag causes us to read and hash anyway, and then ensure they match. It is very bad if this fails!
	WriterSettings.bValidateChunkHashes = FParse::Param(FCommandLine::Get(), TEXT("validateChunkHashes"));

	uint64 PatchPaddingAlignment = 0;
	if (ParseSizeArgument(FCommandLine::Get(), TEXT("-patchpaddingalign="), PatchPaddingAlignment))
	{
		if (PatchPaddingAlignment < WriterSettings.CompressionBlockAlignment)
		{
			WriterSettings.CompressionBlockAlignment = PatchPaddingAlignment;
		}
	}

	// Temporary, this command-line allows us to explicitly override the value otherwise shared between pak building and iostore
	uint64 IOStorePatchPaddingAlignment = 0;
	if (ParseSizeArgument(FCommandLine::Get(), TEXT("-iostorepatchpaddingalign="), IOStorePatchPaddingAlignment))
	{
		WriterSettings.CompressionBlockAlignment = IOStorePatchPaddingAlignment;
	}

	uint64 MaxPartitionSize = 0;
	if (ParseSizeArgument(FCommandLine::Get(), TEXT("-maxPartitionSize="), MaxPartitionSize))
	{
		WriterSettings.MaxPartitionSize = MaxPartitionSize;
	}

	FConfigCacheIni* Config = GConfig;

	FString IniPlatformName;
	if (FParse::Value(FCommandLine::Get(), TEXT("platform="), IniPlatformName, false))
	{
		if (FConfigCacheIni* PlatformConfig = FConfigCacheIni::ForPlatform(*IniPlatformName))
		{
			Config = PlatformConfig;
		}
	}

	TArray<FString> MaxIoStorePartitionSizeOverrideMB;
	if (Config->GetArray(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("MaxIoStorePartitionSizeOverrideMB"), MaxIoStorePartitionSizeOverrideMB, GGameIni))
	{
		WriterSettings.MaxPartitionSizeOverride = GetMaxIoStorePartitionSizeOverride(MaxIoStorePartitionSizeOverrideMB);
	}

	int32 CompressionMinBytesSaved = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-compressionMinBytesSaved="), CompressionMinBytesSaved))
	{
		WriterSettings.CompressionMinBytesSaved = CompressionMinBytesSaved;
	}

	int32 CompressionMinPercentSaved = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-compressionMinPercentSaved="), CompressionMinPercentSaved))
	{
		WriterSettings.CompressionMinPercentSaved = CompressionMinPercentSaved;
	}

	WriterSettings.bCompressionEnableDDC = FParse::Param(FCommandLine::Get(), TEXT("compressionEnableDDC"));
	if (WriterSettings.bCompressionEnableDDC && !FPaths::IsProjectFilePathSet())
	{
		UE_LOGF(LogIoStore, Warning,
			"Ignoring -compressionEnableDDC due to missing .uproject file as the first unnamed argument.");
		WriterSettings.bCompressionEnableDDC = false;
	}

	int32 CompressionMinSizeToConsiderDDC = 0;
	if (FParse::Value(FCommandLine::Get(), TEXT("-compressionMinSizeToConsiderDDC="), CompressionMinSizeToConsiderDDC))
	{
		WriterSettings.CompressionMinSizeToConsiderDDC = CompressionMinSizeToConsiderDDC;
	}

	WriterSettings.bContainerMetaEnabled = FParse::Param(FCommandLine::Get(), TEXT("ContainerMeta"));
	WriterSettings.bPerContainerMeta = Arguments.IsDLC();

	WriterSettings.bDeduplicateChunks = FParse::Param(FCommandLine::Get(), TEXT("DeduplicateChunks"));

	WriterSettings.bUsePatchInPlaceLayout = FParse::Param(FCommandLine::Get(), TEXT("UsePatchInPlaceLayout"));

	if (WriterSettings.bUsePatchInPlaceLayout)
	{
		// default values, chosen by experimentation
		WriterSettings.PatchInPlaceParetoFrontCoarseSamples = 50;
		WriterSettings.PatchInPlaceParetoFrontFineSamples = 1500;

		// typical read granularity for QLC NAND is 16kb, TLC is 8KB
		WriterSettings.PatchInPlaceReadGranularity = 16 * 1024;

		ParseSizeArgument(FCommandLine::Get(), TEXT("-patchInPlaceMaximumTotalGrowth="), WriterSettings.PatchInPlaceMaximumTotalGrowth, 0);
		ParseSizeArgument(FCommandLine::Get(), TEXT("-patchInPlaceMaximumPerContainerGrowth="), WriterSettings.PatchInPlaceMaximumPerContainerGrowth, 0);
		FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceParetoFrontCoarseSamples="), WriterSettings.PatchInPlaceParetoFrontCoarseSamples);
		FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceParetoFrontFineSamples="), WriterSettings.PatchInPlaceParetoFrontFineSamples);
		FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceReadGranularity="), WriterSettings.PatchInPlaceReadGranularity);
		FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceMaxChainSize="), WriterSettings.PatchInPlaceMaxChainSize);
		FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceMaxChainLength="), WriterSettings.PatchInPlaceMaxChainLength);
		FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceParetoFrontHeuristic="), WriterSettings.PatchInPlaceParetoFrontHeuristic);

		if (FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceReferenceContainerPathOverride="), Arguments.PatchInPlaceReferenceContainerPathOverride))
		{
			FString CryptoKeysCacheFilename;
			if (FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceReferenceContainerCryptoOverride="), CryptoKeysCacheFilename))
			{
				if (IFileManager::Get().FileExists(*CryptoKeysCacheFilename))
				{
					UE_LOGF(LogIoStore, Display, "Parsing reference container crypto keys from a crypto key cache file '%ls'", *CryptoKeysCacheFilename);
					KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, Arguments.PatchInPlaceReferenceChunkKeys);
				}
				else
				{
					UE_LOGF(LogIoStore, Warning, "Patch-in-place reference container crypto key file '%ls' does not exist.", *CryptoKeysCacheFilename);
				}

				if (Arguments.PatchInPlaceReferenceChunkKeys.GetEncryptionKeys().Num() == 0)
				{
					UE_LOGF(LogIoStore, Warning, "No encryption keys available for patch-in-place reference containers. Disabling patch-in-place layout.");
					WriterSettings.bUsePatchInPlaceLayout = false;
				}
			}

			FParse::Value(FCommandLine::Get(), TEXT("-patchInPlaceReferenceAdditionalContainersPath="), Arguments.PatchInPlaceReferenceAdditionalContainersPath);
		}
	}

	UE_LOGF(LogIoStore, Display, "Using memory mapping alignment '%lld'", WriterSettings.MemoryMappingAlignment);
	UE_LOGF(LogIoStore, Display, "Using compression block size '%lld'", WriterSettings.CompressionBlockSize);
	UE_LOGF(LogIoStore, Display, "Using compression block alignment '%lld'", WriterSettings.CompressionBlockAlignment);
	UE_LOGF(LogIoStore, Display, "Using compression min bytes saved '%d'", WriterSettings.CompressionMinBytesSaved);
	UE_LOGF(LogIoStore, Display, "Using compression min percent saved '%d'", WriterSettings.CompressionMinPercentSaved);
	UE_LOGF(LogIoStore, Display, "Using max partition size '%lld'", WriterSettings.MaxPartitionSize);
	for (const TPair<FString, uint64>& MaxPartitionSizeOverrideIter : WriterSettings.MaxPartitionSizeOverride)
	{
		UE_LOGF(LogIoStore, Display, "Using max partition size override %ls -> '%lld'", *MaxPartitionSizeOverrideIter.Key, MaxPartitionSizeOverrideIter.Value);
	}

	if (WriterSettings.bCompressionEnableDDC)
	{
		UE_LOGF(LogIoStore, Display, "Using DDC for compression with min size '%d'", WriterSettings.CompressionMinSizeToConsiderDDC);
	}
	else
	{
		UE_LOGF(LogIoStore, Display, "Not using DDC for compression");
	}

	if (WriterSettings.bContainerMetaEnabled)
	{
		if (WriterSettings.bPerContainerMeta)
		{
			UE_LOGF(LogIoStore, Display, "Generating per container metadata file(s)");
		}
		else
		{
			UE_LOGF(LogIoStore, Display, "Generating global container metadata file");
		}
	}
	else
	{
		UE_LOGF(LogIoStore, Display, "Container metadata disabled");
	}

	if (WriterSettings.bUsePatchInPlaceLayout)
	{
		UE_LOGF(LogIoStore, Display, "Using patch-in-place ordering");

		if (WriterSettings.PatchInPlaceMaximumTotalGrowth > 0)
		{
			UE_LOGF(LogIoStore, Display, "Limiting patch-in-place total containers growth to '%llu' bytes",
				WriterSettings.PatchInPlaceMaximumTotalGrowth);
		}

		if (WriterSettings.PatchInPlaceMaximumPerContainerGrowth > 0)
		{
			UE_LOGF(LogIoStore, Display, "Limiting patch-in-place per container growth to '%llu' bytes",
				WriterSettings.PatchInPlaceMaximumPerContainerGrowth);
			UE_LOGF(LogIoStore, Display, "Using patch-in-place Pareto front configuration %llu/%llu/%u",
				WriterSettings.PatchInPlaceParetoFrontCoarseSamples,
				WriterSettings.PatchInPlaceParetoFrontFineSamples,
				WriterSettings.PatchInPlaceParetoFrontHeuristic
			);
		}

		if (WriterSettings.CompressionBlockAlignment)
		{
			UE_LOGF(LogIoStore, Warning, "Ignoring CompressionBlockAlignment due to patch-in-place enabled");
		}
	}

	FString CommandListFile;
	if (FParse::Value(FCommandLine::Get(), TEXT("Commands="), CommandListFile))
	{
		UE_LOGF(LogIoStore, Display, "Using command list file: '%ls'", *CommandListFile);
		TArray<FString> Commands;
		if (!FFileHelper::LoadFileToStringArray(Commands, *CommandListFile))
		{
			UE_LOGF(LogIoStore, Error, "Failed to read command list file '%ls'.", *CommandListFile);
			return false;
		}

		Arguments.Containers.Reserve(Commands.Num());
		for (const FString& Command : Commands)
		{
			FContainerSourceSpec& ContainerSpec = Arguments.Containers.AddDefaulted_GetRef();

			if (FParse::Value(*Command, TEXT("Output="), ContainerSpec.OutputPath))
			{
				ContainerSpec.OutputPath = FPaths::ChangeExtension(ContainerSpec.OutputPath, TEXT(""));
			}
			ContainerSpec.bOnDemand = FParse::Param(*Command, TEXT("OnDemand"));
			FParse::Value(*Command, TEXT("OptionalOutput="), ContainerSpec.OptionalOutputPath);

			FParse::Value(*Command, TEXT("StageLooseFileRootPath="), ContainerSpec.StageLooseFileRootPath);

			FString ContainerName;
			if (FParse::Value(*Command, TEXT("ContainerName="), ContainerName))
			{
				ContainerSpec.Name = FName(ContainerName);
			}

			FString PatchSourceWildcard;
			if (FParse::Value(*Command, TEXT("PatchSource="), PatchSourceWildcard))
			{
				IFileManager::Get().FindFiles(ContainerSpec.PatchSourceContainerFiles, *PatchSourceWildcard, true, false);
				FString PatchSourceContainersDirectory = FPaths::GetPath(*PatchSourceWildcard);
				for (FString& PatchSourceContainerFile : ContainerSpec.PatchSourceContainerFiles)
				{
					PatchSourceContainerFile = PatchSourceContainersDirectory / PatchSourceContainerFile;
					FPaths::NormalizeFilename(PatchSourceContainerFile);
				}
			}

			ContainerSpec.bGenerateDiffPatch = FParse::Param(*Command, TEXT("GenerateDiffPatch"));

			FParse::Value(*Command, TEXT("PatchTarget="), ContainerSpec.PatchTargetFile);

			FString ResponseFilePath;
			if (FParse::Value(*Command, TEXT("ResponseFile="), ResponseFilePath))
			{
				if (!ParsePakResponseFile(*ResponseFilePath, ContainerSpec.SourceFiles))
				{
					UE_LOGF(LogIoStore, Error, "Failed to parse Pak response file '%ls'", *ResponseFilePath);
					return false;
				}
				FParse::Value(*Command, TEXT("EncryptionKeyOverrideGuid="), ContainerSpec.EncryptionKeyOverrideGuid);
			}
		}
	}

	for (const FContainerSourceSpec& Container : Arguments.Containers)
	{
		if (Container.Name.IsNone())
		{
			UE_LOGF(LogIoStore, Error, "ContainerName argument missing for container '%ls'", *Container.OutputPath);
			return false;
		}
	}

	Arguments.bFileRegions = FParse::Param(FCommandLine::Get(), TEXT("FileRegions"));
	WriterSettings.bEnableFileRegions = Arguments.bFileRegions;

	FString PatchReferenceCryptoKeysFilename;
	if (FParse::Value(FCommandLine::Get(), TEXT("PatchCryptoKeys="), PatchReferenceCryptoKeysFilename))
	{
		KeyChainUtilities::LoadKeyChainFromFile(PatchReferenceCryptoKeysFilename, Arguments.PatchKeyChain);
	}
	else
	{
		Arguments.PatchKeyChain = Arguments.KeyChain;
	}

	return true;
}

int32 CreateIoStoreContainerFiles(const TCHAR* CmdLine)
{
	IOSTORE_CPU_SCOPE(CreateIoStoreContainerFiles);
	
	UE_LOGF(LogIoStore, Display, "==================== IoStore Utils ====================");

	FIoStoreArguments Arguments;
	FIoStoreWriterSettings WriterSettings;

	if (!LoadKeyChain(FCommandLine::Get(), Arguments.KeyChain))
	{
		return 1;
	}

	ITargetPlatform* TargetPlatform = nullptr;
	FString TargetPlatformName;
	if (FParse::Value(FCommandLine::Get(), TEXT("TargetPlatform="), TargetPlatformName))
	{
		ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
		TargetPlatform = TPM.FindTargetPlatform(TargetPlatformName);
		if (!TargetPlatform)
		{
			UE_LOGF(LogIoStore, Error, "Invalid TargetPlatform: '%ls'", *TargetPlatformName);
			return 1;
		}
	}

	FParse::Value(FCommandLine::Get(), TEXT("csv="), Arguments.CsvPath);

	FString ArgumentValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("List="), ArgumentValue))
	{
		FString ContainerPathOrWildcard = MoveTemp(ArgumentValue);
		if (Arguments.CsvPath.Len() == 0)
		{
			UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -list=<ContainerFile> -csv=<path>");
		}

		return ListContainer(Arguments.KeyChain, ContainerPathOrWildcard, Arguments.CsvPath);
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("FindAndExtract="), ArgumentValue))
	{
		// Often we know the package we want but not what container it's in, so this lets us
		// just pass in the package and it'll load all the containers and pull it out wherever it is.

		FString ContainerPathOrWildcard = MoveTemp(ArgumentValue);
		
		FString PackageFilter;
		FParse::Value(FCommandLine::Get(), TEXT("PackageFilter="), PackageFilter);
		if (PackageFilter.Len() == 0)
		{
			UE_LOGF(LogIoStore, Error, "-PackageFilter=<wildcard> is required, use * for all packages");
			return -1;
		}

		FString OutPath;
		FParse::Value(FCommandLine::Get(), TEXT("OutPath="), OutPath);
		if (OutPath.Len() == 0)
		{
			UE_LOGF(LogIoStore, Error, "-OutPath=/path/to/out is required");
			return -1;
		}

		return FindAndExtractPackages(Arguments.KeyChain, ContainerPathOrWildcard, OutPath, PackageFilter);
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("Describe="), ArgumentValue))
	{
		FString ContainerPathOrWildcard = ArgumentValue;
		FString PackageFilter;
		FParse::Value(FCommandLine::Get(), TEXT("PackageFilter="), PackageFilter);
		FString OutPath;
		FParse::Value(FCommandLine::Get(), TEXT("DumpToFile="), OutPath);
		bool bIncludeExportHashes = FParse::Param(FCommandLine::Get(), TEXT("IncludeExportHashes"));
		return Describe(ContainerPathOrWildcard, Arguments.KeyChain, PackageFilter, OutPath, bIncludeExportHashes);
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("ValidateCrossContainerRefs="), ArgumentValue))
	{
		FString ContainerPathOrWildcard = ArgumentValue;
		FString ConfigPath;
		FParse::Value(FCommandLine::Get(), TEXT("Config="), ConfigPath);
		FString OutPath;
		FParse::Value(FCommandLine::Get(), TEXT("DumpToFile="), OutPath);
		return ValidateCrossContainerRefs(ContainerPathOrWildcard, Arguments.KeyChain, ConfigPath, OutPath);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("ProfileReadSpeed")))
	{
		// Load the .UTOC file provided and read it in its entirety, cmdline parsed in function
		return ProfileReadSpeed(FCommandLine::Get(), Arguments.KeyChain);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("Diff")))
	{
		FString SourcePath, TargetPath, OutPath;
		FKeyChain SourceKeyChain, TargetKeyChain;

		if (!FParse::Value(FCommandLine::Get(), TEXT("Source="), SourcePath))
		{
			UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -Diff -Source=<Path> -Target=<path>");
			return -1;
		}

		if (!IFileManager::Get().DirectoryExists(*SourcePath))
		{
			UE_LOGF(LogIoStore, Error, "Source directory '%ls' doesn't exist", *SourcePath);
			return -1;
		}

		if (!FParse::Value(FCommandLine::Get(), TEXT("Target="), TargetPath))
		{
			UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -Diff -Source=<Path> -Target=<path>");
		}

		if (!IFileManager::Get().DirectoryExists(*TargetPath))
		{
			UE_LOGF(LogIoStore, Error, "Target directory '%ls' doesn't exist", *TargetPath);
			return -1;
		}

		FParse::Value(FCommandLine::Get(), TEXT("DumpToFile="), OutPath);

		FString CryptoKeysCacheFilename;
		if (FParse::Value(CmdLine, TEXT("CryptoKeys="), CryptoKeysCacheFilename) ||
			FParse::Value(CmdLine, TEXT("SourceCryptoKeys="), CryptoKeysCacheFilename))
		{
			UE_LOGF(LogIoStore, Display, "Parsing source crypto keys from '%ls'", *CryptoKeysCacheFilename);
			KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, SourceKeyChain);
		}

		if (FParse::Value(CmdLine, TEXT("TargetCryptoKeys="), CryptoKeysCacheFilename))
		{
			UE_LOGF(LogIoStore, Display, "Parsing target crypto keys from '%ls'", *CryptoKeysCacheFilename);
			KeyChainUtilities::LoadKeyChainFromFile(CryptoKeysCacheFilename, TargetKeyChain);
		}
		else
		{
			TargetKeyChain = SourceKeyChain;
		}

		EChunkTypeFilter ChunkTypeFilter = EChunkTypeFilter::None;
		if (FParse::Param(FCommandLine::Get(), TEXT("FilterBulkData")))
		{
			ChunkTypeFilter = EChunkTypeFilter::BulkData;
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("FilterPackageData")))
		{
			ChunkTypeFilter = EChunkTypeFilter::PackageData;
		}

		return Diff(SourcePath, SourceKeyChain, TargetPath, TargetKeyChain, OutPath, ChunkTypeFilter);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("Staged2Zen")))
	{
		FString BuildPath;
		FString ProjectName;
		if (!FParse::Value(FCommandLine::Get(), TEXT("BuildPath="), BuildPath) ||
			!FParse::Value(FCommandLine::Get(), TEXT("ProjectName="), ProjectName) ||
			!TargetPlatform)
		{
			UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -Staged2Zen -BuildPath=<Path> -ProjectName=<ProjectName> -TargetPlatform=<Platform>");
			return -1;
		}
		return Staged2Zen(BuildPath, Arguments.KeyChain, ProjectName, TargetPlatform);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("CreateContentPatch")))
	{
		if (!ParseContainerGenerationArguments(Arguments, WriterSettings))
		{
			return -1;
		}

		for (const FContainerSourceSpec& Container : Arguments.Containers)
		{
			if (Container.PatchTargetFile.IsEmpty())
			{
				UE_LOGF(LogIoStore, Error, "PatchTarget argument missing for container '%ls'", *Container.OutputPath);
				return -1;
			}
		}

		return CreateContentPatch(Arguments, WriterSettings);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("GenerateZenFileSystemManifest")))
	{
		if (!TargetPlatform)
		{
			UE_LOGF(LogIoStore, Error, "Incorrect arguments. Expected: -GenerateZenFileSystemManifest -TargetPlatform=<Platform>");
			return -11;
		}
		return GenerateZenFileSystemManifest(TargetPlatform);
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("StartZenServerForStage")))
	{
		TArray<uint32> SponsorProcessIds;
		uint32 SponsorProcessId = 0;
		if (FParse::Value(FCommandLine::Get(), TEXT("SponsorProcessID="), SponsorProcessId) && (SponsorProcessId > 0))
		{
			SponsorProcessIds.Add(SponsorProcessId);
		}

		FString MarkerFilename;
		if (FParse::Value(FCommandLine::Get(), TEXT("ProjectStore="), MarkerFilename))
		{
			TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*MarkerFilename));
			if (Ar)
			{
				UE::Zen::FZenServiceInstance& ZenServiceInstance = UE::Zen::GetDefaultServiceInstance();
				if (!ZenServiceInstance.IsServiceReady())
				{
					UE_LOGF(LogIoStore, Error, "Failed to launch ZenServer.");
					return -1;
				}

				const UE::Zen::FServiceSettings& ZenServiceSettings = ZenServiceInstance.GetServiceSettings();
				if (!SponsorProcessIds.IsEmpty() && ZenServiceSettings.IsAutoLaunch() &&
					ZenServiceInstance.GetServiceSettings().SettingsVariant.Get<UE::Zen::FServiceAutoLaunchSettings>().bLimitProcessLifetime)
				{
					if (!ZenServiceInstance.AddSponsorProcessIDs(SponsorProcessIds))
					{
						UE_LOGF(LogIoStore, Error, "Failed to add sponsor process IDs to launched ZenServer.");
						return -1;
					}
				}
				return 0;
			}
			else
			{
				UE_LOGF(LogIoStore, Error, "Failed reading project store marker");
			}
		}

		UE_LOGF(LogIoStore, Error, "Expected -PackageStoreManifest=<path to package store manifest> or -ProjectStore=<path to project store marker>");
		return -1;
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("CreateDLCContainer="), Arguments.DLCPluginPath))
	{
		if (!ParseContainerGenerationArguments(Arguments, WriterSettings))
		{
			return -1;
		}
		
		Arguments.DLCName = FPaths::GetBaseFilename(*Arguments.DLCPluginPath);
		Arguments.bRemapPluginContentToGame = FParse::Param(FCommandLine::Get(), TEXT("RemapPluginContentToGame"));

		UE_LOGF(LogIoStore, Display, "DLC: '%ls'", *Arguments.DLCPluginPath);
		UE_LOGF(LogIoStore, Display, "Remapping plugin content to game: '%ls'", Arguments.bRemapPluginContentToGame ? TEXT("True") : TEXT("False"));

		bool bAssetRegistryLoaded = false;
		FString BasedOnReleaseVersionPath;
		if (FParse::Value(FCommandLine::Get(), TEXT("BasedOnReleaseVersionPath="), BasedOnReleaseVersionPath))
		{
			UE_LOGF(LogIoStore, Display, "Based on release version path: '%ls'", *BasedOnReleaseVersionPath);
			FString DevelopmentAssetRegistryPath = FPaths::Combine(BasedOnReleaseVersionPath, TEXT("Metadata"), GetDevelopmentAssetRegistryFilename());
			FArrayReader SerializedAssetData;
			if (FPaths::FileExists(*DevelopmentAssetRegistryPath) && FFileHelper::LoadFileToArray(SerializedAssetData, *DevelopmentAssetRegistryPath))
			{
				FAssetRegistryState ReleaseAssetRegistry;
				FAssetRegistrySerializationOptions Options;
				if (ReleaseAssetRegistry.Serialize(SerializedAssetData, Options))
				{
					UE_LOGF(LogIoStore, Display, "Loaded asset registry '%ls'", *DevelopmentAssetRegistryPath);
					bAssetRegistryLoaded = true;

					TArray<FName> PackageNames;
					ReleaseAssetRegistry.GetPackageNames(PackageNames);
					Arguments.ReleasedPackages.PackageNames.Reserve(PackageNames.Num());
					Arguments.ReleasedPackages.PackageIdToName.Reserve(PackageNames.Num());

					for (FName PackageName : PackageNames)
					{
						// skip over packages that were not actually saved out, but were added to the AR - the DLC may now have those packages included,
						// and there will be a conflict later on if the package is in this list and the DLC list. PackageFlags of 0 means it was 
						// evaluated and skipped.
						bool bHasAny = false;
						ReleaseAssetRegistry.EnumerateAssetsByPackageName(PackageName,
							[&bHasAny, &Arguments, PackageName](const FAssetData* AssetData)
							{
								// just check the first one in the list, they will all have the same flags
								if (AssetData->PackageFlags != 0)
								{
									Arguments.ReleasedPackages.PackageNames.Add(PackageName);
									Arguments.ReleasedPackages.PackageIdToName.Add(FPackageId::FromName(PackageName), PackageName);
								}
								bHasAny = true;
								return false; // stop iterating
							});
						checkf(bHasAny,
							TEXT("It is unexpected that no assets were found in DevelopmentAssetRegistry for the package %s. This indicates an invalid AR."),
							*PackageName.ToString());
					}
				}
			}
		}
	}
	else if (FParse::Value(FCommandLine::Get(), TEXT("CreateGlobalContainer="), Arguments.GlobalContainerPath))
	{
		Arguments.GlobalContainerPath = FPaths::ChangeExtension(Arguments.GlobalContainerPath, TEXT(""));

		if (!ParseContainerGenerationArguments(Arguments, WriterSettings))
		{
			return -1;
		}
	}
	else
	{
		UE_LOGF(LogIoStore, Display, "Usage:");
		UE_LOGF(LogIoStore, Display, " -List=</path/to/[container.utoc|*.utoc]> -CSV=<list.csv> [-CryptoKeys=</path/to/crypto.json>]");
		UE_LOGF(LogIoStore, Display, " -Describe=</path/to/global.utoc> [-PackageFilter=<PackageNameWildcard>] [-DumpToFile=<describe.txt>] [-CryptoKeys=</path/to/crypto.json>]");
		UE_LOGF(LogIoStore, Display, " -FindAndExtract=</path/to/[container.utoc|*.utoc]> -PackageFilter=<PackageNameWildcard> -OutPath=</path/to/output> [-CryptoKeys=</path/to/crypto.json>]");
		return -1;
	}

	// Common path for creating containers
	FParse::Value(FCommandLine::Get(), TEXT("CookedDirectory="), Arguments.CookedDir);
	if (Arguments.CookedDir.IsEmpty())
	{
		UE_LOGF(LogIoStore, Error, "CookedDirectory must be specified");
		return -1;
	}

	//
	// -compresslevel is technically consumed by OodleDataCompressionFormat, however the setting that it represents (PackageCompressionLevel_*)
	// is the intention of package compression, and so should also determine the compression we use for shaders. For Shaders we want fast decompression,
	// so we always used Mermaid. Note that this relies on compresslevel being passed even when the containers aren't compressed.
	//
	FString ShaderOodleLevel;
	FParse::Value(FCommandLine::Get(), TEXT("compresslevel="), ShaderOodleLevel);
	if (ShaderOodleLevel.Len())
	{
		if (FOodleDataCompression::ECompressionLevelFromString(*ShaderOodleLevel, Arguments.ShaderOodleLevel))
		{
			UE_LOGF(LogIoStore, Display, "Selected Oodle level %d (%ls) from command line for shaders", (int)Arguments.ShaderOodleLevel, FOodleDataCompression::ECompressionLevelToString(Arguments.ShaderOodleLevel));
		}
		
	}

	// Whether or not to write compressed asset sizes back to the asset registry.

	FString WriteBackMetadataToAssetRegistry;
	if (FParse::Value(FCommandLine::Get(), TEXT("WriteBackMetadataToAssetRegistry="), WriteBackMetadataToAssetRegistry))
	{
		// StaticEnum not available in UnrealPak, so manual conversion:
		if (WriteBackMetadataToAssetRegistry.Equals(TEXT("AdjacentFile"), ESearchCase::IgnoreCase))
		{
			Arguments.WriteBackMetadataToAssetRegistry = EAssetRegistryWritebackMethod::AdjacentFile;
		}
		else if (WriteBackMetadataToAssetRegistry.Equals(TEXT("OriginalFile"), ESearchCase::IgnoreCase))
		{
			Arguments.WriteBackMetadataToAssetRegistry = EAssetRegistryWritebackMethod::OriginalFile;
		}
		else if (WriteBackMetadataToAssetRegistry.Equals(TEXT("Disabled"), ESearchCase::IgnoreCase))
		{
			Arguments.WriteBackMetadataToAssetRegistry = EAssetRegistryWritebackMethod::Disabled;
		}
		else
		{
			UE_LOGF(LogIoStore, Error, "Invalid WriteBackMetdataToAssetRegistry value: %ls - check setting in ProjectSettings -> Packaging", *WriteBackMetadataToAssetRegistry);
			UE_LOGF(LogIoStore, Error, "Valid options are: AdjacentFile, OriginalFile, Disabled.");
			return -1;
		}

		Arguments.bWritePluginSizeSummaryJsons = FParse::Param(FCommandLine::Get(), TEXT("WritePluginSizeSummaryJsons"));
	}

	FString PackageStoreManifestFilename;
	if (FParse::Value(FCommandLine::Get(), TEXT("PackageStoreManifest="), PackageStoreManifestFilename))
	{
		TUniquePtr<FCookedPackageStore> PackageStore = MakeUnique<FCookedPackageStore>(Arguments.CookedDir);
		FIoStatus Status = PackageStore->LoadManifest(*PackageStoreManifestFilename);
		if (Status.IsOk())
		{
			Arguments.PackageStore = MoveTemp(PackageStore);
		}
		else
		{
			UE_LOGF(LogIoStore, Fatal, "Failed loading package store manifest '%ls'", *PackageStoreManifestFilename);
		}
	}
	else
	{
		FString ProjectStoreFilename;
		if (FParse::Value(FCommandLine::Get(), TEXT("ProjectStore="), ProjectStoreFilename))
		{
			TUniquePtr<FCookedPackageStore> PackageStore = MakeUnique<FCookedPackageStore>(Arguments.CookedDir);
			FIoStatus Status = PackageStore->LoadProjectStore(*ProjectStoreFilename);
			if (Status.IsOk())
			{
				Arguments.PackageStore = MoveTemp(PackageStore);
			}
			else
			{
				UE_LOGF(LogIoStore, Fatal, "Failed loading project store '%ls'", *ProjectStoreFilename);
			}
		}
		else
		{
			UE_LOGF(LogIoStore, Error, "Expected -PackageStoreManifest=<path to package store manifest> or -ProjectStore=<path to project store marker>");
			return -1;
		}
	}

	if (!ParseOrderFileArguments(Arguments))
	{
		return -1;
	}

	FString ScriptObjectsFile;
	if (FParse::Value(FCommandLine::Get(), TEXT("ScriptObjects="), ScriptObjectsFile))
	{
		TArray<uint8> ScriptObjectsData;
		if (!FFileHelper::LoadFileToArray(ScriptObjectsData, *ScriptObjectsFile))
		{
			UE_LOGF(LogIoStore, Fatal, "Failed reading script objects file '%ls'", *ScriptObjectsFile);
		}
		Arguments.ScriptObjects = MakeUnique<FIoBuffer>(FIoBuffer::Clone, ScriptObjectsData.GetData(), ScriptObjectsData.Num());
	}
	else
	{
		UE_CLOGF(!Arguments.PackageStore->HasZenStoreClient(), LogIoStore, Fatal, "Expected -ScriptObjects=<path to script objects file> argument");
		TIoStatusOr<FIoBuffer> Status = Arguments.PackageStore->ReadChunk(CreateIoChunkId(0, 0, EIoChunkType::ScriptObjects));
		UE_CLOGF(!Status.IsOk(), LogIoStore, Fatal, "Failed reading script objects chunk '%ls'", *Status.Status().ToString());
		Arguments.ScriptObjects = MakeUnique<FIoBuffer>(Status.ConsumeValueOrDie());
	}

	{
		IOSTORE_CPU_SCOPE(FindCookedAssets);
		UE_LOGF(LogIoStore, Display, "Searching for cooked assets in folder '%ls'", *Arguments.CookedDir);
		FCookedFileVisitor CookedFileVistor(Arguments.CookedFileStatMap);
		IFileManager::Get().IterateDirectoryStatRecursively(*Arguments.CookedDir, CookedFileVistor);
		UE_LOGF(LogIoStore, Display, "Found '%d' files", Arguments.CookedFileStatMap.Num());
	}

	return CreateTarget(Arguments, WriterSettings);
}
