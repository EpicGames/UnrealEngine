// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreUtilitiesCore.h"
#include "Misc/Paths.h"


namespace UE::IoStore::Private
{

FCookedFileStatMap::FCookedFileStatMap()
{
	Extensions.Emplace(TEXT(".umap"), FCookedFileStatData::PackageHeader);
	Extensions.Emplace(TEXT(".uasset"), FCookedFileStatData::PackageHeader);
	Extensions.Emplace(TEXT(".uexp"), FCookedFileStatData::PackageData);
	Extensions.Emplace(TEXT(".ubulk"), FCookedFileStatData::BulkData);
	Extensions.Emplace(TEXT(".uptnl"), FCookedFileStatData::OptionalBulkData);
	Extensions.Emplace(TEXT(".m.ubulk"), FCookedFileStatData::MemoryMappedBulkData);
	Extensions.Emplace(*FString::Printf(TEXT(".uexp%s"), FFileRegion::RegionsFileExtension), FCookedFileStatData::Regions);
	Extensions.Emplace(*FString::Printf(TEXT(".uptnl%s"), FFileRegion::RegionsFileExtension), FCookedFileStatData::Regions);
	Extensions.Emplace(*FString::Printf(TEXT(".ubulk%s"), FFileRegion::RegionsFileExtension), FCookedFileStatData::Regions);
	Extensions.Emplace(*FString::Printf(TEXT(".m.ubulk%s"), FFileRegion::RegionsFileExtension), FCookedFileStatData::Regions);
	Extensions.Emplace(TEXT(".ushaderbytecode"), FCookedFileStatData::ShaderLibrary);
	Extensions.Emplace(TEXT(".o.umap"), FCookedFileStatData::OptionalSegmentPackageHeader);
	Extensions.Emplace(TEXT(".o.uasset"), FCookedFileStatData::OptionalSegmentPackageHeader);
	Extensions.Emplace(TEXT(".o.uexp"), FCookedFileStatData::OptionalSegmentPackageData);
	Extensions.Emplace(TEXT(".o.ubulk"), FCookedFileStatData::OptionalSegmentBulkData);
}

void FCookedFileStatMap::Add(const TCHAR* Path, int64 FileSize)
{
	FString NormalizedPath(Path);
	FPaths::NormalizeFilename(NormalizedPath);

	FStringView NormalizePathView(NormalizedPath);
	int32 ExtensionStartIndex = GetFullExtensionStartIndex(NormalizePathView);
	if (ExtensionStartIndex < 0)
	{
		return;
	}
	FStringView Extension = NormalizePathView.RightChop(ExtensionStartIndex);
	const FCookedFileStatData::EFileType* FindFileType = FindFileTypeFromExtension(Extension);
	if (!FindFileType)
	{
		return;
	}

	FCookedFileStatData& CookedFileStatData = Map.Add(MoveTemp(NormalizedPath));
	CookedFileStatData.FileType = *FindFileType;
	CookedFileStatData.FileSize = FileSize;
}

const FCookedFileStatData* FCookedFileStatMap::Find(FStringView NormalizedPath) const
{
	return Map.FindByHash(GetTypeHash(NormalizedPath), NormalizedPath);
}

const FCookedFileStatData::EFileType* FCookedFileStatMap::FindFileTypeFromExtension(const FStringView& Extension)
{
	for (const auto& Pair : Extensions)
	{
		if (Pair.Key == Extension)
		{
			return &Pair.Value;
		}
	}
	return nullptr;
}

uint32 GetTypeHash(const FIoStoreOrderingOptions& OrderingOptions)
{
	uint32 Hash = ::GetTypeHash(OrderingOptions.bClusterByOrderFilePriority);
	Hash = HashCombine(Hash, ::GetTypeHash(OrderingOptions.bAlphaSortClusterPackageLists));
	Hash = HashCombine(Hash, ::GetTypeHash(OrderingOptions.bPlaceShadersAtEnd));
	Hash = HashCombine(Hash, ::GetTypeHash(OrderingOptions.FallbackOrderMode));
	return Hash;
}

int32 GetFullExtensionStartIndex(FStringView Path)
{
	int32 ExtensionStartIndex = -1;
	for (int32 Index = Path.Len() - 1; Index >= 0; --Index)
	{
		// Check if we reached the end of the filename
		if (FPathViews::IsSeparator(Path[Index]))
		{
			break;
		}
		else if (Path[Index] == '.')
		{
			if (ExtensionStartIndex != -1)
			{
				// As we have already found a '.' we need to check for the cooked index
				FStringView Extension = Path.SubStr(Index + 1, (ExtensionStartIndex - Index) - 1);
				if (UE::String::IsNumericOnlyDigits(Extension))
				{
					return ExtensionStartIndex;
				}
			}

			ExtensionStartIndex = Index;
		}
	}
	return ExtensionStartIndex;
}

FStringView GetFullExtension(FStringView Path)
{
	int32 ExtensionStartIndex = GetFullExtensionStartIndex(Path);
	if (ExtensionStartIndex < 0)
	{
		return FStringView();
	}
	else
	{
		return Path.RightChop(ExtensionStartIndex);
	}
}

} // namespace UE::IoStore::Private
