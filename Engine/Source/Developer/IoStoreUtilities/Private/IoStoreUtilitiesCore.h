// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Compression/OodleDataCompression.h"
#include "CookedPackageStore.h"
#include "IO/IoContainerId.h"
#include "IO/IoContainerHeader.h"
#include "Hash/ShaderHash.h"
#include "IO/IoDispatcher.h"
#include "IoStoreWriter.h"
#include "Misc/KeyChainUtilities.h"
#include "PackageStoreOptimizer.h"
#include "Settings/ProjectPackagingSettings.h" // for EAssetRegistryWritebackMethod
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"


#define IOSTORE_CPU_SCOPE(NAME) TRACE_CPUPROFILER_EVENT_SCOPE(IoStore##NAME);
#define IOSTORE_CPU_SCOPE_DATA(NAME, DATA) TRACE_CPUPROFILER_EVENT_SCOPE(IoStore##NAME);

namespace UE::IoStore::Private
{

struct FContainerTargetSpec;

struct FContainerSourceFile
{
	FString NormalizedPath;
	FString DestinationPath;
	bool bNeedsCompression = false;
	bool bNeedsEncryption = false;
};

struct FContainerSourceSpec
{
	FName Name;
	FString OutputPath;
	FString OptionalOutputPath;
	FString StageLooseFileRootPath;
	TArray<FContainerSourceFile> SourceFiles;
	FString PatchTargetFile;
	TArray<FString> PatchSourceContainerFiles;
	FString EncryptionKeyOverrideGuid;
	bool bGenerateDiffPatch = false;
	bool bOnDemand = false;
};

struct FCookedFileStatData
{
	enum EFileType
	{
		PackageHeader,
		PackageData,
		BulkData,
		OptionalBulkData,
		MemoryMappedBulkData,
		ShaderLibrary,
		Regions,
		OptionalSegmentPackageHeader,
		OptionalSegmentPackageData,
		OptionalSegmentBulkData,
		Invalid
	};

	int64 FileSize = 0;
	EFileType FileType = Invalid;
};

class FCookedFileStatMap
{
public:
	FCookedFileStatMap();

	FORCEINLINE int32 Num() const
	{
		return Map.Num();
	}

	void Add(const TCHAR* Path, int64 FileSize);

	const FCookedFileStatData* Find(FStringView NormalizedPath) const;

private:
	const FCookedFileStatData::EFileType* FindFileTypeFromExtension(const FStringView& Extension);

	TArray<TTuple<FString, FCookedFileStatData::EFileType>> Extensions;
	TMap<FString, FCookedFileStatData> Map;
};

enum class EFallbackOrderMode
{
	LoadOrder, // Default fallback ordering. Uses LoadOrder dependency ordering
	AlphabeticalClustered, // As above but clusters are sorted alphabetically at the end for improved patching stability
	Alphabetical // Pure alphabetical fallback ordering. Best for patching stability but not optimized to reduce seeks
};

struct FIoStoreOrderingOptions
{
	bool bClusterByOrderFilePriority = false;
	bool bAlphaSortClusterPackageLists = false;
	bool bPlaceShadersAtEnd = false;

	EFallbackOrderMode FallbackOrderMode = EFallbackOrderMode::LoadOrder;

	bool operator==(const FIoStoreOrderingOptions& Other) const
	{
		return (bClusterByOrderFilePriority == Other.bClusterByOrderFilePriority) &&
			(bAlphaSortClusterPackageLists == Other.bAlphaSortClusterPackageLists) &&
			(bPlaceShadersAtEnd == Other.bPlaceShadersAtEnd) &&
			(FallbackOrderMode == Other.FallbackOrderMode);
	}
};

uint32 GetTypeHash(const FIoStoreOrderingOptions& OrderingOptions);

struct FReleasedPackages
{
	TSet<FName> PackageNames;
	TMap<FPackageId, FName> PackageIdToName;
};

struct FFileOrderMap
{
	TMap<FName, int64> PackageNameToOrder;
	FString Name;
	int32 Priority;
	int32 Index;

	FFileOrderMap(int32 InPriority, int32 InIndex)
		: Priority(InPriority)
		, Index(InIndex)
	{
	}
};

struct FIoStoreArguments
{
	FString GlobalContainerPath;
	FString CookedDir;
	TArray<FContainerSourceSpec> Containers;
	FCookedFileStatMap CookedFileStatMap;
	TArray<FFileOrderMap> OrderMaps;
	FKeyChain KeyChain;
	FKeyChain PatchKeyChain;
	FString DLCPluginPath;
	FString DLCName;
	FString ReferenceChunkGlobalContainerFileName;
	FString ReferenceChunkChangesCSVFileName;
	FString ReferenceChunkAdditionalContainersPath;
	FString PatchInPlaceReferenceContainerPathOverride;
	FString PatchInPlaceReferenceAdditionalContainersPath;
	FString CsvPath;
	FKeyChain ReferenceChunkKeys;
	FKeyChain PatchInPlaceReferenceChunkKeys;
	FReleasedPackages ReleasedPackages;
	TUniquePtr<FCookedPackageStore> PackageStore;
	TUniquePtr<FIoBuffer> ScriptObjects;
	FIoStoreOrderingOptions OrderingOptions;
	TMap<FString, FIoStoreOrderingOptions> OrderingOptionsOverrides;
	bool bSign = false;
	bool bRemapPluginContentToGame = false;
	bool bCreateDirectoryIndex = true;
	bool bFileRegions = false;
	EAssetRegistryWritebackMethod WriteBackMetadataToAssetRegistry = EAssetRegistryWritebackMethod::Disabled;
	bool bWritePluginSizeSummaryJsons = false; // Only valid if WriteBackMetadataToAssetRegistry != Disabled.

	FOodleDataCompression::ECompressor ShaderOodleCompressor = FOodleDataCompression::ECompressor::Mermaid;
	FOodleDataCompression::ECompressionLevel ShaderOodleLevel = FOodleDataCompression::ECompressionLevel::Normal;

	bool IsDLC() const
	{
		return DLCPluginPath.Len() > 0;
	}
};

// IoChunk per shader group
struct FShaderGroupIoChunk
{
	FIoChunkId ChunkId;
	FIoBuffer CodeIoBuffer;
	uint32 LoadOrderFactor = MAX_uint32; // the smaller the order, the more likely this will be loaded

	static bool Sort(const FShaderGroupIoChunk& A, const FShaderGroupIoChunk& B)
	{
		if (A.LoadOrderFactor == B.LoadOrderFactor)
		{
			// Shader chunk IDs are the hash of the shader so this is consistent across builds.
			return FMemory::Memcmp(A.ChunkId.GetData(), B.ChunkId.GetData(), A.ChunkId.GetSize()) < 0;
		}
		return A.LoadOrderFactor < B.LoadOrderFactor;
	}
};

struct FShaderInfo
{
	enum EShaderType { Normal, Global, Inline };

	FShaderGroupIoChunk ShaderGroupChunk;
	TSet<struct FCookedPackage*> ReferencedByPackages;
	TMap<FContainerTargetSpec*, EShaderType> TypeInContainer;

	// This is used to ensure build determinism and such must be stable across builds.
	static bool Sort(const FShaderInfo* A, const FShaderInfo* B)
	{
		return FShaderGroupIoChunk::Sort(A->ShaderGroupChunk, B->ShaderGroupChunk);
	}
};

struct FCookedPackage
{
	FPackageId GlobalPackageId;
	FName PackageName;
	FName SourcePackageName; // Source package name for redirected package
	TArray<FShaderInfo*> Shaders;
	TArray<FShaderHash> ShaderMapHashes;
	uint64 UAssetSize = 0;
	uint64 OptionalSegmentUAssetSize = 0;
	uint64 UExpSize = 0;
	uint64 TotalBulkDataSize = 0;
	uint64 LoadOrder = MAX_uint64; // Ordered by dependencies
	uint64 DiskLayoutOrder = MAX_uint64; // Final order after considering input order files
	uint64 OverrideDiskLayoutOrder = MAX_uint64; // Final order after applying custom per-container ordering
	FPackageStoreEntryResource PackageStoreEntry;
	int32 PreOrderNumber = -1; // Used for sorting in load order
	bool bPermanentMark = false; // Used for sorting in load order
	bool bIsLocalized = false;
};

struct FLegacyCookedPackage
	: public FCookedPackage
{
	FString FileName;
	FString OptionalSegmentFileName;
	FPackageStorePackage* OptimizedPackage = nullptr;
	FPackageStorePackage* OptimizedOptionalSegmentPackage = nullptr;
};

enum class EContainerChunkType
{
	ShaderCodeLibrary,
	ShaderCode,
	PackageData,
	BulkData,
	OptionalBulkData,
	MemoryMappedBulkData,
	OptionalSegmentPackageData,
	OptionalSegmentBulkData,
	Invalid
};

struct FContainerTargetFile
{
	FContainerTargetSpec* ContainerTarget = nullptr;
	FCookedPackage* Package = nullptr;
	FString NormalizedSourcePath;
	TOptional<FIoBuffer> SourceBuffer;
	FString DestinationPath;
	uint64 SourceSize = 0;
	uint64 IdealOrder = 0;
	FIoChunkId ChunkId;
	FIoHash ChunkHash;
	FBulkDataCookedIndex BulkDataCookedIndex;
	TArray<uint8> PackageHeaderData;
	EContainerChunkType ChunkType;
	bool bForceUncompressed = false;
	FIoChunkId DuplicateOfChunkId = {}; // when valid, is source byte identical to provided chunkid, including alignment requirements
	TArray<FFileRegion> FileRegions;
};

struct FPackageRedirectRequest
{
	FPackageId SourcePackageId;
	FPackageId TargetPackageId;
	FName      SourcePackageName;
};

struct FContainerTargetSpec
{
	FIoContainerId ContainerId;
	FIoContainerId OptionalSegmentContainerId;
	FIoContainerHeader Header;
	FIoContainerHeader OptionalSegmentHeader;
	FName Name;
	FGuid EncryptionKeyGuid;
	FString OutputPath;
	FString OptionalSegmentOutputPath;
	FString StageLooseFileRootPath;
	TSharedPtr<IIoStoreWriter> IoStoreWriter;
	TSharedPtr<IIoStoreWriter> OptionalSegmentIoStoreWriter;
	TArray<FContainerTargetFile> TargetFiles;
	TArray<TUniquePtr<FIoStoreReader>> PatchSourceReaders;
	EIoContainerFlags ContainerFlags = EIoContainerFlags::None;
	TArray<FCookedPackage*> Packages;
	TSet<FShaderInfo*> GlobalShaders;
	TSet<FShaderInfo*> SharedShaders;
	TSet<FShaderInfo*> UniqueShaders;
	TSet<FShaderInfo*> InlineShaders;
	TArray<FPackageRedirectRequest> PackageRedirects;
	TArray<FPackageRedirectRequest> OptionalSegmentPackageRedirects;
	bool bGenerateDiffPatch = false;
};

/**
 * Finds where the extension starts in a path.
 *
 * We can't just find the left most '.' in the file name because the path might
 * contain a bulkdata cooked index (@see FBulkDataCookedIndex) and we don't want
 * to include the index part. We are attempting to match up the extensions with
 * those found in the constructor of FCookedFileStatMap.
 *
 * For example take the following filenames and the extensions we want to find:
 * <packagename>.m.ubulk   -> .m.ubulk
 * <packagename>.o.ubulk   -> .o.ubulk
 * <packagename>.001.ubulk ->.ubulk
 *
 * So we want to find the left most '.' unless that is followed by only numeric
 * values in which case we have found a cooked index and we want to find the '.'
 * after those values.
 */
int32 GetFullExtensionStartIndex(FStringView Path);

FStringView GetFullExtension(FStringView Path);

} // namespace UE::IoStore::Private

