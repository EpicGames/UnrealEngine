// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreWriter.h"
#include "IO/IoStore.h"

#include "Async/Async.h"
#include "Async/AsyncFileHandle.h"
#include "Async/Future.h"
#include "Async/ParallelFor.h"
#include "Compression/CompressedBuffer.h"
#include "Compression/OodleDataCompression.h"
#include "Containers/Map.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataCacheRecord.h"
#include "DerivedDataRequestOwner.h"
#include "Algo/IsSorted.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoContainerMeta.h"
#include "IO/IoDirectoryIndex.h"
#include "Memory/SharedBuffer.h"
#include "Misc/Compression.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"
#include "Templates/Greater.h"

TRACE_DECLARE_MEMORY_COUNTER(IoStoreCompressionMemoryUsed, TEXT("IoStoreWriter/CompressionMemoryUsed"));
TRACE_DECLARE_MEMORY_COUNTER(IoStoreCompressionMemoryScheduled, TEXT("IoStoreWriter/CompressionMemoryScheduled"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreCompressionInflight, TEXT("IoStoreWriter/CompressionInflight"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreRefDbInflight, TEXT("IoStoreWriter/RefDbInFlight"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreRefDbDone, TEXT("IoStoreWriter/RefDbDone"));
TRACE_DECLARE_INT_COUNTER(IoStoreBeginCompressionCount, TEXT("IoStoreWriter/BeginCompression"));
TRACE_DECLARE_INT_COUNTER(IoStoreBeginEncryptionAndSigningCount, TEXT("IoStoreWriter/BeginEncryptionAndSigning"));
TRACE_DECLARE_INT_COUNTER(IoStoreBeginWriteCount, TEXT("IoStoreWriter/BeginWrite"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreDDCGetInflightCount, TEXT("IoStoreWriter/DDCGetInflightCount"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreDDCPutInflightCount, TEXT("IoStoreWriter/DDCPutInflightCount"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreDDCHitCount, TEXT("IoStoreWriter/DDCHitCount"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreDDCMissCount, TEXT("IoStoreWriter/DDCMissCount"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(IoStoreDDCPutCount, TEXT("IoStoreWriter/DDCPutCount"));

static UE::DerivedData::FCacheBucket IoStoreDDCBucket = UE::DerivedData::FCacheBucket(ANSITEXTVIEW("IoStoreCompression"));
static UE::DerivedData::ECachePolicy IoStoreDDCPolicy = UE::DerivedData::ECachePolicy::Default;
static FStringView IoStoreDDCVersion = TEXTVIEW("36EEC49B-E63B-498B-87D0-55FD11E4F9D6");

struct FChunkBlock
{
	const uint8* UncompressedData = nullptr;
	FIoBuffer* IoBuffer = nullptr;

	// This is the size of the actual block after encryption alignment, and is
	// set in EncryptAndSign. This happens whether or not the container is encrypted.
	uint64 DiskSize = 0;
	uint64 CompressedSize = 0;
	uint64 UncompressedSize = 0;
	FName CompressionMethod = NAME_None;
	FSHAHash Signature;

	/** Hash of the block data as it would be found on disk - this includes encryption alignment padding */
	FIoHash DiskHash;
};

struct FIoStoreWriteQueueEntry
{
	FIoStoreWriteQueueEntry* Next = nullptr;
	class FIoStoreWriter* Writer = nullptr;
	IIoStoreWriteRequest* Request = nullptr;
	FIoChunkId ChunkId;
	FIoHash ChunkHash;
	/** Hash of the block data as it would be found on disk after compression and encryption */
	FIoHash ChunkDiskHash;

	uint64 Sequence = 0;
	
	// We make this optional because at the latest it might not be valid until FinishCompressionBarrior
	// completes and we'd like to have a check() on that.
	TOptional<uint64> UncompressedSize;
	uint64 CompressedSize = 0; 

	// this is not filled out until after encryption completes and *includes the alignment padding for encryption*!
	uint64 DiskSize = 0; 

	uint64 Padding = 0;
	uint64 Offset = 0;
	TArray<FChunkBlock> ChunkBlocks;
	FIoWriteOptions Options;
	FName CompressionMethod = NAME_None;
	UE::Tasks::FTask HashTask;
	UE::Tasks::FTaskEvent BeginCompressionBarrier{ TEXT("BeginCompression") };
	UE::Tasks::FTaskEvent FinishCompressionBarrier{ TEXT("FinishCompression") };
	UE::Tasks::FTaskEvent BeginWriteBarrier{ TEXT("BeginWrite") };
	TAtomic<int32> CompressedBlocksCount{ 0 };
	int32 PartitionIndex = -1;
	int32 NumChunkBlocks = 0;
	UE::DerivedData::FCacheKey DDCKey;
	bool bAdded = false;
	bool bModified = false;
	bool bUseDDCForCompression = false;
	bool bFoundInDDC = false;
	bool bStoreCompressedDataInDDC = false;
	
	bool bCouldBeFromReferenceDb = false; // Whether the chunk is a valid candidate for the reference db.
	bool bLoadingFromReferenceDb = false;
};

struct FIoStoreDeferredDuplicate
{
	FIoChunkId ChunkId;
	FIoChunkId DuplicateOfChunkId;
	FString FileName;
	bool bIsMemoryMapped = false;
	bool bForceUncompressed = false;
};

class FIoStoreWriteQueue
{
public:
	FIoStoreWriteQueue()
		: Event(FPlatformProcess::GetSynchEventFromPool(false))
	{ }
	
	~FIoStoreWriteQueue()
	{
		check(Head == nullptr && Tail == nullptr);
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}

	void Enqueue(FIoStoreWriteQueueEntry* Entry)
	{
		check(!bIsDoneAdding);
		{
			FScopeLock _(&CriticalSection);

			if (!Tail)
			{
				Head = Tail = Entry;
			}
			else
			{
				Tail->Next = Entry;
				Tail = Entry;
			}
			Entry->Next = nullptr;
		}

		Event->Trigger();
	}

	FIoStoreWriteQueueEntry* DequeueOrWait()
	{
		for (;;)
		{
			{
				FScopeLock _(&CriticalSection);
				if (Head)
				{
					FIoStoreWriteQueueEntry* Entry = Head;
					Head = Tail = nullptr;
					return Entry;
				}
			}

			if (bIsDoneAdding)
			{
				break;
			}

			Event->Wait();
		}

		return nullptr;
	}

	void CompleteAdding()
	{
		bIsDoneAdding = true;
		Event->Trigger();
	}

	bool IsEmpty() const
	{
		FScopeLock _(&CriticalSection);
		return Head == nullptr;
	}

private:
	mutable FCriticalSection CriticalSection;
	FEvent* Event = nullptr;
	FIoStoreWriteQueueEntry* Head = nullptr;
	FIoStoreWriteQueueEntry* Tail = nullptr;
	TAtomic<bool> bIsDoneAdding { false };
};

struct FIoStoreDDCRequestDispatcherParams
{
	// Maximum time for filling up a batch, after this time limit is reached,
	// any queued requests are dispatched even if the batch is not full
	double QueueTimeLimitMs = 20.f;
	// Maximum number of (estimated) bytes in a batch,
	// when this is reached a batch with the currently queued requests will be dispatched immediately
	uint64 MaxBatchBytes = 16ull << 20;
	// Maximum number of (estimated) bytes for all inflight requests, 
	// if this limit is reached, then wait for requests to complete before dispatching a new batch
	uint64 MaxInflightBytes = 1ull << 30;
	// The number of queued requests to collect before dispatching a batch
	int32 MaxBatchItems = 8;
	// Maximum number of inflight requests,
	// if this limit is reached, then wait for requests to complete before dispatching a new batch
	int32 MaxInflightCount = 128;
	// Do a blocking wait after dispatching each batch (for debugging)
	bool bBlockingWait = false;
};

template<class T>
struct FIoStoreDDCRequestDispatcherQueue
{
	FIoStoreDDCRequestDispatcherQueue(const FIoStoreDDCRequestDispatcherParams& InParams)
		: Params(InParams)
		, RequestOwner(UE::DerivedData::EPriority::Highest)
	{ }

	FIoStoreDDCRequestDispatcherParams Params;
	UE::DerivedData::FRequestOwner RequestOwner;
	TArray<T> Requests;
	FEventRef RequestCompletedEvent;
	TAtomic<uint64> InflightCount{ 0 };
	TAtomic<uint64> InflightBytes{ 0 };
	uint64 QueuedBytes = 0;
	uint64 LastRequestCycle = 0;

	T& EnqueueRequest(uint64 Size)
	{
		if (Requests.Num() == 0)
		{
			LastRequestCycle = FPlatformTime::Cycles64();
		}
		QueuedBytes += Size;
		InflightBytes.AddExchange(Size);
		return Requests.AddDefaulted_GetRef();
	}

	bool ReadyOrWaitForDispatch(bool bForceDispatch);

	void OnDispatch()
	{
		QueuedBytes = 0;
		LastRequestCycle = FPlatformTime::Cycles64();
		InflightCount.AddExchange(Requests.Num());
		Requests.Reset();
		if (Params.bBlockingWait)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForDDC);
			RequestOwner.Wait();
		}
	}

	void OnRequestComplete(uint64 Size)
	{
		InflightCount.DecrementExchange();
		InflightBytes.SubExchange(Size);
		RequestCompletedEvent->Trigger();
	}
};

template<class T>
bool FIoStoreDDCRequestDispatcherQueue<T>::ReadyOrWaitForDispatch(bool bForceDispatch)
{
	int32 NumRequests = Requests.Num();
	if (NumRequests == 0)
	{
		return false;
	}

	bForceDispatch |= (NumRequests >= Params.MaxBatchItems) || (QueuedBytes >= Params.MaxBatchBytes);

	const bool bLazyDispatch = !bForceDispatch &&
		FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - LastRequestCycle) >= Params.QueueTimeLimitMs;

	if (!bForceDispatch && !bLazyDispatch)
	{
		return false;
	}

	int32 LocalInflightCount = InflightCount.Load();
	if (bForceDispatch)
	{
		while (LocalInflightCount > 0 && LocalInflightCount + NumRequests > Params.MaxInflightCount)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForDDCBatch);
			RequestCompletedEvent->Wait();
			LocalInflightCount = InflightCount.Load();
		}
		while (LocalInflightCount > 0 && InflightBytes.Load() + QueuedBytes > Params.MaxInflightBytes)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForDDCMemory);
			RequestCompletedEvent->Wait();
			LocalInflightCount = InflightCount.Load();
		}
	}
	else if (LocalInflightCount + NumRequests > Params.MaxInflightCount)
	{
		return false;
	}
	else if (InflightBytes.Load() + QueuedBytes > Params.MaxInflightBytes)
	{
		return false;
	}

	return true;
}

class FIoStoreDDCGetRequestDispatcher
{
public:
	FIoStoreDDCGetRequestDispatcher(const FIoStoreDDCRequestDispatcherParams& InParams) : RequestQueue(InParams) {};
	void EnqueueGetRequest(FIoStoreWriteQueueEntry* Entry);
	void DispatchGetRequests(
		TFunction<void (FIoStoreWriteQueueEntry* Entry, FSharedBuffer Result)> Callback,
		bool bForceDispatch = false);
	void FlushGetRequests(TFunction<void (FIoStoreWriteQueueEntry* Entry, FSharedBuffer Result)> Callback)
	{
		DispatchGetRequests(Callback, true);
		RequestQueue.RequestOwner.Wait();
	};

private:
	FIoStoreDDCRequestDispatcherQueue<UE::DerivedData::FCacheGetValueRequest> RequestQueue;
};

class FIoStoreDDCPutRequestDispatcher
{
public:
	FIoStoreDDCPutRequestDispatcher(const FIoStoreDDCRequestDispatcherParams& InParams) : RequestQueue(InParams) {};
	void EnqueuePutRequest(FIoStoreWriteQueueEntry* Entry, FSharedBuffer SharedBuffer);
	void DispatchPutRequests(
		TFunction<void (FIoStoreWriteQueueEntry* Entry, bool bSuccess)> Callback,
		bool bForceDispatch = false);
	void FlushPutRequests(TFunction<void (FIoStoreWriteQueueEntry* Entry, bool bSuccess)> Callback)
	{
		DispatchPutRequests(Callback, true);
		RequestQueue.RequestOwner.Wait();
	};

private:
	FIoStoreDDCRequestDispatcherQueue<UE::DerivedData::FCachePutValueRequest> RequestQueue;
};

void FIoStoreDDCGetRequestDispatcher::EnqueueGetRequest(FIoStoreWriteQueueEntry* Entry)
{
	UE::DerivedData::FCacheGetValueRequest& Request = RequestQueue.EnqueueRequest(Entry->Request->GetSourceBufferSizeEstimate());
	Request.Name = Entry->Options.FileName;
	Request.Key = Entry->DDCKey;
	Request.Policy = IoStoreDDCPolicy;
	Request.UserData = reinterpret_cast<uint64>(Entry);
}

void FIoStoreDDCGetRequestDispatcher::DispatchGetRequests(
	TFunction<void (FIoStoreWriteQueueEntry* Entry, FSharedBuffer Result)> Callback,
	bool bForceDispatch)
{
	using namespace UE::DerivedData;

	if (!RequestQueue.ReadyOrWaitForDispatch(bForceDispatch))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(DispatchDDCGetRequests);
	TRACE_COUNTER_ADD(IoStoreDDCGetInflightCount, RequestQueue.Requests.Num());

	{
		FRequestBarrier RequestBarrier(RequestQueue.RequestOwner);
		GetCache().GetValue(RequestQueue.Requests, RequestQueue.RequestOwner, [this, Callback](FCacheGetValueResponse&& Response)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ReadFromDDC_Decompress);
			uint64 SourceBufferSizeEstimate = 0;
			{
				FIoStoreWriteQueueEntry* Entry = reinterpret_cast<FIoStoreWriteQueueEntry*>(Response.UserData);
				SourceBufferSizeEstimate = Entry->Request->GetSourceBufferSizeEstimate();

				FSharedBuffer Result;
				if (Response.Status == EStatus::Ok)
				{
					Result = Response.Value.GetData().Decompress();
				}
				Callback(Entry, Result); // Entry could be deleted after this call
			}
			RequestQueue.OnRequestComplete(SourceBufferSizeEstimate);
			TRACE_COUNTER_DECREMENT(IoStoreDDCGetInflightCount);
		});
	}
	RequestQueue.OnDispatch();
}

void FIoStoreDDCPutRequestDispatcher::EnqueuePutRequest(FIoStoreWriteQueueEntry* Entry, FSharedBuffer SharedBuffer)
{
	FCompressedBuffer CompressedBuffer = FCompressedBuffer::Compress(
		SharedBuffer,
		ECompressedBufferCompressor::NotSet,
		ECompressedBufferCompressionLevel::None);

	UE::DerivedData::FCachePutValueRequest& Request = RequestQueue.EnqueueRequest(Entry->CompressedSize);
	Request.Name = Entry->Options.FileName;
	Request.Key = Entry->DDCKey;
	Request.Policy = IoStoreDDCPolicy;
	Request.Value = UE::DerivedData::FValue(MoveTemp(CompressedBuffer));
	Request.UserData = reinterpret_cast<uint64>(Entry);
}

void FIoStoreDDCPutRequestDispatcher::DispatchPutRequests(
	TFunction<void (FIoStoreWriteQueueEntry* Entry, bool bSuccess)> Callback,
	bool bForceDispatch)
{
	using namespace UE::DerivedData;
	
	if (!RequestQueue.ReadyOrWaitForDispatch(bForceDispatch))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(DispatchDDCPutRequests);
	TRACE_COUNTER_ADD(IoStoreDDCPutInflightCount, RequestQueue.Requests.Num());

	{
		FRequestBarrier RequestBarrier(RequestQueue.RequestOwner);
		GetCache().PutValue(RequestQueue.Requests, RequestQueue.RequestOwner, [this, Callback](FCachePutValueResponse&& Response)
		{
			uint64 CompressedSize = 0;
			{
				FIoStoreWriteQueueEntry* Entry = reinterpret_cast<FIoStoreWriteQueueEntry*>(Response.UserData);
				CompressedSize = Entry->CompressedSize;
				bool bSuccess = Response.Status == EStatus::Ok;
				Callback(Entry, bSuccess); // Entry could be deleted after this call
			}
			RequestQueue.OnRequestComplete(CompressedSize);
			TRACE_COUNTER_DECREMENT(IoStoreDDCPutInflightCount);
		});
	}
	RequestQueue.OnDispatch();
}

class FIoStoreWriterContextImpl
{
public:
	FIoStoreWriterContextImpl()
	{
	}

	~FIoStoreWriterContextImpl()
	{
		BeginCompressionQueue.CompleteAdding();
		BeginEncryptionAndSigningQueue.CompleteAdding();
		WriterQueue.CompleteAdding();
		BeginCompressionThread.Wait();
		BeginEncryptionAndSigningThread.Wait();
		WriterThread.Wait();
		for (FIoBuffer* IoBuffer : AvailableCompressionBuffers)
		{
			delete IoBuffer;
		}
	}

	[[nodiscard]] FIoStatus Initialize(const FIoStoreWriterSettings& InWriterSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreWriterContext::Initialize);
		WriterSettings = InWriterSettings;

		if (WriterSettings.bCompressionEnableDDC)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(InitializeDDC);
			UE_LOGF(LogIoStore, Display, "InitializeDDC");
			GetDerivedDataCacheRef();
			UE::DerivedData::GetCache();
		}

		if (WriterSettings.CompressionMethod != NAME_None)
		{
			CompressionBufferSize = FCompression::CompressMemoryBound(WriterSettings.CompressionMethod, static_cast<int32>(WriterSettings.CompressionBlockSize));
		}
		CompressionBufferSize = FMath::Max(CompressionBufferSize, WriterSettings.CompressionBlockSize);
		CompressionBufferSize = Align(CompressionBufferSize, FAES::AESBlockSize);

		MaxCompressionBufferMemory = 2ull << 30;
		FParse::Value(FCommandLine::Get(), TEXT("MaxCompressionBufferMemory="), MaxCompressionBufferMemory);

		const int32 InitialCompressionBufferCount = int32(MaxCompressionBufferMemory / CompressionBufferSize);
		AvailableCompressionBuffers.Reserve(InitialCompressionBufferCount);
		for (int32 BufferIndex = 0; BufferIndex < InitialCompressionBufferCount; ++BufferIndex)
		{
			AvailableCompressionBuffers.Add(new FIoBuffer(CompressionBufferSize));
		}

		return FIoStatus::Ok;
	}

	TSharedPtr<IIoStoreWriter> CreateContainer(const TCHAR* InContainerPathAndBaseFileName, const FIoContainerSettings& InContainerSettings);

	void FinalizeLayout();
	void FinalizeWrites();

	FIoStoreWriterContext::FProgress GetProgress() const
	{
		FIoStoreWriterContext::FProgress Progress;
		Progress.HashDbChunksCount = HashDbChunksCount.Load();
		for (uint8 i=0; i<(uint8)EIoChunkType::MAX; i++)
		{
			Progress.HashDbChunksByType[i] = HashDbChunksByType[i].Load();
			Progress.CompressedChunksByType[i] = CompressedChunksByType[i].Load();
			Progress.BeginCompressChunksByType[i] = BeginCompressChunksByType[i].Load();
			Progress.RefDbChunksByType[i] = RefDbChunksByType[i].Load();
			Progress.CompressionDDCHitsByType[i] = CompressionDDCHitsByType[i].Load();
			Progress.CompressionDDCPutsByType[i] = CompressionDDCPutsByType[i].Load();
			Progress.CompressionDDCHitCount += Progress.CompressionDDCHitsByType[i];
			Progress.CompressionDDCPutCount += Progress.CompressionDDCPutsByType[i];
		}

		Progress.TotalChunksCount = TotalChunksCount.Load();
		Progress.HashedChunksCount = HashedChunksCount.Load();
		Progress.CompressedChunksCount = CompressedChunksCount.Load();
		Progress.SerializedChunksCount = SerializedChunksCount.Load();
		Progress.ScheduledCompressionTasksCount = ScheduledCompressionTasksCount.Load();
		Progress.ScheduledCompressionMemoryBytes = ScheduledCompressionMemory.Load();
		Progress.CompressionDDCGetBytes = CompressionDDCGetBytes.Load();
		Progress.CompressionDDCPutBytes = CompressionDDCPutBytes.Load();
		Progress.CompressionDDCMissCount = CompressionDDCMissCount.Load();
		Progress.CompressionDDCPutErrorCount = CompressionDDCPutErrorCount.Load();
		Progress.RefDbChunksCount = RefDbChunksCount.Load();

		return Progress;
	}

	const FIoStoreWriterSettings& GetSettings() const
	{
		return WriterSettings;
	}

	FIoBuffer* AllocCompressionBuffer()
	{
		FIoBuffer* AllocatedBuffer = nullptr;
		{
			FScopeLock Lock(&AvailableCompressionBuffersCritical);
			if (AvailableCompressionBuffers.Num() > 0)
			{
				AllocatedBuffer = AvailableCompressionBuffers.Pop(EAllowShrinking::No);
			}
			TRACE_COUNTER_ADD(IoStoreCompressionMemoryUsed, CompressionBufferSize);
		}
		if (!AllocatedBuffer)
		{
			AllocatedBuffer = new FIoBuffer(CompressionBufferSize);
		}
		return AllocatedBuffer;
	}

	void FreeCompressionBuffer(FIoBuffer* Buffer)
	{
		FScopeLock Lock(&AvailableCompressionBuffersCritical);
		AvailableCompressionBuffers.Push(Buffer);
		TRACE_COUNTER_SUBTRACT(IoStoreCompressionMemoryUsed, CompressionBufferSize);
	}

	void ReportError()
	{
		ErrorCount.IncrementExchange();
	}

	uint32 GetErrors() const
	{
		return ErrorCount.Load(EMemoryOrder::Relaxed);
	}

	void AddContainerMeta(const FString& ContainerName, const FIoChunkId& ChunkId, const FString& Filename)
	{
		GlobalContainerMetaWriter.AddFile(ContainerName, ChunkId, Filename);
	}

	void SaveContainerMeta(const FString& FilePath)
	{
		if (!WriterSettings.bContainerMetaEnabled || WriterSettings.bPerContainerMeta)
		{
			return;
		}

		if (const int64 FileSize = GlobalContainerMetaWriter.Save(FilePath); FileSize > 0)
		{
			UE_LOGF(LogIoStore, Display, "Saved global metadata '%ls' (%.2lf KiB)", *FilePath, double(FileSize) / 1024.0);
		}
		else
		{
			UE_LOGF(LogIoStore, Warning, "Failed to save global metadata '%ls'", *FilePath);
		}
	}

private:
	UE::DerivedData::FCacheKey MakeDDCKey(FIoStoreWriteQueueEntry* Entry) const;

	void ScheduleAllEntries(TArrayView<FIoStoreWriteQueueEntry*> AllEntries);
	void BeginCompressionThreadFunc();
	void BeginEncryptionAndSigningThreadFunc();
	void WriterThreadFunc();

	FIoStoreWriterSettings WriterSettings;
	FEventRef CompressionMemoryReleasedEvent;
	TFuture<void> BeginCompressionThread;
	TFuture<void> BeginEncryptionAndSigningThread;
	TFuture<void> WriterThread;
	FIoStoreWriteQueue BeginCompressionQueue;
	FIoStoreWriteQueue BeginEncryptionAndSigningQueue;
	FIoStoreWriteQueue WriterQueue;
	TAtomic<uint64> TotalChunksCount{ 0 };
	TAtomic<uint64> HashedChunksCount{ 0 };
	TAtomic<uint64> HashDbChunksCount{ 0 };
	TAtomic<uint64> HashDbChunksByType[(int8)EIoChunkType::MAX] = {0};
	TAtomic<uint64> RefDbChunksCount{ 0 };
	TAtomic<uint64> RefDbChunksByType[(int8)EIoChunkType::MAX] = { 0 };
	TAtomic<uint64> CompressedChunksCount{ 0 };
	TAtomic<uint64> CompressedChunksByType[(int8)EIoChunkType::MAX] = { 0 };
	TAtomic<uint64> BeginCompressChunksByType[(int8)EIoChunkType::MAX] = { 0 };
	TAtomic<uint64> CompressionDDCHitsByType[(int8)EIoChunkType::MAX]{ 0 };
	TAtomic<uint64> CompressionDDCPutsByType[(int8)EIoChunkType::MAX]{ 0 };
	TAtomic<uint64> SerializedChunksCount{ 0 };
	TAtomic<uint64> WriteCycleCount{ 0 };
	TAtomic<uint64> WriteByteCount{ 0 };
	TAtomic<uint64> ScheduledCompressionTasksCount{ 0 };
	TAtomic<uint64> CompressionDDCGetBytes{ 0 };
	TAtomic<uint64> CompressionDDCPutBytes{ 0 };
	TAtomic<uint64> CompressionDDCMissCount{ 0 };
	TAtomic<uint64> CompressionDDCPutErrorCount{ 0 };
	TAtomic<uint64> ScheduledCompressionMemory{ 0 };
	TAtomic<int32> ErrorCount{ 0 };
	FCriticalSection AvailableCompressionBuffersCritical;
	TArray<FIoBuffer*> AvailableCompressionBuffers;
	uint64 MaxCompressionBufferMemory = 0;
	uint64 CompressionBufferSize = 0;
	TArray<TSharedPtr<FIoStoreWriter>> IoStoreWriters;
	FIoContainerMetaWriter GlobalContainerMetaWriter;

	friend class FIoStoreWriter;
};

FIoStoreWriterContext::FIoStoreWriterContext()
	: Impl(new FIoStoreWriterContextImpl())
{

}

FIoStoreWriterContext::~FIoStoreWriterContext()
{
	delete Impl;
}

[[nodiscard]] FIoStatus FIoStoreWriterContext::Initialize(const FIoStoreWriterSettings& InWriterSettings)
{
	return Impl->Initialize(InWriterSettings);
}

TSharedPtr<IIoStoreWriter> FIoStoreWriterContext::CreateContainer(const TCHAR* InContainerPath, const FIoContainerSettings& InContainerSettings)
{
	return Impl->CreateContainer(InContainerPath, InContainerSettings);
}

void FIoStoreWriterContext::FinalizeLayout()
{
	Impl->FinalizeLayout();
}

void FIoStoreWriterContext::FinalizeWrites()
{
	Impl->FinalizeWrites();
}

FIoStoreWriterContext::FProgress FIoStoreWriterContext::GetProgress() const
{
	return Impl->GetProgress();
}

uint32 FIoStoreWriterContext::GetErrors() const
{
	return Impl->GetErrors();
}

void FIoStoreWriterContext::SaveContainerMeta(const FString& FilePath)
{
	Impl->SaveContainerMeta(FilePath);
}

class FIoStoreTocBuilder
{
public:
	FIoStoreTocBuilder()
	{
		FMemory::Memzero(&Toc.Header, sizeof(FIoStoreTocHeader));
	}

	int32 AddChunkEntry(const FIoChunkId& ChunkId, const FIoOffsetAndLength& OffsetLength, const FIoStoreTocEntryMeta& Meta)
	{
		if (ChunkIdToIndex.Find(ChunkId))
		{
			return INDEX_NONE;
		}

		const int32 NewIndex = Toc.ChunkIds.Add(ChunkId);
		Toc.ChunkOffsetLengths.Add(OffsetLength);
		Toc.ChunkMetas.Add(Meta);
		ChunkIdToIndex.Add(ChunkId, NewIndex);
		return NewIndex;
	}

	FIoStoreTocCompressedBlockEntry& AddCompressionBlockEntry()
	{
		return Toc.CompressionBlocks.AddDefaulted_GetRef();
	}

	FSHAHash& AddBlockSignatureEntry()
	{
		return Toc.ChunkBlockSignatures.AddDefaulted_GetRef();
	}

	uint8 AddCompressionMethodEntry(FName CompressionMethod)
	{
		if (CompressionMethod == NAME_None)
		{
			return 0;
		}

		uint8 Index = 1;
		for (const FName& Name : Toc.CompressionMethods)
		{
			if (Name == CompressionMethod)
			{
				return Index;
			}
			++Index;
		}

		return 1 + uint8(Toc.CompressionMethods.Add(CompressionMethod));
	}

	void AddToFileIndex(const FIoChunkId& ChunkId, FString&& FileName)
	{
		ChunkIdToFileName.Emplace(ChunkId, MoveTemp(FileName));
	}

	FIoStoreTocResource& GetTocResource()
	{
		return Toc;
	}

	const FIoStoreTocResource& GetTocResource() const
	{
		return Toc;
	}

	const int32* GetTocEntryIndex(const FIoChunkId& ChunkId) const
	{
		return ChunkIdToIndex.Find(ChunkId);
	}

	void RebuildChunkIdToIndex()
	{
		ChunkIdToIndex.Reset();
		ChunkIdToIndex.Reserve(Toc.ChunkIds.Num());
		for (int32 Index = 0; Index < Toc.ChunkIds.Num(); ++Index)
		{
			ChunkIdToIndex.Add(Toc.ChunkIds[Index], Index);
		}
	}

	void GetFileNamesToIndex(TArray<FStringView>& OutFileNames) const
	{
		OutFileNames.Empty(ChunkIdToFileName.Num());
		for (auto& ChinkIdAndFileName : ChunkIdToFileName)
		{
			OutFileNames.Emplace(ChinkIdAndFileName.Value);
		}
	}

	const FString* GetFileName(const FIoChunkId& ChunkId) const
	{
		return ChunkIdToFileName.Find(ChunkId);
	}

	FIoStoreTocChunkInfo GetTocChunkInfo(int32 TocEntryIndex) const
	{
		FIoStoreTocChunkInfo ChunkInfo = Toc.GetTocChunkInfo(TocEntryIndex);

		ChunkInfo.FileName = FString::Printf(TEXT("<%s>"), *LexToString(ChunkInfo.ChunkType));
		ChunkInfo.bHasValidFileName = false;

		return ChunkInfo;
	}

private:
	FIoStoreTocResource Toc;
	TMap<FIoChunkId, int32> ChunkIdToIndex;
	TMap<FIoChunkId, FString> ChunkIdToFileName;
};

class FPatchInPlaceLayout
{
	static constexpr bool bParanoidChecks = false;

public:
	FPatchInPlaceLayout(
		const FString& InReferenceContainerPath,
		const FString& InSourceContainerPath,
		const FString& InTargetContainerPath,
		const FIoStoreTocResourceView& InSourceToc,
		const FIoStoreWriterSettings& InSettings
	)
		: ReferenceContainerPath(InReferenceContainerPath)
		, SourceContainerPath(InSourceContainerPath)
		, TargetContainerPath(InTargetContainerPath)
		, SourceToc(InSourceToc)
		, Settings(InSettings)
		, ReferenceToc()
	{
	}

	bool PrepareForLayout(
		const uint64 SizeBeforeLayout,
		uint64& OutPinnedSizeBeforeDefrag,
		int32& OutTargetPartitionNum,
		uint64& OutTotalPadding,
		int64& OutLocalityChange
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareForLayout);

		OutPinnedSizeBeforeDefrag = 0;
		OutTargetPartitionNum = 0;
		OutTotalPadding = 0;
		OutLocalityChange = 0;

		const FIoStatus ReferenceTocValid = FIoStoreTocResourceView::Read(*ReferenceContainerPath, EIoStoreTocReadOptions::ReadAll, ReferenceToc, ReferenceTocStorage);
		if (!ReferenceTocValid.IsOk())
		{
			return false;
		}

		BuildTocMap(ReferenceToc, ReferenceTocChunkIdToIndex);
		BuildTocMap(SourceToc, SourceTocChunkIdToIndex);

		// TODO this algo needs a 0%-100% progressive tuning
		// e.g. 0% would mean nothing is stays in place, 100% would mean everything stays in place
		// this is needed to let data move around to shrink containers between big releases

		TArray<FTocEntry> EntriesToKeep; // sorted by Offset increasing
		TArray<FTocEntry> EntriesToInsert; // sorted by OriginalOffset increasing
		PinChunksEntries(SourceToc, ReferenceToc, ReferenceTocChunkIdToIndex, EntriesToKeep, EntriesToInsert);

		// only do the layout if there are entries to be kept in place
		if (EntriesToKeep.IsEmpty())
		{
			return false;
		}

		OutPinnedSizeBeforeDefrag = 0;
		for (const FTocEntry& Entry : EntriesToKeep)
		{
			OutPinnedSizeBeforeDefrag += Entry.Size;
		}

		// we rely on Settings.MaxPartitionSize
		ensure(Settings.MaxPartitionSize == 0 || Settings.MaxPartitionSize == UINT64_MAX || Settings.MaxPartitionSize == SourceToc.Header.PartitionSize);

		const FBFDLayoutResult ResultBeforeDefrag = BestFitDecreasing(EntriesToKeep, EntriesToInsert, Settings);

		// attempt to defrag if container grew above size limit 
		if ((Settings.PatchInPlaceMaximumPerContainerGrowth != 0) &&
			(ResultBeforeDefrag.SizeAfter > SizeBeforeLayout + Settings.PatchInPlaceMaximumPerContainerGrowth))
		{
			TMap<FIoChunkId, int32> UnpinCandidates;
			BuildUnpinCandidates(UnpinCandidates, EntriesToKeep, EntriesToInsert, Settings);

			if (bParanoidChecks)
			{
				// unpinning all should result in valid pareto front that doesn't go over existing size
				TArray<FParetoPoint> ParetoFrontTemp;
				ProbeParetoFront(
					EntriesToKeep,
					EntriesToInsert,
					UnpinCandidates,
					SizeBeforeLayout,
					Settings,
					ParetoFrontTemp,
					UnpinCandidates.Num() - 1,
					UnpinCandidates.Num(),
					1);
				ensure(!ParetoFrontTemp.IsEmpty());
				ensure(ParetoFrontTemp.Last().UnpinCount == UnpinCandidates.Num());
				ensure(ParetoFrontTemp.Last().LayoutResult.SizeAfter <= SizeBeforeLayout);
			}

			TArray<FParetoPoint> ParetoFront;

			// probe whole unpin set with coarse steps to collect initial pareto front
			ProbeParetoFront(
				EntriesToKeep,
				EntriesToInsert,
				UnpinCandidates,
				SizeBeforeLayout + Settings.PatchInPlaceMaximumPerContainerGrowth,
				Settings,
				ParetoFront,
				0,
				UnpinCandidates.Num(),
				Settings.PatchInPlaceParetoFrontCoarseSamples);

			// refine pareto front if found, re-sample whole set otherwise
			ProbeParetoFront(
				EntriesToKeep,
				EntriesToInsert,
				UnpinCandidates,
				SizeBeforeLayout + Settings.PatchInPlaceMaximumPerContainerGrowth,
				Settings,
				ParetoFront,
				ParetoFront.Num() > 1 ? ParetoFront[0].UnpinCount : 0,
				ParetoFront.Num() > 1 ? ParetoFront.Last().UnpinCount : UnpinCandidates.Num(),
				Settings.PatchInPlaceParetoFrontFineSamples);

			// chose smallest unpin penalty size
			const FParetoPoint* BestPoint = Algo::MinElementBy(ParetoFront, [](const FParetoPoint& X){return X.LayoutResult.BytesWritten;});

			if (!BestPoint)
			{
				UE_LOGF(LogIoStore, Warning, "Patch-in-place '%ls' container grew above limit (%.2f MiB before pip -> %.2f MiB after pip), failed to fit into growth limit, disabling patch-in-place",
					*ReferenceContainerPath,
					(double)SizeBeforeLayout / 1024.0 / 1024.0,
					(double)ResultBeforeDefrag.SizeAfter / 1024.0 / 1024.0);
				return false;
			}

			// preserve EntriesToKeep order
			EntriesToKeep.RemoveAll([&EntriesToInsert, &UnpinCandidates, &BestPoint, this](const FTocEntry& Entry)
			{
				const int32* UnpinIndex = UnpinCandidates.Find(Entry.Id);
				if (!UnpinIndex || *UnpinIndex >= BestPoint->UnpinCount)
				{
					return false;
				}

				EntriesToInsert.Add(Entry.Unpin());
				return true;
			});

			Algo::SortBy(EntriesToInsert, &FTocEntry::OriginalOffset);

			// redo BFD on the new set
			const FBFDLayoutResult ResultAfterDefrag = BestFitDecreasing(EntriesToKeep, EntriesToInsert, Settings);

			UE_LOGF(LogIoStore, Display,
				"Patch-in-place '%ls' container grew above limit (%.2f MiB before pip -> %.2f MiB after pip), defragmented by unpinning %.2f MiB, new container size %.2f MiB",
				*ReferenceContainerPath,
				(double)SizeBeforeLayout / 1024.0 / 1024.0,
				(double)ResultBeforeDefrag.SizeAfter / 1024.0 / 1024.0,
				(double)((int64)ResultAfterDefrag.BytesWritten - (int64)ResultBeforeDefrag.BytesWritten) / 1024.0 / 1024.0,
				(double)ResultAfterDefrag.SizeAfter / 1024.0 / 1024.0
			);
		}

		// finalize TargetEntries
		TargetEntries = MoveTemp(EntriesToKeep);
		TargetEntries.Append(MoveTemp(EntriesToInsert));
		Algo::SortBy(TargetEntries, [](const FTocEntry& X) {return X.Offset;});
		if (bParanoidChecks)
		{
			EnsureSortedAndNonOverlapping(TargetEntries);
		}

		// finalize resulting values
		uint64 PreviousEntryEnd = 0;
		for (const FTocEntry& Entry : TargetEntries)
		{
			OutTargetPartitionNum = FMath::Max(Entry.PartitionIndexBegin(Settings.MaxPartitionSize) + 1, OutTargetPartitionNum);

			OutTotalPadding += Entry.Offset - PreviousEntryEnd;
			PreviousEntryEnd = Entry.End();

			// at this point no chunk shall cross partition boundary
			if (!ensure(Entry.PartitionIndexBegin(Settings.MaxPartitionSize) == Entry.PartitionIndexEnd(Settings.MaxPartitionSize)))
			{
				UE_LOGF(LogIoStore, Warning, "Patch-in-place '%ls' container failed to partition chunk '%ls', disabling patch-in-place",
					*ReferenceContainerPath,
					*LexToString(Entry.Id));
				return false;
			}

			if (Entry.bMemoryMapped && !IsAligned(Entry.Offset, Settings.MemoryMappingAlignment))
			{
				UE_LOGF(LogIoStore, Warning, "Patch-in-place '%ls' container failed to align chunk '%ls', disabling patch-in-place",
					*ReferenceContainerPath,
					*LexToString(Entry.Id));
				return false;
			}
		}

		uint64 LocalityBefore = 0;
		uint64 LocalityAfter = 0;
		CalculateLocalityChange(TargetEntries, Settings, LocalityBefore, LocalityAfter);
		OutLocalityChange = (int64)LocalityAfter - (int64)LocalityBefore;

		UE_LOGF(LogIoStore, Display, "Access locality for '%ls' with granularity %llu, original %.2f MiB, new %.2f MiB, diff %.2f MiB",
			*TargetContainerPath,
			Settings.PatchInPlaceReadGranularity,
			(double)LocalityBefore / 1024.0 / 1024.0,
			(double)LocalityAfter / 1024.0 / 1024.0,
			(double)OutLocalityChange / 1024.0 / 1024.0);

		return true;
	}

	void RunLayout(
		FIoStoreTocResource& TargetToc,
		const int32 TargetPartitionNum,
		uint64& OutActualPinnedSize
	)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PatchInPlaceLayout);

		UE_LOGF(LogIoStore, Display, "Patch-in-place '%ls' -> '%ls'", *ReferenceContainerPath, *TargetContainerPath);

		TArray<FFileOp> FileOps = {};
		int32 TargetPartitionNumCheck = 0;
		int32 SourcePartitionNum = 0;
		BuildTargetToc(TargetToc, FileOps, TargetEntries, SourcePartitionNum, TargetPartitionNumCheck);
		verify(TargetPartitionNumCheck == TargetPartitionNum);
		WriteChunksToTarget(TargetEntries, FileOps, SourcePartitionNum, TargetPartitionNum, OutActualPinnedSize);

		for (int32 Index = 0; Index < TargetPartitionNum; ++Index)
		{
			OverwriteFile(GetPartitionFilePath(SourceContainerPath, Index), GetPartitionFilePath(TargetContainerPath, Index));
		}
	}

private:
	const FString ReferenceContainerPath;
	const FString SourceContainerPath;
	const FString TargetContainerPath;
	const FIoStoreTocResourceView SourceToc;
	const FIoStoreWriterSettings& Settings;

	FIoStoreTocResourceView ReferenceToc;
	FIoStoreTocResourceStorage ReferenceTocStorage;

	TMap<FIoChunkId, int32> ReferenceTocChunkIdToIndex;
	TMap<FIoChunkId, int32> SourceTocChunkIdToIndex;

	struct FTocEntry;
	TArray<FTocEntry> TargetEntries; // sorted by Offset increasing

	static void BuildTocMap(const FIoStoreTocResourceView& Toc, TMap<FIoChunkId, int32>& OutResult)
	{
		OutResult.Reserve(Toc.ChunkIds.Num());
		for (int32 Index = 0; Index < Toc.ChunkIds.Num(); ++Index)
		{
			OutResult.Add(Toc.ChunkIds[Index], Index);
		}
	}

	struct FTocEntry
	{
		const FIoChunkId Id;
		uint64 Offset;
		const uint64 OriginalOffset;
		const uint64 Size;
		const bool bMemoryMapped;

		FTocEntry(FIoChunkId InId, uint64 InOffset, uint64 InOriginalOffset, uint64 InSize, const bool bInMemoryMapped)
			: Id(InId), Offset(InOffset), OriginalOffset(InOriginalOffset), Size(InSize), bMemoryMapped(bInMemoryMapped)
		{
		}

		uint64 OriginalEnd() const
		{
			return OriginalOffset + Size;
		}

		uint64 End() const
		{
			return Offset + Size;
		}

		int32 PartitionIndexBegin(const uint64 PartitionSize) const
		{
			return PartitionSize > 0 && PartitionSize < UINT64_MAX ? Offset / PartitionSize : 0;
		}

		int32 PartitionIndexEnd(const uint64 PartitionSize) const
		{
			return PartitionSize > 0 && PartitionSize < UINT64_MAX ? (Size > 0 ? Offset + Size - 1 : Offset) / PartitionSize : 0;
		}

		FTocEntry Unpin() const
		{
			return FTocEntry(Id, UINT64_MAX, OriginalOffset, Size, bMemoryMapped);
		}

		static FTocEntry Unpinned(const FIoStoreTocChunkInfo& ChunkInfo)
		{
			return FTocEntry(ChunkInfo.Id, UINT64_MAX, ChunkInfo.OffsetOnDisk, ChunkInfo.SizeOnDisk, ChunkInfo.bIsMemoryMapped);
		}

		static FTocEntry Pinned(const FIoStoreTocChunkInfo& ChunkInfo, const uint64 AtOffset)
		{
			return FTocEntry(ChunkInfo.Id, AtOffset, ChunkInfo.OffsetOnDisk, ChunkInfo.SizeOnDisk, ChunkInfo.bIsMemoryMapped);
		}
	};

	struct FInsertionChain
	{
		int32 Index;
		int32 Count;
		uint64 Size;

		FInsertionChain(const int32 InIndex, const uint64 InSize)
			: Index(InIndex), Count(1), Size(InSize)
		{
		}

		void Coalesce(const int32 InIndex, const uint64 InSize)
		{
			ensure(Index + Count == InIndex);
			Count++;
			Size += InSize;
		}

		uint64 CalculateEnd(uint64 AtOffset, const TConstArrayView<FTocEntry>& InEntriesToInsert, const uint64 MemoryMappingAlignment, const uint64 PartitionSize) const
		{
			for (int32 I = Index; I < Index + Count; ++I)
			{
				AtOffset = AlignOffset(AtOffset, InEntriesToInsert[I].Size, InEntriesToInsert[I].bMemoryMapped, MemoryMappingAlignment, PartitionSize);
				AtOffset += InEntriesToInsert[I].Size;
			}

			return AtOffset;
		}

		uint64 InsertEntries(uint64 AtOffset, const TArrayView<FTocEntry>& InEntriesToInsert, const uint64 MemoryMappingAlignment, const uint64 PartitionSize) const
		{
			for (int32 I = Index; I < Index + Count; ++I)
			{
				AtOffset = AlignOffset(AtOffset, InEntriesToInsert[I].Size, InEntriesToInsert[I].bMemoryMapped, MemoryMappingAlignment, PartitionSize);
				InEntriesToInsert[I].Offset = AtOffset;
				AtOffset = InEntriesToInsert[I].End();
			}

			return AtOffset;
		}
	};

	struct FLayoutGap
	{
		uint64 Begin; // inclusive, unaligned
		uint64 End; // exclusive
		bool bTail; // gap is in the tail after all pinned entries

		FLayoutGap(uint64 InBegin, uint64 InEnd, bool bInTail)
			: Begin(InBegin), End(InEnd), bTail(bInTail)
		{
		}

		uint64 Size() const
		{
			return End >= Begin ? End - Begin : 0;
		}

		bool Fits(const FTocEntry& Entry, const uint64 MemoryMappingAlignment, const uint64 PartitionSize) const
		{
			const uint64 NewBegin = AlignOffset(Begin, Entry.Size, Entry.bMemoryMapped, MemoryMappingAlignment, PartitionSize);
			return NewBegin + Entry.Size <= End;
		}

		uint64 Place(const FTocEntry& Entry, const uint64 MemoryMappingAlignment, const uint64 PartitionSize)
		{
			const uint64 NewBegin = AlignOffset(Begin, Entry.Size, Entry.bMemoryMapped, MemoryMappingAlignment, PartitionSize);
			Begin = NewBegin + Entry.Size;
			return NewBegin;
		}
	};

	struct FFileOp
	{
		const uint64 SourceOffset;
		const uint64 TargetOffset;
		uint64 Size;
		const bool bZeroFill;

		FFileOp(uint64 InSourceOffset, uint64 InTargetOffset, uint64 InSize, bool bInZeroFill)
			: SourceOffset(InSourceOffset)
			, TargetOffset(InTargetOffset)
			, Size(InSize)
			, bZeroFill(bInZeroFill)
		{
		}

		int32 SourcePartitionIndex(const uint64 PartitionSize) const
		{
			return PartitionSize > 0 && PartitionSize < UINT64_MAX ? SourceOffset / PartitionSize : 0;
		}

		int32 TargetPartitionIndex(const uint64 PartitionSize) const
		{
			return PartitionSize > 0 && PartitionSize < UINT64_MAX ? TargetOffset / PartitionSize : 0;
		}

		uint64 SourcePartitionOffset(const uint64 PartitionSize) const
		{
			return PartitionSize > 0 && PartitionSize < UINT64_MAX ? SourceOffset % PartitionSize : SourceOffset;
		}

		uint64 TargetPartitionOffset(const uint64 PartitionSize) const
		{
			return PartitionSize > 0 && PartitionSize < UINT64_MAX ? TargetOffset % PartitionSize : TargetOffset;
		}

		bool CanCoalesce(const uint64 SourceReadOffset, const uint64 TargetWriteOffset, const uint64 PartitionSize) const
		{
			return (SourceOffset + Size == SourceReadOffset) &&
				(TargetOffset + Size == TargetWriteOffset) &&
				(bZeroFill == false) &&
				(PartitionSize > 0 ? ( // only coalesce with-in same partition
					SourceOffset / PartitionSize == SourceReadOffset / PartitionSize &&
					TargetOffset / PartitionSize == TargetWriteOffset / PartitionSize
				) : true);
		}

		void Coalesce(const uint64 OtherSize)
		{
			Size += OtherSize;
		}
	};

	struct FBFDLayoutResult
	{
		uint64 BytesWritten;
		uint64 SizeAfter;

		FBFDLayoutResult(uint64 InBytesWritten, uint64 InSizeAfter)
			: BytesWritten(InBytesWritten)
			, SizeAfter(InSizeAfter)
		{
		}
	};

	struct FParetoPoint
	{
		int32 UnpinCount; // exclusive, unpin all elements in unpin array [0, UnpinCount)
		FBFDLayoutResult LayoutResult;

		FParetoPoint(int32 InUnpinCount, FBFDLayoutResult InLayouResult)
			: UnpinCount(InUnpinCount)
			, LayoutResult(InLayouResult)
		{
		}
		
		bool Dominates(const FParetoPoint& Other) const
		{
			return (LayoutResult.BytesWritten < Other.LayoutResult.BytesWritten &&
					LayoutResult.SizeAfter <= Other.LayoutResult.SizeAfter) ||
					(LayoutResult.BytesWritten <= Other.LayoutResult.BytesWritten &&
					LayoutResult.SizeAfter < Other.LayoutResult.SizeAfter);
		}
	};

	static void PinChunksEntries(
		const FIoStoreTocResourceView& SourceToc,
		const FIoStoreTocResourceView& ReferenceToc,
		const TMap<FIoChunkId, int32>& ReferenceTocChunkIdToIndex,
		TArray<FTocEntry>& OutEntriesToKeep,
		TArray<FTocEntry>& OutEntriesToInsert
	)
	{
		for (int32 ChunkIndex = 0; ChunkIndex < SourceToc.ChunkIds.Num(); ChunkIndex++)
		{
			const FIoStoreTocChunkInfo& SourceChunk = SourceToc.GetTocChunkInfo(ChunkIndex);

			const int32* ReferenceTocIndex = ReferenceTocChunkIdToIndex.Find(SourceChunk.Id);
			if (!ReferenceTocIndex)
			{
				OutEntriesToInsert.Add(FTocEntry::Unpinned(SourceChunk));
				continue;
			}

			const FIoStoreTocChunkInfo& ReferenceChunk = ReferenceToc.GetTocChunkInfo(*ReferenceTocIndex);
			if (SourceChunk.ChunkHash == ReferenceChunk.ChunkHash &&
				SourceChunk.SizeOnDisk == ReferenceChunk.SizeOnDisk)
			{
				OutEntriesToKeep.Add(FTocEntry::Pinned(SourceChunk, ReferenceChunk.OffsetOnDisk));
			}
			else
			{
				OutEntriesToInsert.Add(FTocEntry::Unpinned(SourceChunk));
			}
		}

		Algo::SortBy(OutEntriesToKeep, &FTocEntry::Offset);
		Algo::SortBy(OutEntriesToInsert, &FTocEntry::OriginalOffset);
	}

	static uint64 GapsBinarySearchProjection(const FLayoutGap& Gap)
	{
		return Gap.Size();
	}

	static uint64 AlignOffset(const uint64 Offset, const uint64 Size, const bool bMemoryMapped, const uint64 MemoryMappingAlignment, const uint64 PartitionSize)
	{
		uint64 Result = Offset;
		if (bMemoryMapped && MemoryMappingAlignment > 0)
		{
			Result = Align(Result, MemoryMappingAlignment);
		}
		if ((PartitionSize > 0) && (PartitionSize < UINT64_MAX) && (Size > 0) && (Result / PartitionSize) != ((Result + Size - 1) / PartitionSize))
		{
			Result = ((Result / PartitionSize) + 1) * PartitionSize;
			if (bMemoryMapped && MemoryMappingAlignment > 0) // align again if partition size is not a modulo of memory mapping alignment 
			{
				Result = Align(Result, MemoryMappingAlignment);
			}
		}
		return Result;
	}

	static void EnsureSortedAndNonOverlapping(TConstArrayView<FTocEntry> Entries, bool bUseOriginalOffset = false)
	{
		if (bUseOriginalOffset)
		{
			for (int32 Index = 1; Index < Entries.Num(); ++Index)
			{
				verify(Entries[Index].OriginalOffset > Entries[Index - 1].OriginalOffset);
			}

			for (int32 I = 0; I < Entries.Num(); ++I)
			{
				for (int32 J = 0; J < Entries.Num(); ++J)
				{
					if (I == J)
					{
						continue;
					}

					const uint64 L1 = Entries[I].OriginalOffset;
					const uint64 R1 = Entries[I].OriginalEnd() - 1;

					const uint64 L2 = Entries[J].OriginalOffset;
					const uint64 R2 = Entries[J].OriginalEnd() - 1;

					verify(R1 < L2 || L1 > R2);
				}
			}
		}
		else
		{
			for (int32 Index = 1; Index < Entries.Num(); ++Index)
			{
				verify(Entries[Index].Offset > Entries[Index - 1].Offset);
			}

			for (int32 I = 0; I < Entries.Num(); ++I)
			{
				for (int32 J = 0; J < Entries.Num(); ++J)
				{
					if (I == J)
					{
						continue;
					}

					const uint64 L1 = Entries[I].Offset;
					const uint64 R1 = Entries[I].End() - 1;

					const uint64 L2 = Entries[J].Offset;
					const uint64 R2 = Entries[J].End() - 1;

					verify(R1 < L2 || L1 > R2);
				}
			}
		}
	}

	static void BuildUnpinCandidates(
		TMap<FIoChunkId, int32>& OutUnpinCandidates, // chunk id -> index of all unpin candidates 
		const TConstArrayView<FTocEntry>& EntriesToKeep,
		const TConstArrayView<FTocEntry>& EntriesToInsert,
		const FIoStoreWriterSettings& Settings
	)
	{
		struct FUnpinCandidate
		{
			FIoChunkId Id;
			double Contribution;
		};

		TArray<FUnpinCandidate> UnpinCandidates;
		UnpinCandidates.Reserve(EntriesToKeep.Num());

		for (int32 i = 0; i < EntriesToKeep.Num(); ++i)
		{
			const FTocEntry& Entry = EntriesToKeep[i];

			// [previous entry] [gap before] [current entry] [gap after] [next entry]
			const uint64 PrevEnd = i > 0 ? EntriesToKeep[i - 1].End() : 0; // don't align
			const uint64 CurrBegin = Entry.Offset;
			const uint64 CurrEnd = Entry.End(); // don't align
			const uint64 NextBegin = i + 1 < EntriesToKeep.Num() ? EntriesToKeep[i + 1].Offset : CurrEnd;

			const uint64 GapBefore = CurrBegin > PrevEnd ? CurrBegin - PrevEnd : 0;
			const uint64 GapAfter  = NextBegin > CurrEnd ? NextBegin - CurrEnd : 0;
			const uint64 EntrySize = Entry.Size;

			double Contribution;
			switch (Settings.PatchInPlaceParetoFrontHeuristic)
			{
			default:
			case 0: // Prioritize based on how "useful" the gap for incoming dataset
				{
					FLayoutGap Gap(PrevEnd, NextBegin, false);
					uint64 UsableSpace = 0;

					for (const FTocEntry& Insert : EntriesToInsert)
					{
						if (!Gap.Fits(Insert, Settings.MemoryMappingAlignment, Settings.MaxPartitionSize))
						{
							continue;
						}

						UsableSpace += Insert.Size;
						Gap.Place(Insert, Settings.MemoryMappingAlignment, Settings.MaxPartitionSize);

						if (Gap.Size() == 0)
						{
							break;
						}
					}

					Contribution = (double)UsableSpace / (double)EntrySize;
					break;
				}
			case 1: // Prioritize most gap space regardless of entry size
				{
					Contribution = GapBefore + GapAfter;
					break;
				}
			case 2: // Prioritize freed up space per byte written
				{
					Contribution = (double)(GapBefore + GapAfter) / (double)EntrySize;
					break;
				}
			}

			UnpinCandidates.Add(FUnpinCandidate(Entry.Id, Contribution));
		}

		Algo::SortBy(UnpinCandidates, &FUnpinCandidate::Contribution, TGreater<double>());

		OutUnpinCandidates.Reserve(UnpinCandidates.Num());
		for (int32 i = 0; i < UnpinCandidates.Num(); i++)
		{
			OutUnpinCandidates.Add(UnpinCandidates[i].Id, i);
		}
	}

	static FBFDLayoutResult BestFitDecreasing(
		const TConstArrayView<FTocEntry>& EntriesToKeep,
		TArray<FTocEntry>& EntriesToInsert,
		const FIoStoreWriterSettings& Settings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BestFitDecreasing);

		if (bParanoidChecks)
		{
			EnsureSortedAndNonOverlapping(EntriesToKeep);
			EnsureSortedAndNonOverlapping(EntriesToInsert, true);
		}

		TArray<FLayoutGap> Gaps;
		for (int32 Index = 0; Index <= EntriesToKeep.Num(); ++Index)
		{
			// don't align offsets here, will be aligned later
			const uint64 PrevEnd = Index > 0 ? EntriesToKeep[Index - 1].End() : 0;
			const uint64 CurrentBegin = Index < EntriesToKeep.Num() ? EntriesToKeep[Index].Offset : UINT64_MAX;
			if (CurrentBegin > PrevEnd)
			{
				Gaps.Add(FLayoutGap(PrevEnd, CurrentBegin, Index == EntriesToKeep.Num()));
			}
		}

		Algo::SortBy(Gaps, GapsBinarySearchProjection);

		const uint64 MemoryMappingAlignment = Settings.MemoryMappingAlignment;
		const uint64 PartitionSize = Settings.MaxPartitionSize;
		const uint64 Granularity = Settings.PatchInPlaceReadGranularity;
		const uint64 MaxChainSize = Settings.PatchInPlaceMaxChainSize;
		const int32 MaxChainLength = Settings.PatchInPlaceMaxChainLength;

		TArray<FInsertionChain> Chains;
		for (int32 I = 0; I < EntriesToInsert.Num(); ++I)
		{
			const FTocEntry* Prev = I > 0 ? &EntriesToInsert[I - 1] : nullptr;
			const FTocEntry& Curr = EntriesToInsert[I];

			ensure(I == 0 || Prev->OriginalEnd() <= Curr.OriginalOffset);

			const bool bCanCoalesce = 
				Granularity > 0 &&
				I > 0 &&
				// heuristic, coalesce if previous end and current start were close enough to potentially end up in the same read block
				Curr.OriginalOffset - Prev->OriginalEnd() < Granularity &&
				(MaxChainLength == 0 || Chains.Last().Count < MaxChainLength) &&
				(MaxChainSize == 0 || Chains.Last().Size + Curr.Size <= MaxChainSize);

			if (bCanCoalesce)
			{
				Chains.Last().Coalesce(I, Curr.Size);
			}
			else
			{
				Chains.Add(FInsertionChain(I, Curr.Size));
			}
		}

		Algo::SortBy(Chains, &FInsertionChain::Size, TGreater<uint64>());

		uint64 BytesWritten = 0;

		// TODO support compression block alignment?
		for (const FInsertionChain& Chain : Chains)
		{
			BytesWritten += Chain.Size;

			// Binary search for first gap that fits by total size (lower bound, ignoring alignment padding)
			const int32 StartIndex = Algo::LowerBoundBy(Gaps, Chain.Size, GapsBinarySearchProjection);
			verify(StartIndex < Gaps.Num());

			// Linear scan for a gap that actually fits the whole chain with alignment of each element
			const int32 RelativeIndex = TConstArrayView<FLayoutGap>(Gaps)
				.RightChop(StartIndex)
				.IndexOfByPredicate([&](const FLayoutGap& Gap)
			{
				return Chain.CalculateEnd(Gap.Begin, EntriesToInsert, MemoryMappingAlignment, PartitionSize) <= Gap.End;
			});
			verify(RelativeIndex != INDEX_NONE);

			const int32 BestGapIndex = StartIndex + RelativeIndex;

			FLayoutGap Gap = Gaps[BestGapIndex];
			Gaps.RemoveAt(BestGapIndex, EAllowShrinking::No);
			Gap.Begin = Chain.InsertEntries(Gap.Begin, EntriesToInsert, MemoryMappingAlignment, PartitionSize);

			if (Gap.Size() > 0)
			{
				Gaps.Insert(Gap, Algo::UpperBoundBy(Gaps, Gap.Size(), GapsBinarySearchProjection));

				if (bParanoidChecks)
				{
					verify(Algo::IsSortedBy(Gaps, GapsBinarySearchProjection));
				}
			}
		}

		verify(Gaps.Last().bTail == true);
		return FBFDLayoutResult(BytesWritten, Gaps.Last().Begin);
	}

	// Simulates BFD by unpinning elements starting from [0, FromUnpinCount to [0, ToUnpinCount]
	// with sample count provided, collects pareto front from samples and sorts by unpin count.
	static void ProbeParetoFront(
		const TConstArrayView<FTocEntry>& EntriesToKeep,
		const TConstArrayView<FTocEntry>& EntriesToInsert,
		const TMap<FIoChunkId, int32>& UnpinCandidates,
		const uint64 MaxContainerSize,
		const FIoStoreWriterSettings& Settings,
		TArray<FParetoPoint>& ParetoFront,
		const int32 FromUnpinCount, // inclusive
		const int32 ToUnpinCount, // exclusive
		const int32 WithSampleCount
	)
	{
		const int32 IterationCount = FMath::DivideAndRoundUp(ToUnpinCount - FromUnpinCount, WithSampleCount);
		if (IterationCount == 0)
		{
			return;
		}

		TArray<FParetoPoint> Results;
		Results.SetNumUninitialized(IterationCount);

		ParallelFor(IterationCount, [&](int32 Index)
		{
			// ensure ToUnpinCount is sampled
			const int32 UnpinCount = (Index + 1 == IterationCount) ? ToUnpinCount : FMath::Lerp(FromUnpinCount, ToUnpinCount, Index / (double)(IterationCount - 1));

			TArray<FTocEntry> EntriesToInsertUnpinned;
			EntriesToInsertUnpinned.Reserve(EntriesToInsert.Num() + UnpinCount);
			EntriesToInsertUnpinned.Append(EntriesToInsert);
			TArray<FTocEntry> EntriesToKeepUnpinned = EntriesToKeep.FilterByPredicate([&](const FTocEntry& Entry)
			{
				const int32* UnpinIndex = UnpinCandidates.Find(Entry.Id);
				if (!UnpinIndex || *UnpinIndex >= UnpinCount)
				{
					return true;
				}

				EntriesToInsertUnpinned.Add(Entry.Unpin());
				return false;
			});

			Algo::SortBy(EntriesToInsertUnpinned, &FTocEntry::OriginalOffset);

			const FBFDLayoutResult Result = BestFitDecreasing(EntriesToKeepUnpinned, EntriesToInsertUnpinned, Settings);

			Results[Index] = FParetoPoint(UnpinCount, Result);
		});

		for (const FParetoPoint& Point : Results)
		{
			if (Point.LayoutResult.SizeAfter <= MaxContainerSize)
			{
				AddParetoPoint(ParetoFront, Point);
			}
		}

		Algo::SortBy(ParetoFront, [](const FParetoPoint& X){return X.UnpinCount;});
	}

	static bool AddParetoPoint(TArray<FParetoPoint>& ParetoFront, const FParetoPoint& NewPoint)
	{
		// Check if any points dominates the new point
		for (const FParetoPoint& CurrPoint : ParetoFront)
		{
			if (CurrPoint.Dominates(NewPoint))
			{
				return false;
			}
		}

		// Remove all points dominated by the new point
		ParetoFront.RemoveAll([&NewPoint](const FParetoPoint& CurrPoint)
		{
			return NewPoint.Dominates(CurrPoint);
		});

		ParetoFront.Add(NewPoint);
		return true;
	}

	// returns amount of bytes that are prefetched when reading all entries in their provided order
	static void CalculateLocalityChange(
		TConstArrayView<FTocEntry> Chunks,
		const FIoStoreWriterSettings& Settings,
		uint64& OutLocalityBefore,
		uint64& OutLocalityAfter
	)
	{
		OutLocalityBefore = 0;
		OutLocalityAfter = 0;

		const uint64 Granularity = Settings.PatchInPlaceReadGranularity;
		if (Chunks.Num() < 2 || Granularity == 0)
		{
			return;
		}

		// assuming everything is accessed as in OriginalOffset order
		TArray<int32> AccessOrder;
		AccessOrder.SetNumUninitialized(Chunks.Num());
		for (int32 I = 0; I < Chunks.Num(); ++I)
		{
			AccessOrder[I] = I;
		}
		Algo::SortBy(AccessOrder, [&Chunks](const int32 X){return Chunks[X].OriginalOffset;});

		for (int32 I = 1; I < AccessOrder.Num(); ++I)
		{
			const FTocEntry& Prev = Chunks[AccessOrder[I - 1]];
			const FTocEntry& Curr = Chunks[AccessOrder[I]];

			if (Prev.Size == 0)
			{
				continue;
			}

			const uint64 OriginalPrevEndPage = (Prev.OriginalEnd() - 1) / Granularity;
			const uint64 OriginalPrevNextPageOffset = (OriginalPrevEndPage + 1) * Granularity;
			const uint64 OriginalCurrBeginPage = Curr.OriginalOffset / Granularity;

			if (OriginalPrevEndPage == OriginalCurrBeginPage)
			{
				OutLocalityBefore += FMath::Min(Curr.Size, OriginalPrevNextPageOffset - Curr.OriginalOffset);
			}

			const uint64 PrevEndPage = (Prev.End() - 1) / Granularity;
			const uint64 PrevNextPageOffset = (PrevEndPage + 1) * Granularity;
			const uint64 CurrBeginPage = Curr.Offset / Granularity;

			if (PrevEndPage == CurrBeginPage)
			{
				OutLocalityAfter += FMath::Min(Curr.Size, PrevNextPageOffset - Curr.Offset);
			}
		}
	}

	// emit zero fill of a provided region, splitting ops between partitions
	static void ZeroFillRegion(TArray<FFileOp>& OutOps, const uint64 Begin, const uint64 End, const uint64 MaxPartitionSize)
	{
		if (Begin >= End)
		{
			return;
		}

		if (MaxPartitionSize == 0 || MaxPartitionSize == UINT64_MAX)
		{
			OutOps.Add(FFileOp(0, Begin, End - Begin, true));
			return;
		}

		for (uint64 Position = Begin; Position < End;)
		{
			const uint64 NextPartitionBegin = ((Position / MaxPartitionSize) + 1) * MaxPartitionSize;
			const uint64 OpEnd = FMath::Min(NextPartitionBegin, End);
			OutOps.Add(FFileOp(0, Position, OpEnd - Position, true));
			Position = OpEnd;
		}
	}

	void BuildTargetToc(FIoStoreTocResource& TargetToc, TArray<FFileOp>& OutOps, TConstArrayView<FTocEntry> Chunks, int32& OutSourcePartitionNum, int32& OutTargetPartitionNum)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(BuildTargetToc);

		if (Chunks.Num() != SourceToc.ChunkIds.Num())
		{
			UE_LOGF(LogIoStore, Error, "Broken ToC of '%ls'", *SourceContainerPath);
			return;
		}

		const int32 ChunksNum = Chunks.Num();

		TargetToc.ChunkIds.SetNum(ChunksNum);
		TargetToc.ChunkOffsetLengths.SetNum(ChunksNum);
		TargetToc.ChunkMetas.SetNum(ChunksNum);
		TargetToc.CompressionBlocks.SetNum(SourceToc.CompressionBlocks.Num());
		TargetToc.ChunkBlockSignatures.SetNum(SourceToc.ChunkBlockSignatures.Num());
		TargetToc.CompressionMethods = SourceToc.CompressionMethods;
		TargetToc.DirectoryIndexBuffer = SourceToc.DirectoryIndexBuffer;

		const uint32 CompressionBlockSize = SourceToc.Header.CompressionBlockSize;
		const uint64 MaxPartitionSize = SourceToc.Header.PartitionSize;

		uint64 UncompressedOffset = 0;
		uint64 TargetOffset = 0;

		int32 TargetChunkIndex = 0;
		int32 TargetBlockIndex = 0;

		OutOps.Reserve(Chunks.Num() * 2); // likely have same amount of gaps

		int32 SourcePartitionNum = 0;
		int32 TargetPartitionNum = 0;

		const uint64 MaxPossibleOffset = FIoStoreTocCompressedBlockEntry::OffsetMask;

		for (const FTocEntry& Chunk : Chunks)
		{
			if (!ensure(SourceTocChunkIdToIndex.Contains(Chunk.Id)))
			{
				continue;
			}

			if (Chunk.Offset > TargetOffset)
			{
				// padding is defined as zero filled given expected small amount of padding bytes 
				ZeroFillRegion(OutOps, TargetOffset, Chunk.Offset, MaxPartitionSize);
			}

			const int32 SourceIndex = *SourceTocChunkIdToIndex.Find(Chunk.Id);

			TargetToc.ChunkIds[TargetChunkIndex] = Chunk.Id;
			TargetToc.ChunkMetas[TargetChunkIndex] = SourceToc.ChunkMetas[SourceIndex];

			const uint64 SourceOffset = SourceToc.ChunkOffsetLengths[SourceIndex].GetOffset();
			const uint64 SourceSize = SourceToc.ChunkOffsetLengths[SourceIndex].GetLength();
			const int32 SourceFirstBlockIndex = int32(SourceOffset / CompressionBlockSize);
			const int32 SourceLastBlockIndex = int32((Align(SourceOffset + SourceSize, CompressionBlockSize) - 1) / CompressionBlockSize);

			const uint64 SourceReadOffset = SourceToc.CompressionBlocks[SourceFirstBlockIndex].GetOffset();
			uint64 UncompressedSize = 0;
			uint64 DiskSize = 0;

			for (int32 SourceBlockIndex = SourceFirstBlockIndex; SourceBlockIndex <= SourceLastBlockIndex; ++SourceBlockIndex, ++TargetBlockIndex)
			{
				const FIoStoreTocCompressedBlockEntry& SourceBlock = SourceToc.CompressionBlocks[SourceBlockIndex];
				FIoStoreTocCompressedBlockEntry& TargetBlock = TargetToc.CompressionBlocks[TargetBlockIndex];

				const uint64 TargetBlockOffset = Chunk.Offset + DiskSize;
				verify(TargetBlockOffset <= MaxPossibleOffset);

				TargetBlock.SetOffset(Chunk.Offset + DiskSize); // this is disk offset between all partitions
				TargetBlock.SetCompressedSize(SourceBlock.GetCompressedSize());
				TargetBlock.SetUncompressedSize(SourceBlock.GetUncompressedSize());
				TargetBlock.SetCompressionMethodIndex(SourceBlock.GetCompressionMethodIndex());

				if (SourceBlockIndex < SourceToc.ChunkBlockSignatures.Num())
				{
					TargetToc.ChunkBlockSignatures[TargetBlockIndex] = SourceToc.ChunkBlockSignatures[SourceBlockIndex];
				}

				UncompressedSize += SourceBlock.GetUncompressedSize();
				DiskSize += Align(SourceBlock.GetCompressedSize(), FAES::AESBlockSize);
			}

			verify(UncompressedOffset <= MaxPossibleOffset);
			verify(UncompressedSize <= MaxPossibleOffset);
			verify(DiskSize == Chunk.Size);

			TargetToc.ChunkOffsetLengths[TargetChunkIndex].SetOffset(UncompressedOffset);
			TargetToc.ChunkOffsetLengths[TargetChunkIndex].SetLength(UncompressedSize);

			// try to coalesce with previous operation
			if (OutOps.Num() > 0 && OutOps.Last().CanCoalesce(SourceReadOffset, Chunk.Offset, MaxPartitionSize))
			{
				OutOps.Last().Coalesce(Chunk.Size);
			}
			else
			{
				const FFileOp& Op = OutOps.Add_GetRef(FFileOp(SourceReadOffset, Chunk.Offset, Chunk.Size, false));
				SourcePartitionNum = FMath::Max(Op.SourcePartitionIndex(MaxPartitionSize) + 1, SourcePartitionNum);
				TargetPartitionNum = FMath::Max(Op.TargetPartitionIndex(MaxPartitionSize) + 1, TargetPartitionNum);
			}

			TargetOffset = Chunk.Offset + Chunk.Size;

			UncompressedOffset += (SourceLastBlockIndex - SourceFirstBlockIndex + 1) * CompressionBlockSize;
			TargetChunkIndex++;
		}

		Algo::SortBy(OutOps, &FFileOp::TargetOffset);

		OutSourcePartitionNum = SourcePartitionNum;
		OutTargetPartitionNum = TargetPartitionNum;

		if (bParanoidChecks && !Chunks.IsEmpty())
		{
			// copy ops should write to every single byte in all partitions and not overlap
			uint64 ExpectedOffset = 0;
			for (const FFileOp& Op : OutOps)
			{
				verify(Op.TargetOffset == ExpectedOffset);
				verify(Op.Size > 0);
				ExpectedOffset = Op.TargetOffset + Op.Size;
			}

			const uint64 ExpectedTotalSize = Chunks.Last().Offset + Chunks.Last().Size;
			verify(ExpectedOffset == ExpectedTotalSize);
		}
	}

	void WriteChunksToTarget(TConstArrayView<FTocEntry> Chunks, TConstArrayView<FFileOp> FileOps, const int32 SourcePartitionNum, const int32 TargetPartitionNum, uint64& OutActualPinnedSize)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WriteChunksToTarget);

		OutActualPinnedSize = 0;

		if (Chunks.IsEmpty() || SourcePartitionNum == 0 || TargetPartitionNum == 0 || FileOps.IsEmpty())
		{
			return;
		}

		const uint64 MaxPartitionSize = SourceToc.Header.PartitionSize;
		const uint64 LastPartitionSizeLogical = Chunks.Last().Offset + Chunks.Last().Size;
		if (LastPartitionSizeLogical == 0)
		{
			return;
		}

		const uint64 LastPartitionSize =
			MaxPartitionSize > 0
			? (LastPartitionSizeLogical % MaxPartitionSize == 0
				? MaxPartitionSize // edge case, LastPartitionSizeLogical precisely matches partition size
				: LastPartitionSizeLogical % MaxPartitionSize)
			: LastPartitionSizeLogical;
		ensure(LastPartitionSize != 0);

		IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
		Ipf.CreateDirectoryTree(*FPaths::GetPath(TargetContainerPath));

		TArray<TUniquePtr<IFileHandle>> SourcePartitions;
		SourcePartitions.SetNumZeroed(SourcePartitionNum);
		for (int32 I = 0; I < SourcePartitionNum; ++I)
		{
			const FString PartitionFilePath = GetPartitionFilePath(SourceContainerPath, I);
			FFileOpenResult Result = Ipf.OpenRead(*PartitionFilePath, IPlatformFile::EOpenReadFlags::None);
			if (!Result.IsValid())
			{
				UE_LOGF(LogIoStore, Fatal, "Failed to read partition '%ls'", *PartitionFilePath);
				return;
			}
			SourcePartitions[I] = MoveTemp(Result.GetValue());
		}

		TArray<TUniquePtr<IFileHandle>> TargetPartitions;
		TargetPartitions.SetNumZeroed(TargetPartitionNum);
		for (int32 I = 0; I < TargetPartitionNum; ++I)
		{
			const FString PartitionFilePath = GetPartitionFilePath(TargetContainerPath, I);
			FFileOpenResult Result = Ipf.OpenWrite(*PartitionFilePath, IPlatformFile::EOpenWriteFlags::AllowRead | IPlatformFile::EOpenWriteFlags::Append);
			if (!Result.IsValid())
			{
				UE_LOGF(LogIoStore, Fatal, "Failed to write partition '%ls'", *PartitionFilePath);
				return;
			}
			TargetPartitions[I] = MoveTemp(Result.GetValue());
			if (!TargetPartitions[I]->Truncate(I + 1 < TargetPartitionNum ? MaxPartitionSize : LastPartitionSize))
			{
				UE_LOGF(LogIoStore, Error, "Failed to resize partition '%ls'", *PartitionFilePath);
			}
		}

		constexpr uint64 TempBufferSize = 4 * 1024 * 1024;
		FIoBuffer TempBuffer(TempBufferSize);

		for (const FFileOp& Op : FileOps)
		{
			IFileHandle* Source = Op.bZeroFill ? nullptr : SourcePartitions[Op.SourcePartitionIndex(MaxPartitionSize)].Get();
			IFileHandle* Target = TargetPartitions[Op.TargetPartitionIndex(MaxPartitionSize)].Get();

			const uint64 SourceOffset = Op.SourcePartitionOffset(MaxPartitionSize);
			const uint64 TargetOffset = Op.TargetPartitionOffset(MaxPartitionSize);

			for (uint64 Offset = 0; Offset < Op.Size; Offset += TempBufferSize)
			{
				const uint64 Size = FMath::Min(Op.Size - Offset, TempBufferSize);

				if (bParanoidChecks)
				{
					if (!Op.bZeroFill)
					{
						verify(SourceOffset + Offset + Size <= (uint64)Source->Size());
					}
					verify(TargetOffset + Offset + Size <= (uint64)Target->Size());
				}

				if (Op.bZeroFill)
				{
					FMemory::Memzero(TempBuffer.GetData(), Size);
				}
				else
				{
					Source->ReadAt(TempBuffer.GetData(), Size, SourceOffset + Offset);
				}

				Target->Seek(TargetOffset + Offset);
				Target->Write(TempBuffer.GetData(), Size);
			}
		}

		if (bParanoidChecks)
		{
			FIoBuffer TempBuffer2(TempBufferSize);

			for (const FFileOp& Op : FileOps)
			{
				IFileHandle* Source = Op.bZeroFill ? nullptr : SourcePartitions[Op.SourcePartitionIndex(MaxPartitionSize)].Get();
				IFileHandle* Target = TargetPartitions[Op.TargetPartitionIndex(MaxPartitionSize)].Get();

				const uint64 SourceOffset = Op.SourcePartitionOffset(MaxPartitionSize);
				const uint64 TargetOffset = Op.TargetPartitionOffset(MaxPartitionSize);

				for (uint64 Offset = 0; Offset < Op.Size; Offset += TempBufferSize)
				{
					const uint64 Size = FMath::Min(Op.Size - Offset, TempBufferSize);

					if (Op.bZeroFill)
					{
						FMemory::Memzero(TempBuffer.GetData(), Size);
					}
					else
					{
						Source->ReadAt(TempBuffer.GetData(), Size, SourceOffset + Offset);
					}
					Target->ReadAt(TempBuffer2.GetData(), Size, TargetOffset + Offset);

					verify(FMemory::Memcmp(TempBuffer.GetData(), TempBuffer2.GetData(), Size) == 0);
				}
			}
		}

		// verifying new container against the reference to ensure correct pinned byte count
		TRACE_CPUPROFILER_EVENT_SCOPE(VerifyPinnedChunks);

		const uint64 ReferencePartitionSize = ReferenceToc.Header.PartitionSize;

		TArray<TUniquePtr<IFileHandle>> ReferencePartitions;
		ReferencePartitions.SetNum((int32)ReferenceToc.Header.PartitionCount);
		for (int32 I = 0; I < ReferencePartitions.Num(); ++I)
		{
			const FString Path = GetPartitionFilePath(ReferenceContainerPath, I);
			FFileOpenResult Result = Ipf.OpenRead(*Path, IPlatformFile::EOpenReadFlags::None);
			if (Result.IsValid())
			{
				ReferencePartitions[I] = MoveTemp(Result.GetValue());
			}
			else
			{
				UE_LOGF(LogIoStore, Warning,
					"Patch-in-place verify '%ls': failed to read '%ls', PIP pinned bytes might be wrong",
					*SourceContainerPath,
					*Path);
				break;
			}
		}

		FIoBuffer ReferenceReadBuffer(TempBufferSize);
		FIoBuffer TargetReadBuffer(TempBufferSize);

		uint64 PinnedChunksMismatched = 0;

		for (const FTocEntry& Chunk : Chunks)
		{
			const int32* ReferenceIndex = ReferenceTocChunkIdToIndex.Find(Chunk.Id);
			if (!ReferenceIndex || !ensure(SourceTocChunkIdToIndex.Contains(Chunk.Id)))
			{
				continue;
			}

			const FIoStoreTocChunkInfo& ReferenceChunk = ReferenceToc.GetTocChunkInfo(*ReferenceIndex);
			const FIoStoreTocChunkInfo& SourceChunk = SourceToc.GetTocChunkInfo(*SourceTocChunkIdToIndex.Find(Chunk.Id));

			if (Chunk.Offset != ReferenceChunk.OffsetOnDisk ||
				Chunk.Size != ReferenceChunk.SizeOnDisk ||
				SourceChunk.ChunkHash != ReferenceChunk.ChunkHash)
			{
				continue;
			}

			const int32 TargetPartitionIndex = Chunk.PartitionIndexBegin(MaxPartitionSize);
			const uint64 TargetPartitionOffset = MaxPartitionSize > 0 && MaxPartitionSize < UINT64_MAX ? Chunk.Offset % MaxPartitionSize : Chunk.Offset;
			const int32 ReferencePartitionIndex = ReferencePartitionSize > 0 && ReferencePartitionSize < UINT64_MAX ? ReferenceChunk.OffsetOnDisk / ReferencePartitionSize : 0;
			const uint64 ReferencePartitionOffset = ReferencePartitionSize > 0 && ReferencePartitionSize < UINT64_MAX ? ReferenceChunk.OffsetOnDisk % ReferencePartitionSize : ReferenceChunk.OffsetOnDisk;

			if (!ensure(TargetPartitionIndex < TargetPartitions.Num()))
			{
				continue;
			}

			if (ReferencePartitionIndex >= ReferencePartitions.Num() || !ReferencePartitions[ReferencePartitionIndex].IsValid())
			{
				// assume it's valid as we can't compare the bytes against the reference
				OutActualPinnedSize += Chunk.Size;
				continue;
			}

			bool bMatch = true;
			for (uint64 Offset = 0; Offset < Chunk.Size; Offset += TempBufferSize)
			{
				const uint64 ReadSize = FMath::Min(Chunk.Size - Offset, TempBufferSize);

				TargetPartitions[TargetPartitionIndex]->ReadAt(TargetReadBuffer.GetData(), ReadSize, TargetPartitionOffset + Offset);
				ReferencePartitions[ReferencePartitionIndex]->ReadAt(ReferenceReadBuffer.GetData(), ReadSize, ReferencePartitionOffset + Offset);

				if (FMemory::Memcmp(TargetReadBuffer.GetData(), ReferenceReadBuffer.GetData(), ReadSize) != 0)
				{
					bMatch = false;
					break;
				}
			}

			if (bMatch)
			{
				OutActualPinnedSize += Chunk.Size;
			}
			else
			{
				PinnedChunksMismatched++;
			}
		}

		if (PinnedChunksMismatched > 0)
		{
			UE_LOGF(LogIoStore, Display,
				"Patch-in-place verify '%ls': %llu of pinned chunks didn't binary-match reference",
				*SourceContainerPath,
				PinnedChunksMismatched);
		}
	}

	static FString GetPartitionFilePath(FString ContainerFilePath, int32 PartitionIndex)
	{
		FString BasePath = FPaths::GetBaseFilename(ContainerFilePath, false);
		if (PartitionIndex > 0)
		{
			BasePath += FString::Printf(TEXT("_s%d"), PartitionIndex);
		}
		return BasePath + TEXT(".ucas");
	}

	static void OverwriteFile(const FString& DstPath, const FString& SrcPath)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(OverwriteFile);

		IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
		if (!Ipf.FileExists(*SrcPath))
		{
			UE_LOGF(LogIoStore, Error, "Source file '%ls' doesn't exist", *SrcPath);
			return;
		}

		if (bParanoidChecks)
		{
			Ipf.MoveFile(*(DstPath + TEXT(".backup")), *DstPath);
		}

		Ipf.DeleteFile(*DstPath);

		if (Ipf.MoveFile(*DstPath, *SrcPath))
		{
			UE_LOGF(LogIoStore, Display, "Moved '%ls' -> '%ls'", *SrcPath, *DstPath);
			return;
		}

		if (Ipf.CopyFile(*DstPath, *SrcPath))
		{
			UE_LOGF(LogIoStore, Display, "Copied '%ls' -> '%ls'", *SrcPath, *DstPath);
			Ipf.DeleteFile(*SrcPath);
			return;
		}

		UE_LOGF(LogIoStore, Error, "Failed to move '%ls' -> '%ls'", *SrcPath, *DstPath);
	}
};

class FIoStoreWriter
	: public IIoStoreWriter
{
public:
	FIoStoreWriter(const TCHAR* InContainerPathAndBaseFileName)
		: ContainerPathAndBaseFileName(InContainerPathAndBaseFileName)
	{
	}

	void SetReferenceChunkDatabase(TSharedPtr<IIoStoreWriterReferenceChunkDatabase> InReferenceChunkDatabase, TSharedPtr<IIoStoreWriterReferenceChunkDatabase> InPatchInPlaceReferenceChunkDatabase)
	{
		if (InPatchInPlaceReferenceChunkDatabase.IsValid())
		{
			PatchInPlaceReferenceChunkDatabase = InPatchInPlaceReferenceChunkDatabase;
			PatchInPlaceReferenceChunkDatabase->NotifyAddedToWriter(ContainerSettings.ContainerId, FPaths::GetBaseFilename(TocFilePath));
		}
		else
		{
			// fallback to using reference chunk database
			PatchInPlaceReferenceChunkDatabase = InReferenceChunkDatabase;
		}

		if (InReferenceChunkDatabase.IsValid() == false)
		{
			ReferenceChunkDatabase = InReferenceChunkDatabase;
			return;
		}

		if (InReferenceChunkDatabase->GetCompressionBlockSize() != WriterContext->GetSettings().CompressionBlockSize)
		{
			UE_LOGF(LogIoStore, Warning, "Reference chunk database has a different compression block size than the current writer!");
			UE_LOGF(LogIoStore, Warning, "No chunks will match, so ignoring. ReferenceChunkDb: %d, IoStoreWriter: %lld", InReferenceChunkDatabase->GetCompressionBlockSize(), WriterContext->GetSettings().CompressionBlockSize);
			return;
		}

		ReferenceChunkDatabase = InReferenceChunkDatabase;

		// Add ourselves to the reference chunk db's list of possibles
		ReferenceChunkDatabase->NotifyAddedToWriter(ContainerSettings.ContainerId, FPaths::GetBaseFilename(TocFilePath));
	}

	void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const
	{
		const FIoStoreTocResource& TocResource = TocBuilder.GetTocResource();

		for (int32 ChunkIndex = 0; ChunkIndex < TocResource.ChunkIds.Num(); ++ChunkIndex)
		{
			FIoStoreTocChunkInfo ChunkInfo = TocBuilder.GetTocChunkInfo(ChunkIndex);
			if (!Callback(MoveTemp(ChunkInfo)))
			{
				break;
			}
		}
	}

	[[nodiscard]] FIoStatus Initialize(FIoStoreWriterContextImpl& InContext, const FIoContainerSettings& InContainerSettings)
	{
		WriterContext = &InContext;
		ContainerSettings = InContainerSettings;

		TocFilePath = ContainerPathAndBaseFileName + TEXT(".utoc");
		
		FString PakChunkName = FPaths::GetBaseFilename(ContainerPathAndBaseFileName);

		const uint64* MaxPartitionSizeOverrideValue = WriterContext->WriterSettings.MaxPartitionSizeOverride.Find(PakChunkName);
		MaxPartitionSize =  MaxPartitionSizeOverrideValue ? *MaxPartitionSizeOverrideValue : WriterContext->WriterSettings.MaxPartitionSize;

		if (MaxPartitionSize > 0)
		{
			UE_LOGF(LogIoStore, Display, "FIoStoreWriter: using a max partition size of %llu bytes for %ls", MaxPartitionSize, *PakChunkName);
		}

		IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
		Ipf.CreateDirectoryTree(*FPaths::GetPath(TocFilePath));

		FPartition& Partition = Partitions.AddDefaulted_GetRef();
		Partition.Index = 0;

		return FIoStatus::Ok;
	}

	virtual void EnableDiskLayoutOrdering(const TArray<TUniquePtr<FIoStoreReader>>& PatchSourceReaders) override
	{
		check(!LayoutEntriesHead);
		check(!Entries.Num());
		LayoutEntriesHead = new FLayoutEntry();
		LayoutEntries.Add(LayoutEntriesHead);
		FLayoutEntry* PrevEntryLink = LayoutEntriesHead;

		for (const TUniquePtr<FIoStoreReader>& PatchSourceReader : PatchSourceReaders)
		{
			TArray<TPair<uint64, FLayoutEntry*>> LayoutEntriesWithOffsets;
			PatchSourceReader->EnumerateChunks([this, &PrevEntryLink, &LayoutEntriesWithOffsets](const FIoStoreTocChunkInfo& ChunkInfo)
				{
					FLayoutEntry* PreviousBuildEntry = new FLayoutEntry();
					PreviousBuildEntry->ChunkHash = ChunkInfo.ChunkHash;
					PreviousBuildEntry->PartitionIndex = ChunkInfo.PartitionIndex;
					PreviousBuildEntry->CompressedSize = ChunkInfo.CompressedSize;
					LayoutEntriesWithOffsets.Emplace(ChunkInfo.Offset, PreviousBuildEntry);
					PreviousBuildLayoutEntryByChunkId.Add(ChunkInfo.Id, PreviousBuildEntry);
					return true;
				});

			// Sort entries by offset
			Algo::Sort(LayoutEntriesWithOffsets, [](const TPair<uint64, FLayoutEntry*>& A, const TPair<uint64, FLayoutEntry*>& B)
				{
					return A.Get<0>() < B.Get<0>();
				});

			for (const TPair<uint64, FLayoutEntry*>& EntryWithOffset : LayoutEntriesWithOffsets)
			{
				FLayoutEntry* PreviousBuildEntry = EntryWithOffset.Get<1>();
				LayoutEntries.Add(PreviousBuildEntry);
				PrevEntryLink->Next = PreviousBuildEntry;
				PreviousBuildEntry->Prev = PrevEntryLink;
				PrevEntryLink = PreviousBuildEntry;
			}
			if (!ContainerSettings.bGenerateDiffPatch)
			{
				break;
			}
		}

		LayoutEntriesTail = new FLayoutEntry();
		LayoutEntries.Add(LayoutEntriesTail);
		PrevEntryLink->Next = LayoutEntriesTail;
		LayoutEntriesTail->Prev = PrevEntryLink;
	}

	virtual void Append(const FIoChunkId& ChunkId, IIoStoreWriteRequest* Request, const FIoWriteOptions& WriteOptions) override
	{
		//
		// This function sets up the sequence of events that takes a chunk from source data on disc
		// to written to a container. The first thing that happens is the source data is read in order
		// to hash it to detect whether or not it's modified as well as look up in reference databases.
		// Load the data -> PrepareSourceBufferAsync
		// Hash the data -> HashTask lambda
		//
		// The hash task itself doesn't continue to the next steps - the Flush() call
		// waits for all hashes to be complete before kicking the next steps.
		//
		TRACE_CPUPROFILER_EVENT_SCOPE(AppendWriteRequest);
		check(!bHasFlushed);
		checkf(ChunkId.IsValid(), TEXT("ChunkId is not valid!"));

		// Skip the entire compress/encrypt/write pipeline for known duplicates.
		// ResolveDeferredDuplicates will emit TOC entries that point to existing compression blocks.
		if (WriteOptions.DuplicateOf.IsValid())
		{
			ensureMsgf(WriterContext->GetSettings().bDeduplicateChunks,
				TEXT("Append received DuplicateOf=%ls for chunk %ls but bDeduplicateChunks is disabled on this writer"),
				*LexToString(WriteOptions.DuplicateOf), *LexToString(ChunkId));

			WriterContext->TotalChunksCount.IncrementExchange();
			WriterContext->HashedChunksCount.IncrementExchange();
			FIoStoreDeferredDuplicate& Dup = DeferredDuplicates.AddDefaulted_GetRef();
			Dup.ChunkId = ChunkId;
			Dup.DuplicateOfChunkId = WriteOptions.DuplicateOf;
			Dup.FileName = WriteOptions.FileName;
			Dup.bIsMemoryMapped = WriteOptions.bIsMemoryMapped;
			Dup.bForceUncompressed = WriteOptions.bForceUncompressed;
			if (Request)
			{
				delete Request;
			}
			return;
		}

		const FIoStoreWriterSettings& WriterSettings = WriterContext->GetSettings();

		WriterContext->TotalChunksCount.IncrementExchange();
		FIoStoreWriteQueueEntry* Entry = new FIoStoreWriteQueueEntry();
		Entries.Add(Entry);
		Entry->Writer = this;
		Entry->Sequence = Entries.Num();
		Entry->ChunkId = ChunkId;
		Entry->Options = WriteOptions;
		Entry->CompressionMethod = CompressionMethodForEntry(WriteOptions);
		Entry->bUseDDCForCompression =
			WriterSettings.bCompressionEnableDDC &&
			Entry->CompressionMethod != NAME_None &&
			Request->GetSourceBufferSizeEstimate() > WriterSettings.CompressionMinBytesSaved &&
			Request->GetSourceBufferSizeEstimate() > WriterSettings.CompressionMinSizeToConsiderDDC &&
			!Entry->Options.FileName.EndsWith(TEXT(".umap")); // avoid cache churn while maps are known to cook non-deterministically
		Entry->Request = Request;		

		// If we can get the hash without reading the whole thing and hashing it, do so to avoid the IO.
		if (const FIoHash* ChunkHash = Request->GetChunkHash(); ChunkHash != nullptr)
		{
			check(!ChunkHash->IsZero());
			Entry->ChunkHash = *ChunkHash;
			if (WriterSettings.bValidateChunkHashes == false)
			{
				// If we aren't validating then we just use it and bail.
				WriterContext->HashDbChunksCount.IncrementExchange();
				WriterContext->HashDbChunksByType[(int8)Entry->ChunkId.GetChunkType()].IncrementExchange();
				WriterContext->HashedChunksCount.IncrementExchange();

				if (ReferenceChunkDatabase.IsValid() && Entry->CompressionMethod != NAME_None)
				{
					Entry->bLoadingFromReferenceDb = ReferenceChunkDatabase->ChunkExists(ContainerSettings.ContainerId, Entry->ChunkHash, Entry->ChunkId, Entry->NumChunkBlocks);
					Entry->bCouldBeFromReferenceDb = true;
				}
				Entry->bUseDDCForCompression &= !Entry->bLoadingFromReferenceDb;
				return;
			}
			// If we are validating run the normal path to verify it.
		}
		// Otherwise, we have to do the load & hash
		UE::Tasks::FTaskEvent HashEvent{ TEXT("HashEvent") };
		Entry->HashTask = UE::Tasks::Launch(TEXT("HashChunk"), [this, Entry]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(HashChunk);
			const FIoBuffer* SourceBuffer = Entry->Request->GetSourceBuffer();
			if (!SourceBuffer)
			{
				UE_LOGF(LogIoStore, Error, "HashChunk: Failed to load Chunk %ls from %ls for file %ls.",
					*LexToString(Entry->ChunkId), Entry->Request->DebugNameOfRepository(), *Entry->Options.FileName);
				Entry->ChunkHash.Reset();
				Entry->bUseDDCForCompression = false;
				WriterContext->HashedChunksCount.IncrementExchange();
				WriterContext->ReportError();
				return;
			}

			FIoHash ChunkHash = FIoHash::HashBuffer(SourceBuffer->Data(), SourceBuffer->DataSize());

			if (!Entry->ChunkHash.IsZero() && Entry->ChunkHash != ChunkHash)
			{
				UE_LOGF(LogIoStore, Warning, "Hash Validation Failed: ChunkId %ls has mismatching hash, new calculated '%ls' vs old cached '%ls'",
					*LexToString(Entry->ChunkId),
					*LexToString(ChunkHash),
					*LexToString(Entry->ChunkHash));
			}

			Entry->ChunkHash = ChunkHash;
			WriterContext->HashedChunksCount.IncrementExchange();

			if (ReferenceChunkDatabase.IsValid() && Entry->CompressionMethod != NAME_None)
			{
				Entry->bLoadingFromReferenceDb = ReferenceChunkDatabase->ChunkExists(ContainerSettings.ContainerId, Entry->ChunkHash, Entry->ChunkId, Entry->NumChunkBlocks);
				Entry->bCouldBeFromReferenceDb = true;
			}
			Entry->bUseDDCForCompression &= !Entry->bLoadingFromReferenceDb;

			// Release the source data buffer, it will be reloaded later when we start compressing the chunk
			Entry->Request->FreeSourceBuffer();
		}, HashEvent, UE::Tasks::ETaskPriority::High);

		// Kick off the source buffer read to run the hash task
		Entry->Request->PrepareSourceBufferAsync(HashEvent);
	}

	virtual void Append(const FIoChunkId& ChunkId, FIoBuffer Chunk, const FIoWriteOptions& WriteOptions, uint64 OrderHint) override
	{
		struct FWriteRequest
			: IIoStoreWriteRequest
		{
			FWriteRequest(FIoBuffer InSourceBuffer, uint64 InOrderHint)
				: OrderHint(InOrderHint)
			{
				SourceBuffer = InSourceBuffer;
				SourceBuffer.MakeOwned();
			}

			virtual ~FWriteRequest() = default;

			void PrepareSourceBufferAsync(UE::Tasks::FTaskEvent& CompletionEvent) override
			{
				CompletionEvent.Trigger();
			}

			const FIoBuffer* GetSourceBuffer() override
			{
				return &SourceBuffer;
			}

			void FreeSourceBuffer() override
			{
			}

			uint64 GetOrderHint() override
			{
				return OrderHint;
			}

			TArrayView<const FFileRegion> GetRegions()
			{
				return TArrayView<const FFileRegion>();
			}

			virtual const FIoHash* GetChunkHash() override
			{
				return nullptr;
			}

			virtual uint64 GetSourceBufferSizeEstimate() override
			{
				return SourceBuffer.DataSize();
			}

			virtual const TCHAR* DebugNameOfRepository() const override
			{
				return TEXT("FIoStoreWriter::Append");
			}

			FIoBuffer SourceBuffer;
			uint64 OrderHint;
		};

		Append(ChunkId, new FWriteRequest(Chunk, OrderHint), WriteOptions);
	}

	bool GeneratePerfectHashes(FIoStoreTocResource& TocResource, const TCHAR* ContainerDebugName)
	{
		// https://en.wikipedia.org/wiki/Perfect_hash_function
		TRACE_CPUPROFILER_EVENT_SCOPE(TocGeneratePerfectHashes);
		uint32 ChunkCount = TocResource.ChunkIds.Num();
		uint32 SeedCount = FMath::Max(1, FMath::RoundToInt32(ChunkCount / 2.0));
		check(TocResource.ChunkOffsetLengths.Num() == ChunkCount);
		
		TArray<FIoChunkId> OutTocChunkIds;
		OutTocChunkIds.SetNum(ChunkCount);
		TArray<FIoOffsetAndLength> OutTocOffsetAndLengths;
		OutTocOffsetAndLengths.SetNum(ChunkCount);
		TArray<FIoStoreTocEntryMeta> OutTocChunkMetas;
		OutTocChunkMetas.SetNum(ChunkCount);
		TArray<int32> OutTocChunkHashSeeds;
		OutTocChunkHashSeeds.SetNumZeroed(SeedCount);
		TArray<int32> OutTocChunkIndicesWithoutPerfectHash;

		TArray<TArray<int32>> Buckets;
		Buckets.SetNum(SeedCount);

		TBitArray<> FreeSlots(true, ChunkCount);
		// Put each chunk in a bucket, each bucket contains the chunk ids that have colliding hashes
		for (uint32 ChunkIndex = 0; ChunkIndex < ChunkCount; ++ChunkIndex)
		{
			const FIoChunkId& ChunkId = TocResource.ChunkIds[ChunkIndex];
			Buckets[FIoStoreTocResource::HashChunkIdWithSeed(0, ChunkId) % SeedCount].Add(ChunkIndex);
		}

		uint64 TotalIterationCount = 0;
		uint64 TotalOverflowBucketsCount = 0;
		
		// For each bucket containing more than one chunk id find a seed that makes its chunk ids
		// hash to unused slots in the output array
		Algo::Sort(Buckets, [](const TArray<int32>& A, const TArray<int32>& B)
			{
				return A.Num() > B.Num();
			});
		for (uint32 BucketIndex = 0; BucketIndex < SeedCount; ++BucketIndex)
		{
			const TArray<int32>& Bucket = Buckets[BucketIndex];
			if (Bucket.Num() <= 1)
			{
				break;
			}
			uint64 BucketHash = FIoStoreTocResource::HashChunkIdWithSeed(0, TocResource.ChunkIds[Bucket[0]]);

			static constexpr uint32 Primes[] = {
				2, 3, 5, 7, 11, 13, 17, 19, 23, 29, 31, 37, 41, 43, 47, 53, 59, 61, 67, 71, 73, 79,
				83, 89, 97, 101, 103, 107, 109, 113, 127, 131, 137, 139, 149, 151, 157, 163, 167,
				173, 179, 181, 191, 193, 197, 199, 211, 223, 227, 229, 233, 239, 241, 251, 257, 263,
				269, 271, 277, 281, 283, 293, 307, 311, 313, 317, 331, 337, 347, 349, 353, 359, 367,
				373, 379, 383, 389, 397, 401, 409, 419, 421, 431, 433, 439, 443, 449, 457, 461, 463,
				467, 479, 487, 491, 499, 503, 509, 521, 523, 541, 547, 557, 563, 569, 571, 577, 587,
				593, 599, 601, 607, 613, 617, 619, 631, 641, 643, 647, 653, 659, 661, 673, 677, 683,
				691, 701, 709, 719, 727, 733, 739, 743, 751, 757, 761, 769, 773, 787, 797, 809, 811,
				821, 823, 827, 829, 839, 853, 857, 859, 863, 877, 881, 883, 887, 907, 911, 919, 929,
				937, 941, 947, 953, 967, 971, 977, 983, 991, 997, 1009, 1013, 1019, 1021, 1031, 1033,
				1039, 1049, 1051, 1061, 1063, 1069, 1087, 1091, 1093, 1097, 1103, 1109, 1117, 1123,
				1129, 1151, 1153, 1163, 1171, 1181, 1187, 1193, 1201, 1213, 1217, 1223, 1229, 1231,
				1237, 1249, 1259, 1277, 1279, 1283, 1289, 1291, 1297, 1301, 1303, 1307, 1319, 1321,
				1327, 1361, 1367, 1373, 1381, 1399, 1409, 1423, 1427, 1429, 1433, 1439, 1447, 1451,
				1453, 1459, 1471, 1481, 1483, 1487, 1489, 1493, 1499, 1511, 1523, 1531, 1543, 1549,
				1553, 1559, 1567, 1571, 1579, 1583, 1597, 1601, 1607, 1609, 1613, 1619, 1621, 1627,
				1637, 1657, 1663, 1667, 1669, 1693, 1697, 1699, 1709, 1721, 1723, 1733, 1741, 1747,
				1753, 1759, 1777, 1783, 1787, 1789, 1801, 1811, 1823, 1831, 1847, 1861, 1867, 1871,
				1873, 1877, 1879, 1889, 1901, 1907, 1913, 1931, 1933, 1949, 1951, 1973, 1979, 1987,
				1993, 1997, 1999, 2003, 2011, 2017, 2027, 2029, 2039, 2053, 2063, 2069, 2081, 2083,
				2087, 2089, 2099, 2111, 2113, 2129, 2131, 2137, 2141, 2143, 2153, 2161, 2179, 2203,
				2207, 2213, 2221, 2237, 2239, 2243, 2251, 2267, 2269, 2273, 2281, 2287, 2293, 2297,
				2309, 2311, 2333, 2339, 2341, 2347, 2351, 2357, 2371, 2377, 2381, 2383, 2389, 2393,
				2399, 2411, 2417, 2423, 2437, 2441, 2447, 2459, 2467, 2473, 2477, 2503, 2521, 2531,
				2539, 2543, 2549, 2551, 2557, 2579, 2591, 2593, 2609, 2617, 2621, 2633, 2647, 2657,
				2659, 2663, 2671, 2677, 2683, 2687, 2689, 2693, 2699, 2707, 2711, 2713, 2719, 2729,
				2731, 2741, 2749, 2753, 2767, 2777, 2789, 2791, 2797, 2801, 2803, 2819, 2833, 2837,
				2843, 2851, 2857, 2861, 2879, 2887, 2897, 2903, 2909, 2917, 2927, 2939, 2953, 2957,
				2963, 2969, 2971, 2999, 3001, 3011, 3019, 3023, 3037, 3041, 3049, 3061, 3067, 3079,
				3083, 3089, 3109, 3119, 3121, 3137, 3163, 3167, 3169, 3181, 3187, 3191, 3203, 3209,
				3217, 3221, 3229, 3251, 3253, 3257, 3259, 3271, 3299, 3301, 3307, 3313, 3319, 3323,
				3329, 3331, 3343, 3347, 3359, 3361, 3371, 3373, 3389, 3391, 3407, 3413, 3433, 3449,
				3457, 3461, 3463, 3467, 3469, 3491, 3499, 3511, 3517, 3527, 3529, 3533, 3539, 3541,
				3547, 3557, 3559, 3571, 3581, 3583, 3593, 3607, 3613, 3617, 3623, 3631, 3637, 3643,
				3659, 3671, 3673, 3677, 3691, 3697, 3701, 3709, 3719, 3727, 3733, 3739, 3761, 3767,
				3769, 3779, 3793, 3797, 3803, 3821, 3823, 3833, 3847, 3851, 3853, 3863, 3877, 3881,
				3889, 3907, 3911, 3917, 3919, 3923, 3929, 3931, 3943, 3947, 3967, 3989, 4001, 4003,
				4007, 4013, 4019, 4021, 4027, 4049, 4051, 4057, 4073, 4079, 4091, 4093, 4099, 4111,
				4127, 4129, 4133, 4139, 4153, 4157, 4159, 4177, 4201, 4211, 4217, 4219, 4229, 4231,
				4241, 4243, 4253, 4259, 4261, 4271, 4273, 4283, 4289, 4297, 4327, 4337, 4339, 4349,
				4357, 4363, 4373, 4391, 4397, 4409, 4421, 4423, 4441, 4447, 4451, 4457, 4463, 4481,
				4483, 4493, 4507, 4513, 4517, 4519, 4523, 4547, 4549, 4561, 4567, 4583, 4591, 4597,
				4603, 4621, 4637, 4639, 4643, 4649, 4651, 4657, 4663, 4673, 4679, 4691, 4703, 4721,
				4723, 4729, 4733, 4751, 4759, 4783, 4787, 4789, 4793, 4799, 4801, 4813, 4817, 4831,
				4861, 4871, 4877, 4889, 4903, 4909, 4919, 4931, 4933, 4937, 4943, 4951, 4957, 4967,
				4969, 4973, 4987, 4993, 4999, 5003, 5009, 5011, 5021, 5023, 5039, 5051, 5059, 5077,
				5081, 5087, 5099, 5101, 5107, 5113, 5119, 5147, 5153, 5167, 5171, 5179, 5189, 5197,
				5209, 5227, 5231, 5233, 5237, 5261, 5273, 5279, 5281, 5297, 5303, 5309, 5323, 5333,
				5347, 5351, 5381, 5387, 5393, 5399, 5407, 5413, 5417, 5419, 5431, 5437, 5441, 5443,
				5449, 5471, 5477, 5479, 5483, 5501, 5503, 5507, 5519, 5521, 5527, 5531, 5557, 5563,
				5569, 5573, 5581, 5591, 5623, 5639, 5641, 5647, 5651, 5653, 5657, 5659, 5669, 5683,
				5689, 5693, 5701, 5711, 5717, 5737, 5741, 5743, 5749, 5779, 5783, 5791, 5801, 5807,
				5813, 5821, 5827, 5839, 5843, 5849, 5851, 5857, 5861, 5867, 5869, 5879, 5881, 5897,
				5903, 5923, 5927, 5939, 5953, 5981, 5987, 6007, 6011, 6029, 6037, 6043, 6047, 6053,
				6067, 6073, 6079, 6089, 6091, 6101, 6113, 6121, 6131, 6133, 6143, 6151, 6163, 6173,
				6197, 6199, 6203, 6211, 6217, 6221, 6229, 6247, 6257, 6263, 6269, 6271, 6277, 6287,
				6299, 6301, 6311, 6317, 6323, 6329, 6337, 6343, 6353, 6359, 6361, 6367, 6373, 6379,
				6389, 6397, 6421, 6427, 6449, 6451, 6469, 6473, 6481, 6491, 6521, 6529, 6547, 6551,
				6553, 6563, 6569, 6571, 6577, 6581, 6599, 6607, 6619, 6637, 6653, 6659, 6661, 6673,
				6679, 6689, 6691, 6701, 6703, 6709, 6719, 6733, 6737, 6761, 6763, 6779, 6781, 6791,
				6793, 6803, 6823, 6827, 6829, 6833, 6841, 6857, 6863, 6869, 6871, 6883, 6899, 6907,
				6911, 6917, 6947, 6949, 6959, 6961, 6967, 6971, 6977, 6983, 6991, 6997, 7001, 7013,
				7019, 7027, 7039, 7043, 7057, 7069, 7079, 7103, 7109, 7121, 7127, 7129, 7151, 7159,
				7177, 7187, 7193, 7207, 7211, 7213, 7219, 7229, 7237, 7243, 7247, 7253, 7283, 7297,
				7307, 7309, 7321, 7331, 7333, 7349, 7351, 7369, 7393, 7411, 7417, 7433, 7451, 7457,
				7459, 7477, 7481, 7487, 7489, 7499, 7507, 7517, 7523, 7529, 7537, 7541, 7547, 7549,
				7559, 7561, 7573, 7577, 7583, 7589, 7591, 7603, 7607, 7621, 7639, 7643, 7649, 7669,
				7673, 7681, 7687, 7691, 7699, 7703, 7717, 7723, 7727, 7741, 7753, 7757, 7759, 7789,
				7793, 7817, 7823, 7829, 7841, 7853, 7867, 7873, 7877, 7879, 7883, 7901, 7907, 7919
			};
			static constexpr uint32 MaxIterations = UE_ARRAY_COUNT(Primes);

			uint32 PrimeIndex = 0;
			TBitArray<> BucketUsedSlots(false, ChunkCount);
			int32 IndexInBucket = 0;
			bool bFoundSeedForBucket = true;
			uint64 BucketIterationCount = 0;
			while (IndexInBucket < Bucket.Num())
			{
				++BucketIterationCount;
				const FIoChunkId& ChunkId = TocResource.ChunkIds[Bucket[IndexInBucket]];
				uint32 Seed = Primes[PrimeIndex];
				uint32 Slot = FIoStoreTocResource::HashChunkIdWithSeed(Seed, ChunkId) % ChunkCount;
				if (!FreeSlots[Slot] || BucketUsedSlots[Slot])
				{
					++PrimeIndex;
					if (PrimeIndex == MaxIterations)
					{
						// Unable to resolve collisions for this bucket, put items in the overflow list and
						// save the negative index of the first item in the bucket as the seed
						// (-ChunkCount - 1 to separate from the single item buckets below)
						UE_LOGF(LogIoStore, Verbose, "%ls: Failed finding seed for bucket with %d items after %lld iterations.", ContainerDebugName, Bucket.Num(), BucketIterationCount);
						bFoundSeedForBucket = false;
						OutTocChunkHashSeeds[BucketHash % SeedCount] = -OutTocChunkIndicesWithoutPerfectHash.Num() - ChunkCount - 1;
						OutTocChunkIndicesWithoutPerfectHash.Append(Bucket);
						++TotalOverflowBucketsCount;
						break;

					}
					IndexInBucket = 0;
					BucketUsedSlots.Init(false, ChunkCount);
				}
				else
				{
					BucketUsedSlots[Slot] = true;
					++IndexInBucket;
				}
			}

			TotalIterationCount += BucketIterationCount;

			if (bFoundSeedForBucket)
			{
				uint32 Seed = Primes[PrimeIndex];
				OutTocChunkHashSeeds[BucketHash % SeedCount] = Seed;
				for (IndexInBucket = 0; IndexInBucket < Bucket.Num(); ++IndexInBucket)
				{
					int32 ChunkIndex = Bucket[IndexInBucket];
					const FIoChunkId& ChunkId = TocResource.ChunkIds[ChunkIndex];
					uint32 Slot = FIoStoreTocResource::HashChunkIdWithSeed(Seed, ChunkId) % ChunkCount;
					check(FreeSlots[Slot]);
					FreeSlots[Slot] = false;
					OutTocChunkIds[Slot] = ChunkId;
					OutTocOffsetAndLengths[Slot] = TocResource.ChunkOffsetLengths[ChunkIndex];
					OutTocChunkMetas[Slot] = TocResource.ChunkMetas[ChunkIndex];
				}
			}
		}

		// For the remaining buckets with only one chunk id put that chunk id in the first empty position in
		// the output array and store the index as a negative seed for the bucket (-1 to allow use of slot 0)
		TConstSetBitIterator<> FreeSlotIt(FreeSlots);
		for (uint32 BucketIndex = 0; BucketIndex < SeedCount; ++BucketIndex)
		{
			const TArray<int32>& Bucket = Buckets[BucketIndex];
			if (Bucket.Num() == 1)
			{
				uint32 Slot = FreeSlotIt.GetIndex();
				++FreeSlotIt;
				int32 ChunkIndex = Bucket[0];
				const FIoChunkId& ChunkId = TocResource.ChunkIds[ChunkIndex];
				uint64 BucketHash = FIoStoreTocResource::HashChunkIdWithSeed(0, ChunkId);
				OutTocChunkHashSeeds[BucketHash % SeedCount] = -static_cast<int32>(Slot) - 1;
				OutTocChunkIds[Slot] = ChunkId;
				OutTocOffsetAndLengths[Slot] = TocResource.ChunkOffsetLengths[ChunkIndex];
				OutTocChunkMetas[Slot] = TocResource.ChunkMetas[ChunkIndex];
			}
		}

		if (!OutTocChunkIndicesWithoutPerfectHash.IsEmpty())
		{
			// Put overflow items in the remaining free slots and update the index for each overflow entry
			UE_LOGF(LogIoStore, Display, "%ls: Failed finding perfect hashmap for %d items. %lld overflow buckets with %d items.", ContainerDebugName, ChunkCount, TotalOverflowBucketsCount, OutTocChunkIndicesWithoutPerfectHash.Num());
			for (int32& OverflowEntryIndex : OutTocChunkIndicesWithoutPerfectHash)
			{
				uint32 Slot = FreeSlotIt.GetIndex();
				++FreeSlotIt;
				const FIoChunkId& ChunkId = TocResource.ChunkIds[OverflowEntryIndex];
				OutTocChunkIds[Slot] = ChunkId;
				OutTocOffsetAndLengths[Slot] = TocResource.ChunkOffsetLengths[OverflowEntryIndex];
				OutTocChunkMetas[Slot] = TocResource.ChunkMetas[OverflowEntryIndex];
				OverflowEntryIndex = Slot;
			}
		}
		else
		{
			UE_LOGF(LogIoStore, Display, "%ls: Found perfect hashmap for %d items.", ContainerDebugName, ChunkCount);
		}
		double AverageIterationCount = ChunkCount > 0 ? static_cast<double>(TotalIterationCount) / ChunkCount : 0.0;
		UE_LOGF(LogIoStore, Verbose, "%ls: %f iterations/chunk", ContainerDebugName, AverageIterationCount);

		TocResource.ChunkIds = MoveTemp(OutTocChunkIds);
		TocResource.ChunkOffsetLengths = MoveTemp(OutTocOffsetAndLengths);
		TocResource.ChunkMetas = MoveTemp(OutTocChunkMetas);
		TocResource.ChunkPerfectHashSeeds = MoveTemp(OutTocChunkHashSeeds);
		TocResource.ChunkIndicesWithoutPerfectHash = MoveTemp(OutTocChunkIndicesWithoutPerfectHash);

		return true;
	}

	void DoPatchInPlaceLayout()
	{
		PatchInPlaceSizeBeforeLayout = 0;
		for (FPartition& Partition : Partitions)
		{
			PatchInPlaceSizeBeforeLayout += Partition.Offset;
		}

		PatchInPlacePinnedSizeBeforeDefrag = 0;
		PatchInPlacePinnedSizeAfterDefrag = 0;

		if (!PatchInPlaceReferenceChunkDatabase.IsValid())
		{
			UE_LOGF(LogIoStore, Warning, "Trying to use patch-in-place without valid reference database");
			return;
		}

		FString ReferenceContainerPath;
		if (!PatchInPlaceReferenceChunkDatabase->GetContainerFilePath(ContainerSettings.ContainerId, ReferenceContainerPath))
		{
			UE_LOGF(LogIoStore, Display, "Skipping patch-in-place for container '%ls' because reference container was not found", *ContainerPathAndBaseFileName);
			return;
		}

		const FIoStoreWriterSettings& WriterSettings = WriterContext->GetSettings();

		FIoStoreTocResource& SourceToc = TocBuilder.GetTocResource();
		// needed for FIoStoreTocResource::GetTocChunkInfo to work 
		SourceToc.Header.CompressionBlockSize = WriterSettings.CompressionBlockSize;
		SourceToc.Header.PartitionSize = MaxPartitionSize > 0 ? MaxPartitionSize : MAX_uint64;

		FIoStoreTocResource TargetToc;
		int32 TargetPartitionNum = 0;

		FPatchInPlaceLayout Layout(
			ReferenceContainerPath,
			TocFilePath,
			ContainerPathAndBaseFileName + TEXT("_patchinplace.utoc"),
			TocBuilder.GetTocResource().AsView(),
			WriterSettings
		);

		if (!Layout.PrepareForLayout(
			PatchInPlaceSizeBeforeLayout,
			PatchInPlacePinnedSizeBeforeDefrag,
			TargetPartitionNum,
			TotalPaddingSize,
			PatchInPlaceLocalityChange
		))
		{
			return;
		}

		// release file handles for patch-in-place
		Partitions.Empty();

		Layout.RunLayout(TargetToc, TargetPartitionNum, PatchInPlacePinnedSizeAfterDefrag);

		SourceToc = MoveTemp(TargetToc);

		TocBuilder.RebuildChunkIdToIndex();

		// recreate file handles
		for (int32 i = 0; i < TargetPartitionNum; ++i)
		{
			FPartition& Partition = Partitions.AddDefaulted_GetRef();
			Partition.Index = Partitions.Num() - 1;
			CreatePartitionContainerFile(Partition, true);
			Partition.Offset = Partition.ContainerFileHandle->Tell();
		}

		if (WriterContext->GetSettings().bEnableFileRegions)
		{
			int32 PreviousPartitionIndex = -1;

			for (int32 ChunkIndex = 0; ChunkIndex < SourceToc.ChunkIds.Num(); ++ChunkIndex)
			{
				const FIoStoreTocChunkInfo& SourceChunk = SourceToc.GetTocChunkInfo(ChunkIndex);

				ensure(SourceChunk.PartitionIndex >= PreviousPartitionIndex);
				PreviousPartitionIndex = SourceChunk.PartitionIndex;

				const TArray<FFileRegion>* FileRegionsPtr = PatchInPlaceFileRegions.Find(SourceChunk.Id);
				FFileRegion::AccumulateFileRegions(
					Partitions[SourceChunk.PartitionIndex].AllFileRegions,
					SourceChunk.OffsetOnDisk,
					SourceChunk.OffsetOnDisk,
					SourceChunk.OffsetOnDisk + SourceChunk.SizeOnDisk,
					FileRegionsPtr ? *FileRegionsPtr : TConstArrayView<FFileRegion>()
				);
			}

			PatchInPlaceFileRegions.Empty();
		}

		CurrentPartitionIndex = TargetPartitionNum - 1;
	}

	void ResolveDeferredDuplicates()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ResolveDeferredDuplicates);

		const FIoStoreTocResource& TocResource = TocBuilder.GetTocResource();
		const FIoStoreWriterSettings& WriterSettings = WriterContext->GetSettings();

		for (FIoStoreDeferredDuplicate& Dup : DeferredDuplicates)
		{
			const int32* CanonicalIndex = TocBuilder.GetTocEntryIndex(Dup.DuplicateOfChunkId);
			if (!CanonicalIndex || *CanonicalIndex == INDEX_NONE)
			{
				// fail the build if this happens, the container is invalid as current chunkid doesn't have data anymore
				UE_LOGF(LogIoStore, Error, "Deduplication: canonical chunk %ls not found in container %ls for duplicate %ls, leaving duplicate out of the TOC",
					*LexToString(Dup.DuplicateOfChunkId),
					*FPaths::GetBaseFilename(TocFilePath),
					*LexToString(Dup.ChunkId));
				continue;
			}

			checkf(TocResource.ChunkIds[*CanonicalIndex] == Dup.DuplicateOfChunkId,
				TEXT("Deduplication: stale TocBuilder index - Toc.ChunkIds[%d]=%ls but expected %ls"),
				*CanonicalIndex, *LexToString(TocResource.ChunkIds[*CanonicalIndex]), *LexToString(Dup.DuplicateOfChunkId));

			const FIoOffsetAndLength CanonicalOffsetLength = TocResource.ChunkOffsetLengths[*CanonicalIndex];
			const FIoStoreTocEntryMeta CanonicalMeta = TocResource.ChunkMetas[*CanonicalIndex];

			// calculate savings
			if (WriterSettings.CompressionBlockSize > 0)
			{
				const uint64 CanonicalOffset = CanonicalOffsetLength.GetOffset();
				const uint64 CanonicalSize = CanonicalOffsetLength.GetLength();
				const uint64 CompressionBlockSize = WriterSettings.CompressionBlockSize;
				const int32 FirstBlockIndex = int32(CanonicalOffset / CompressionBlockSize);
				const int32 LastBlockIndex = int32((Align(CanonicalOffset + CanonicalSize, CompressionBlockSize) - 1) / CompressionBlockSize);
				uint64 CanonicalDiskSize = 0;
				for (int32 BlockIdx = FirstBlockIndex; BlockIdx <= LastBlockIndex; ++BlockIdx)
				{
					CanonicalDiskSize += TocResource.CompressionBlocks[BlockIdx].GetCompressedSize();
				}
				Result.DeduplicatedSavedBytes += CanonicalDiskSize;
			}

			// a mmap'ed duplicate must always point to a mmap'ed canonical
			if (Dup.bIsMemoryMapped && WriterSettings.MemoryMappingAlignment > 0)
			{
				checkf((CanonicalOffsetLength.GetOffset() % WriterSettings.MemoryMappingAlignment) == 0,
					TEXT("Deduplication: mmap duplicate %ls points at non-mmap-aligned canonical %ls (offset %llu not aligned to %llu)"),
					*LexToString(Dup.ChunkId), *LexToString(Dup.DuplicateOfChunkId),
					CanonicalOffsetLength.GetOffset(), WriterSettings.MemoryMappingAlignment);
			}

			// a uncompressed duplicate must always point to a uncompressed canonical
			if (Dup.bForceUncompressed && WriterSettings.CompressionBlockSize > 0)
			{
				const uint64 CanonicalOffset = CanonicalOffsetLength.GetOffset();
				const uint64 CanonicalSize = CanonicalOffsetLength.GetLength();
				const uint64 CompressionBlockSize = WriterSettings.CompressionBlockSize;
				const int32 FirstBlockIndex = int32(CanonicalOffset / CompressionBlockSize);
				const int32 LastBlockIndex = int32((Align(CanonicalOffset + CanonicalSize, CompressionBlockSize) - 1) / CompressionBlockSize);
				for (int32 BlockIdx = FirstBlockIndex; BlockIdx <= LastBlockIndex; ++BlockIdx)
				{
					const FIoStoreTocCompressedBlockEntry& Block = TocResource.CompressionBlocks[BlockIdx];
					checkf(Block.GetCompressionMethodIndex() == 0,
						TEXT("Deduplication: bForceUncompressed duplicate %ls points at compressed canonical %ls (block %d compression method index %u)"),
						*LexToString(Dup.ChunkId), *LexToString(Dup.DuplicateOfChunkId),
						BlockIdx, Block.GetCompressionMethodIndex());
				}
			}

			const int32 NewIndex = TocBuilder.AddChunkEntry(Dup.ChunkId, CanonicalOffsetLength, CanonicalMeta);
			if (NewIndex == INDEX_NONE)
			{
				UE_LOGF(LogIoStore, Warning, "Deduplication: duplicated chunk %ls already exist in container %ls, leaving duplicate out of the TOC",
					*LexToString(Dup.ChunkId),
					*FPaths::GetBaseFilename(TocFilePath));
				continue;
			}

			if (ContainerSettings.IsIndexed() && Dup.FileName.Len() > 0)
			{
				TocBuilder.AddToFileIndex(Dup.ChunkId, MoveTemp(Dup.FileName));
			}
		}

		DeferredDuplicates.Empty();
	}

	void Finalize()
	{
		check(bHasFlushed);

		const FIoStoreWriterSettings& WriterSettings = WriterContext->GetSettings();

		if (WriterSettings.bUsePatchInPlaceLayout)
		{
			DoPatchInPlaceLayout();
		}

		if (WriterSettings.bDeduplicateChunks)
		{
			ResolveDeferredDuplicates();
		}

		UncompressedContainerSize = TotalEntryUncompressedSize + TotalPaddingSize;
		CompressedContainerSize = 0;

		for (FPartition& Partition : Partitions)
		{
			CompressedContainerSize += Partition.Offset;

			if (bHasMemoryMappedEntry)
			{
				uint64 ExtraPaddingBytes = Align(Partition.Offset, WriterSettings.MemoryMappingAlignment) - Partition.Offset;
				if (ExtraPaddingBytes)
				{
					TArray<uint8> Padding;
					Padding.SetNumZeroed(int32(ExtraPaddingBytes));
					Partition.ContainerFileHandle->Serialize(Padding.GetData(), ExtraPaddingBytes);
					CompressedContainerSize += ExtraPaddingBytes;
					UncompressedContainerSize += ExtraPaddingBytes;
					Partition.Offset += ExtraPaddingBytes;
					TotalPaddingSize += ExtraPaddingBytes;
				}
			}
			
			if (Partition.ContainerFileHandle)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FlushContainerFile);
				Partition.ContainerFileHandle->Flush();
				check(Partition.ContainerFileHandle->Tell() == Partition.Offset);
			}

			if (Partition.RegionsArchive)
			{
				FFileRegion::SerializeFileRegions(*Partition.RegionsArchive.Get(), Partition.AllFileRegions);
				Partition.RegionsArchive->Flush();
			}
		}

		FIoStoreTocResource& TocResource = TocBuilder.GetTocResource();

		GeneratePerfectHashes(TocResource, *FPaths::GetBaseFilename(TocFilePath));

		if (ContainerSettings.IsIndexed())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildIndex);
			TArray<FStringView> FilesToIndex;
			TocBuilder.GetFileNamesToIndex(FilesToIndex);

			FString MountPoint = IoDirectoryIndexUtils::GetCommonRootPath(FilesToIndex);
			FIoDirectoryIndexWriter DirectoryIndexWriter;
			DirectoryIndexWriter.SetMountPoint(MountPoint);

			uint32 TocEntryIndex = 0;
			for (const FIoChunkId& ChunkId : TocResource.ChunkIds)
			{
				const FString* ChunkFileName = TocBuilder.GetFileName(ChunkId);
				if (ChunkFileName)
				{
					const uint32 FileEntryIndex = DirectoryIndexWriter.AddFile(*ChunkFileName);
					check(FileEntryIndex != ~uint32(0));
					DirectoryIndexWriter.SetFileUserData(FileEntryIndex, TocEntryIndex);
				}
				++TocEntryIndex;
			}

			DirectoryIndexWriter.Flush(
				TocResource.DirectoryIndexBuffer,
				ContainerSettings.IsEncrypted() ? ContainerSettings.EncryptionKey : FAES::FAESKey());
		}

		if (WriterContext->GetSettings().bContainerMetaEnabled)
		{
			const FString EncryptedFilename = TEXT("<Encrypted>");
			FString ContainerName = FPaths::GetBaseFilename(TocFilePath);

			if (WriterContext->GetSettings().bPerContainerMeta)
			{
				FIoContainerMetaWriter ContainerMetaWriter;
				for (const FIoChunkId& ChunkId : TocResource.ChunkIds)
				{
					if (const FString* Filename = TocBuilder.GetFileName(ChunkId))
					{
						ContainerMetaWriter.AddFile(ContainerName, ChunkId, ContainerSettings.IsEncrypted() ? EncryptedFilename : *Filename);
					}
				}

				FString ContainerMetaPath = FPaths::ChangeExtension(TocFilePath, FIoContainerMetaHeader::FileExtension);
				if (const int64 FileSize = ContainerMetaWriter.Save(ContainerMetaPath); FileSize > 0)
				{
					UE_LOGF(LogIoStore, Display, "Saved container metadata '%ls' (%.2lf KiB)", *ContainerMetaPath, double(FileSize) / 1024.0);
				}
				else
				{
					UE_LOGF(LogIoStore, Warning, "Failed to save container metadata '%ls'", *ContainerMetaPath);
				}
			}
			else
			{
				for (const FIoChunkId& ChunkId : TocResource.ChunkIds)
				{
					if (const FString* Filename = TocBuilder.GetFileName(ChunkId))
					{
						WriterContext->AddContainerMeta(ContainerName, ChunkId, ContainerSettings.IsEncrypted() ? EncryptedFilename : *Filename);
					}
				}
			}
		}

		TIoStatusOr<uint64> TocSize = FIoStoreTocResource::Write(*TocFilePath, TocResource, static_cast<uint32>(WriterSettings.CompressionBlockSize), MaxPartitionSize, ContainerSettings);
		checkf(TocSize.IsOk(), TEXT("FIoStoreTocResource::Write failed with error %s"), *TocSize.Status().ToString());

		Result.ContainerId = ContainerSettings.ContainerId;
		Result.ContainerName = FPaths::GetBaseFilename(TocFilePath);
		Result.ContainerFlags = ContainerSettings.ContainerFlags;
		Result.TocSize = TocSize.ConsumeValueOrDie();
		Result.TocEntryCount = TocResource.Header.TocEntryCount;
		Result.PaddingSize = TotalPaddingSize;
		Result.UncompressedContainerSize = UncompressedContainerSize;
		Result.CompressedContainerSize = CompressedContainerSize;
		Result.TotalEntryCompressedSize = TotalEntryCompressedSize;
		Result.ReferenceCacheMissBytes = ReferenceCacheMissBytes;
		Result.DirectoryIndexSize = TocResource.Header.DirectoryIndexSize;
		Result.CompressionMethod = EnumHasAnyFlags(ContainerSettings.ContainerFlags, EIoContainerFlags::Compressed)
			? WriterSettings.CompressionMethod
			: NAME_None;
		Result.PatchInPlaceSizeBeforeLayout = PatchInPlaceSizeBeforeLayout;
		Result.PatchInPlacePinnedSizeBeforeDefrag = PatchInPlacePinnedSizeBeforeDefrag;
		Result.PatchInPlacePinnedSizeAfterDefrag = PatchInPlacePinnedSizeAfterDefrag;
		Result.PatchInPlaceLocalityChange = PatchInPlaceLocalityChange;
		Result.ModifiedChunksCount = 0;
		Result.AddedChunksCount = 0;
		Result.ModifiedChunksSize= 0;
		Result.AddedChunksSize = 0;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Cleanup);
			for (FIoStoreWriteQueueEntry* Entry : Entries)
			{
				if (Entry->bModified)
				{
					++Result.ModifiedChunksCount;
					Result.ModifiedChunksSize += Entry->DiskSize;
				}
				else if (Entry->bAdded)
				{
					++Result.AddedChunksCount;
					Result.AddedChunksSize += Entry->DiskSize;
				}
				delete Entry;
			}
		}

		Entries.Empty();
		bHasResult = true;
	}

	TIoStatusOr<FIoStoreWriterResult> GetResult()
	{
		if (!bHasResult)
		{
			return FIoStatus::Invalid;
		}
		return Result;
	}

private:
	struct FPartition
	{
		TUniquePtr<FArchive> ContainerFileHandle;
		TUniquePtr<FArchive> RegionsArchive;
		uint64 Offset = 0;
		uint64 ReservedSpace = 0;
		TArray<FFileRegion> AllFileRegions;
		int32 Index = -1;
	};

	struct FLayoutEntry
	{
		FLayoutEntry* Prev = nullptr;
		FLayoutEntry* Next = nullptr;
		uint64 IdealOrder = 0;
		uint64 CompressedSize = uint64(-1);
		FIoHash ChunkHash;
		FIoStoreWriteQueueEntry* QueueEntry = nullptr;
		int32 PartitionIndex = -1;
	};

	void FinalizeLayout()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FinalizeLayout);
		
		Algo::Sort(Entries, [](const FIoStoreWriteQueueEntry* A, const FIoStoreWriteQueueEntry* B)
		{
			uint64 AOrderHint = A->Request->GetOrderHint();
			uint64 BOrderHint = B->Request->GetOrderHint();
			if (AOrderHint != BOrderHint)
			{
				return AOrderHint < BOrderHint;
			}
			return A->Sequence < B->Sequence;
		});

		TMap<int64, FLayoutEntry*> LayoutEntriesByOrderMap;
		int64 IdealOrder = 0;
		TArray<FLayoutEntry*> UnassignedEntries;
		for (FIoStoreWriteQueueEntry* WriteQueueEntry : Entries)
		{
			FLayoutEntry* FindPreviousEntry = PreviousBuildLayoutEntryByChunkId.FindRef(WriteQueueEntry->ChunkId);
			if (FindPreviousEntry)
			{
				if (FindPreviousEntry->ChunkHash != WriteQueueEntry->ChunkHash)
				{
					WriteQueueEntry->bModified = true;
				}
				else
				{
					FindPreviousEntry->QueueEntry = WriteQueueEntry;
					FindPreviousEntry->IdealOrder = IdealOrder;
					WriteQueueEntry->PartitionIndex = FindPreviousEntry->PartitionIndex;
				}
			}
			else
			{
				WriteQueueEntry->bAdded = true;
			}
			if (WriteQueueEntry->bModified || WriteQueueEntry->bAdded)
			{
				FLayoutEntry* NewLayoutEntry = new FLayoutEntry();
				NewLayoutEntry->QueueEntry = WriteQueueEntry;
				NewLayoutEntry->IdealOrder = IdealOrder;
				LayoutEntries.Add(NewLayoutEntry);
				UnassignedEntries.Add(NewLayoutEntry);
			}
			++IdealOrder;
		}
			
		if (ContainerSettings.bGenerateDiffPatch)
		{
			LayoutEntriesHead->Next = LayoutEntriesTail;
			LayoutEntriesTail->Prev = LayoutEntriesHead;
		}
		else
		{
			for (FLayoutEntry* EntryIt = LayoutEntriesHead->Next; EntryIt != LayoutEntriesTail; EntryIt = EntryIt->Next)
			{
				if (!EntryIt->QueueEntry)
				{
					EntryIt->Prev->Next = EntryIt->Next;
					EntryIt->Next->Prev = EntryIt->Prev;
				}
				else
				{
					LayoutEntriesByOrderMap.Add(EntryIt->IdealOrder, EntryIt);
				}
			}
		}
		FLayoutEntry* LastAddedEntry = LayoutEntriesHead;
		for (FLayoutEntry* UnassignedEntry : UnassignedEntries)
		{
			check(UnassignedEntry->QueueEntry);
			FLayoutEntry* PutAfterEntry = LayoutEntriesByOrderMap.FindRef(UnassignedEntry->IdealOrder - 1);
			if (!PutAfterEntry)
			{
				PutAfterEntry = LastAddedEntry;
			}

			UnassignedEntry->Prev = PutAfterEntry;
			UnassignedEntry->Next = PutAfterEntry->Next;
			PutAfterEntry->Next->Prev = UnassignedEntry;
			PutAfterEntry->Next = UnassignedEntry;
			LayoutEntriesByOrderMap.Add(UnassignedEntry->IdealOrder, UnassignedEntry);
			LastAddedEntry = UnassignedEntry;
		}

		TArray<FIoStoreWriteQueueEntry*> IncludedQueueEntries;
		for (FLayoutEntry* EntryIt = LayoutEntriesHead->Next; EntryIt != LayoutEntriesTail; EntryIt = EntryIt->Next)
		{
			check(EntryIt->QueueEntry);
			IncludedQueueEntries.Add(EntryIt->QueueEntry);
			int32 ReserveInPartitionIndex = EntryIt->QueueEntry->PartitionIndex;
			if (ReserveInPartitionIndex >= 0)
			{
				while (Partitions.Num() <= ReserveInPartitionIndex)
				{
					FPartition& NewPartition = Partitions.AddDefaulted_GetRef();
					NewPartition.Index = Partitions.Num() - 1;
				}
				FPartition& ReserveInPartition = Partitions[ReserveInPartitionIndex];
				check(EntryIt->CompressedSize != uint64(-1));
				ReserveInPartition.ReservedSpace += EntryIt->CompressedSize;
			}
		}
		Swap(Entries, IncludedQueueEntries);

		LayoutEntriesHead = nullptr;
		LayoutEntriesTail = nullptr;
		PreviousBuildLayoutEntryByChunkId.Empty();
		for (FLayoutEntry* Entry : LayoutEntries)
		{
			delete Entry;
		}
		LayoutEntries.Empty();
	}

	FIoStatus CreatePartitionContainerFile(FPartition& Partition, const bool bUseExistingData = false)
	{
		const uint32 WriteFlags = bUseExistingData ? FILEWRITE_Append | FILEWRITE_AllowRead : 0;

		check(!Partition.ContainerFileHandle);
		FString ContainerFilePath = ContainerPathAndBaseFileName;
		if (Partition.Index > 0)
		{
			ContainerFilePath += FString::Printf(TEXT("_s%d"), Partition.Index);
		}
		ContainerFilePath += TEXT(".ucas");

		Partition.ContainerFileHandle.Reset(IFileManager::Get().CreateFileWriter(*ContainerFilePath, WriteFlags));
		if (!Partition.ContainerFileHandle)
		{
			return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open IoStore container file '") << *ContainerFilePath << TEXT("'");
		}
		if (WriterContext->GetSettings().bEnableFileRegions)
		{
			FString RegionsFilePath = ContainerFilePath + FFileRegion::RegionsFileExtension;
			Partition.RegionsArchive.Reset(IFileManager::Get().CreateFileWriter(*RegionsFilePath, WriteFlags));
			if (!Partition.RegionsArchive)
			{
				return FIoStatusBuilder(EIoErrorCode::FileOpenFailed) << TEXT("Failed to open IoStore regions file '") << *RegionsFilePath << TEXT("'");
			}
		}

		return FIoStatus::Ok;
	}

	void CompressBlock(FChunkBlock* Block)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CompressBlock);
		check(Block->CompressionMethod != NAME_None);
		uint64 CompressedBlockSize = Block->IoBuffer->DataSize();
		bool bCompressed;
		{
			if (!FCompression::CompressMemoryIfWorthDecompressing(
				Block->CompressionMethod,
				bCompressed,
				(int64)WriterContext->WriterSettings.CompressionMinBytesSaved,
				WriterContext->WriterSettings.CompressionMinPercentSaved,
				Block->IoBuffer->Data(),
				(int64&)CompressedBlockSize,
				Block->UncompressedData,
				(int64)Block->UncompressedSize,
				COMPRESS_ForPackaging))
			{
				UE_LOGF(LogIoStore, Error, "Compression failed: Method=%ls, CompressedSize=0x%llx, UncompressedSize=0x%llx", 
					*Block->CompressionMethod.ToString(), CompressedBlockSize, Block->UncompressedSize);
				bCompressed = false;
			}
		}
		if (!bCompressed)
		{
			Block->CompressionMethod = NAME_None;
			Block->CompressedSize = Block->UncompressedSize;
			FMemory::Memcpy(Block->IoBuffer->Data(), Block->UncompressedData, Block->UncompressedSize);
		}
		else
		{
			check(CompressedBlockSize > 0);
			check(CompressedBlockSize < Block->UncompressedSize);
			Block->CompressedSize = CompressedBlockSize;
		}
	}

	bool SerializeCompressedDDCData(FIoStoreWriteQueueEntry* Entry, FArchive& Ar, uint64* OutCompressedSize = nullptr)
	{
		uint64 UncompressedSize = Entry->UncompressedSize.Get(0);
		uint32 NumChunkBlocks = Entry->ChunkBlocks.Num();
		Ar << UncompressedSize;
		Ar << NumChunkBlocks;
		if (Ar.IsLoading())
		{
			Entry->NumChunkBlocks = NumChunkBlocks;
			Entry->UncompressedSize.Emplace(UncompressedSize);
			AllocateCompressionBuffers(Entry);
		}
		bool bError = false;
		for (FChunkBlock& Block : Entry->ChunkBlocks)
		{
			Ar << Block.CompressedSize;
			if (Block.CompressedSize > Block.UncompressedSize)
			{
				bError = true;
				break;
			}
			if (Ar.IsLoading() && Block.CompressedSize == Block.UncompressedSize)
			{
				Block.CompressionMethod = NAME_None;
			}
			if (Block.IoBuffer->DataSize() < Block.CompressedSize)
			{
				bError = true;
				break;
			}
			Ar.Serialize(Block.IoBuffer->Data(), Block.CompressedSize);
			if (OutCompressedSize)
			{
				*OutCompressedSize += Block.CompressedSize;
			}
		}
		bError |= Ar.IsError();
		if (Ar.IsLoading() && bError)
		{
			FreeCompressionBuffers(Entry);
		}
		return !bError;
	}

	FName CompressionMethodForEntry(const FIoWriteOptions& Options) const
	{
		FName CompressionMethod = NAME_None;
		const FIoStoreWriterSettings& WriterSettings = WriterContext->WriterSettings;
		if (ContainerSettings.IsCompressed() && !Options.bForceUncompressed && !Options.bIsMemoryMapped)
		{
			CompressionMethod = WriterSettings.CompressionMethod;
		}
		return CompressionMethod;
	}

	int32 CalculateNumChunkBlocks(uint64 ChunkSize) const
	{
		const uint64 BlockSize = WriterContext->WriterSettings.CompressionBlockSize;
		const uint64 NumChunkBlocks64 = Align(ChunkSize, BlockSize) / BlockSize;
		return IntCastChecked<int32>(NumChunkBlocks64);
	}

	void AllocateCompressionBuffers(FIoStoreWriteQueueEntry* Entry, const uint8* UncompressedData = nullptr)
	{
		check(Entry->ChunkBlocks.Num() == 0);
		const FIoStoreWriterSettings& WriterSettings = WriterContext->WriterSettings;
		check(WriterSettings.CompressionBlockSize > 0);
		
		Entry->ChunkBlocks.SetNum(Entry->NumChunkBlocks);
		{
			uint64 BytesToProcess = Entry->UncompressedSize.GetValue();
			for (int32 BlockIndex = 0; BlockIndex < Entry->NumChunkBlocks; ++BlockIndex)
			{
				FChunkBlock& Block = Entry->ChunkBlocks[BlockIndex];
				Block.IoBuffer = WriterContext->AllocCompressionBuffer();
				Block.CompressionMethod = Entry->CompressionMethod;
				Block.UncompressedSize = FMath::Min(BytesToProcess, WriterSettings.CompressionBlockSize);
				BytesToProcess -= Block.UncompressedSize;
				if (UncompressedData)
				{
					Block.UncompressedData = UncompressedData;
					UncompressedData += Block.UncompressedSize;
				}
			}
		}
	}

	void FreeCompressionBuffers(FIoStoreWriteQueueEntry* Entry)
	{
		for (FChunkBlock& ChunkBlock : Entry->ChunkBlocks)
		{
			WriterContext->FreeCompressionBuffer(ChunkBlock.IoBuffer);
		}
		Entry->ChunkBlocks.Empty();
	}

	void LoadFromReferenceDb(FIoStoreWriteQueueEntry* Entry)
	{
		if (Entry->NumChunkBlocks == 0)
		{
			Entry->BeginCompressionBarrier.Trigger();
			TRACE_COUNTER_INCREMENT(IoStoreRefDbDone);
			return;
		}

		// Allocate resources before launching the read tasks to reduce contention. Note this will
		// allocate iobuffers big enough for uncompressed size, when we only actually need it for
		// compressed size.
		Entry->ChunkBlocks.SetNum(Entry->NumChunkBlocks);
		for (int32 BlockIndex = 0; BlockIndex < Entry->NumChunkBlocks; ++BlockIndex)
		{
			FChunkBlock& Block = Entry->ChunkBlocks[BlockIndex];
			Block.IoBuffer = WriterContext->AllocCompressionBuffer();
			// Everything else in a block gets filled out from the refdb.
		}

		// Valid chunks must create the same decompressed bits, but can have different compressed bits.
		// Since we are on a lightweight dispatch thread, the actual read is async, as is the processing
		// of the results.
		TRACE_COUNTER_INCREMENT(IoStoreRefDbInflight);
		UE::Tasks::FTask RetrieveChunkTask = ReferenceChunkDatabase->RetrieveChunk(
			ContainerSettings.ContainerId, Entry->ChunkHash, Entry->ChunkId,
			[this, Entry](TIoStatusOr<FIoStoreCompressedReadResult> InReadResult)
		{

			// If we fail here, in order to recover we effectively need to re-kick this chunk's
			// BeginCompress() as well as source buffer read... however, this is just a direct read and should only fail
			// in catastrophic scenarios (loss of connection on a network drive?).
			UE_CLOGF(!InReadResult.IsOk(), LogIoStore, Error, "RetrieveChunk from ReferenceChunkDatabase failed: %ls",
				*InReadResult.Status().ToString());
			FIoStoreCompressedReadResult ReadResult = InReadResult.ValueOrDie();

			uint64 TotalUncompressedSize = 0;
			uint8* ReferenceData = ReadResult.IoBuffer.GetData();
			uint64 TotalAlignedSize = 0;
			for (int32 BlockIndex = 0; BlockIndex < ReadResult.Blocks.Num(); ++BlockIndex)
			{
				FIoStoreCompressedBlockInfo& ReferenceBlock = ReadResult.Blocks[BlockIndex];
				FChunkBlock& Block = Entry->ChunkBlocks[BlockIndex];
				Block.CompressionMethod = ReferenceBlock.CompressionMethod;
				Block.CompressedSize = ReferenceBlock.CompressedSize;
				Block.UncompressedSize = ReferenceBlock.UncompressedSize;
				TotalUncompressedSize += ReferenceBlock.UncompressedSize;

				// Future optimization: ReadCompressed returns the memory ready to encrypt in one
				// large contiguous buffer (i.e. padded). We could use the FIoBuffer functionality of referencing a 
				// sub block from a parent buffer, however this would mean that we need to add support
				// for tracking the memory usage in order to remain within our prescribed limits. To do this
				// requires releasing the entire chunk's memory at once after WriteEntry.
				// As it stands, we temporarily use untracked memory in the ReadCompressed call (in RetrieveChunk),
				// then immediately copy it to tracked memory. There's some waste as tracked memory is mod CompressionBlockSize
				// and we are post compression, so with the average 50% compression rate, we're using double the memory
				// we "could".
				FMemory::Memcpy(Block.IoBuffer->GetData(), ReferenceData, Block.CompressedSize);
				ReferenceData += ReferenceBlock.AlignedSize;
				TotalAlignedSize += ReferenceBlock.AlignedSize;
			}

			if (TotalAlignedSize != ReadResult.IoBuffer.GetSize())
			{
				// If we hit this, we might have read garbage memory above! This is very bad.
				UE_LOGF(LogIoStore, Error, "Block aligned size does not match iobuffer source size! Blocks: %ls source size: %ls",
					*FText::AsNumber(TotalAlignedSize).ToString(),
					*FText::AsNumber(ReadResult.IoBuffer.GetSize()).ToString());
			}

			Entry->UncompressedSize.Emplace(TotalUncompressedSize);
			TRACE_COUNTER_DECREMENT(IoStoreRefDbInflight);
			TRACE_COUNTER_INCREMENT(IoStoreRefDbDone);
		});
		Entry->BeginCompressionBarrier.AddPrerequisites(RetrieveChunkTask);
		Entry->BeginCompressionBarrier.Trigger();

		WriterContext->RefDbChunksCount.IncrementExchange();
		WriterContext->RefDbChunksByType[(int8)Entry->ChunkId.GetChunkType()].IncrementExchange();
	}

	void BeginCompress(FIoStoreWriteQueueEntry* Entry)
	{
		WriterContext->BeginCompressChunksByType[(int8)Entry->ChunkId.GetChunkType()].IncrementExchange();
		
		if (Entry->bLoadingFromReferenceDb || Entry->bFoundInDDC)
		{
			check(Entry->UncompressedSize.IsSet());
			Entry->FinishCompressionBarrier.Trigger();
			return;
		}

		const FIoBuffer* SourceBuffer = Entry->Request->GetSourceBuffer();
		if (!SourceBuffer)
		{
			UE_LOGF(LogIoStore, Error, "CompressChunk: Failed to load Chunk %ls from %ls for file %ls. Runtime will receive 0 bytes when loading the file.",
				*LexToString(Entry->ChunkId), Entry->Request->DebugNameOfRepository(), *Entry->Options.FileName);
			WriterContext->ReportError();
			Entry->bStoreCompressedDataInDDC = false;
			Entry->UncompressedSize.Emplace(0);
			Entry->NumChunkBlocks = 0;
			Entry->FinishCompressionBarrier.Trigger();
			return;
		}

		Entry->UncompressedSize.Emplace(SourceBuffer->DataSize());
		Entry->NumChunkBlocks = CalculateNumChunkBlocks(Entry->UncompressedSize.GetValue());

		if (Entry->NumChunkBlocks == 0)
		{
			Entry->FinishCompressionBarrier.Trigger();
			return;
		}

		AllocateCompressionBuffers(Entry, SourceBuffer->Data());

		if (Entry->CompressionMethod == NAME_None)
		{
			for (FChunkBlock& Block : Entry->ChunkBlocks)
			{
				Block.CompressionMethod = NAME_None;
				Block.CompressedSize = Block.UncompressedSize;
				FMemory::Memcpy(Block.IoBuffer->Data(), Block.UncompressedData, Block.UncompressedSize);
			}
			Entry->FinishCompressionBarrier.Trigger();
			return;
		}

		ScheduleCompressionTasks(Entry);
	}

	void ScheduleCompressionTasks(FIoStoreWriteQueueEntry* Entry)
	{
		TRACE_COUNTER_INCREMENT(IoStoreCompressionInflight);
		constexpr int32 BatchSize = 4;
		const int32 NumBatches = 1 + (Entry->ChunkBlocks.Num() / BatchSize);
		for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
		{
			const int32 BeginIndex = BatchIndex * BatchSize;
			const int32 EndIndex = FMath::Min(BeginIndex + BatchSize, Entry->ChunkBlocks.Num());
			WriterContext->ScheduledCompressionTasksCount.IncrementExchange();
			UE::Tasks::FTask CompressTask = UE::Tasks::Launch(TEXT("CompressBlocks"), [this, Entry, BeginIndex, EndIndex]()
			{
				for (int32 Index = BeginIndex; Index < EndIndex; ++Index)
				{
					FChunkBlock* BlockPtr = &Entry->ChunkBlocks[Index];
					CompressBlock(BlockPtr);
					int32 CompressedBlocksCount = Entry->CompressedBlocksCount.IncrementExchange();
					if (CompressedBlocksCount + 1 == Entry->ChunkBlocks.Num())
					{
						WriterContext->CompressedChunksByType[(int8)Entry->ChunkId.GetChunkType()].IncrementExchange();
						WriterContext->CompressedChunksCount.IncrementExchange();
						TRACE_COUNTER_DECREMENT(IoStoreCompressionInflight);
					}
				}
				WriterContext->ScheduledCompressionTasksCount.DecrementExchange();
			}, UE::Tasks::ETaskPriority::High);
			Entry->FinishCompressionBarrier.AddPrerequisites(CompressTask);
		}
		Entry->FinishCompressionBarrier.Trigger();
	}

	void BeginEncryptAndSign(FIoStoreWriteQueueEntry* Entry)
	{
		Entry->Request->FreeSourceBuffer();

		Entry->CompressedSize = 0;
		for (const FChunkBlock& ChunkBlock : Entry->ChunkBlocks)
		{
			Entry->CompressedSize += ChunkBlock.CompressedSize;
		}

		if (ContainerSettings.IsEncrypted() || ContainerSettings.IsSigned())
		{
			UE::Tasks::FTask EncryptAndSignTask = UE::Tasks::Launch(TEXT("EncryptAndSign"), [this, Entry]()
			{
				EncryptAndSign(Entry);
			}, UE::Tasks::ETaskPriority::High);
			Entry->BeginWriteBarrier.AddPrerequisites(EncryptAndSignTask);
			Entry->BeginWriteBarrier.Trigger();
		}
		else
		{
			EncryptAndSign(Entry);
			Entry->BeginWriteBarrier.Trigger();
		}
	}

	void EncryptAndSign(FIoStoreWriteQueueEntry* Entry)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EncryptAndSignChunk);
		for (FChunkBlock& Block : Entry->ChunkBlocks)
		{
			// Always align each compressed block to AES block size but store the compressed block size in the TOC
			Block.DiskSize = Block.CompressedSize;
			if (!IsAligned(Block.DiskSize, FAES::AESBlockSize))
			{
				uint64 AlignedCompressedBlockSize = Align(Block.DiskSize, FAES::AESBlockSize);
				uint8* CompressedData = Block.IoBuffer->Data();
				for (uint64 FillIndex = Block.DiskSize; FillIndex < AlignedCompressedBlockSize; ++FillIndex)
				{
					check(FillIndex < Block.IoBuffer->DataSize());
					CompressedData[FillIndex] = CompressedData[(FillIndex - Block.DiskSize) % Block.DiskSize];
				}
				Block.DiskSize = AlignedCompressedBlockSize;
			}

			if (ContainerSettings.IsEncrypted())
			{
				FAES::EncryptData(Block.IoBuffer->Data(), static_cast<uint32>(Block.DiskSize), ContainerSettings.EncryptionKey);
			}

			if (ContainerSettings.IsSigned())
			{
				FSHA1::HashBuffer(Block.IoBuffer->Data(), Block.DiskSize, Block.Signature.Hash);
			}
		}
		Entry->DiskSize = 0;
		for (const FChunkBlock& ChunkBlock : Entry->ChunkBlocks)
		{
			Entry->DiskSize += ChunkBlock.DiskSize;
		}
	}

	void WriteEntry(FIoStoreWriteQueueEntry* Entry)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WriteEntry);
		ON_SCOPE_EXIT
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FreeBlocks);
			const uint64 EntryMemoryEstimate = Entry->Request->GetSourceBufferSizeEstimate();
			FreeCompressionBuffers(Entry);
			delete Entry->Request;
			Entry->Request = nullptr;

			uint64 ScheduledMemory = WriterContext->ScheduledCompressionMemory.SubExchange(EntryMemoryEstimate);
			WriterContext->CompressionMemoryReleasedEvent->Trigger();
			TRACE_COUNTER_SET(IoStoreCompressionMemoryScheduled, ScheduledMemory - EntryMemoryEstimate);
		};
		const int32* FindExistingIndex = TocBuilder.GetTocEntryIndex(Entry->ChunkId);
		if (FindExistingIndex)
		{
			// afaict this should never happen so add a warning. If there's a legit reason for it
			// we can pull this back out. If would violate some assumptions in the reference chunk
			// database if we DO hit this, however...
			UE_LOGF(LogIoStore, Warning, "ChunkId was added twice in container %ls, %ls, file %ls hash %ls vs %ls", 
				*FPaths::GetBaseFilename(TocFilePath),
				*LexToString(Entry->ChunkId),
				*Entry->Options.FileName,
				*LexToString(TocBuilder.GetTocResource().ChunkMetas[*FindExistingIndex].ChunkHash),
				*LexToString(Entry->ChunkHash)
				);

			checkf(TocBuilder.GetTocResource().ChunkMetas[*FindExistingIndex].ChunkHash == Entry->ChunkHash, TEXT("Chunk id has already been added with different content"));
			return;
		}

		FPartition* TargetPartition = &Partitions[CurrentPartitionIndex];
		int32 NextPartitionIndexToTry = CurrentPartitionIndex + 1;
		if (Entry->PartitionIndex >= 0)
		{
			TargetPartition = &Partitions[Entry->PartitionIndex];
			if (TargetPartition->ReservedSpace > Entry->DiskSize)
			{
				TargetPartition->ReservedSpace -= Entry->DiskSize;
			}
			else
			{
				TargetPartition->ReservedSpace = 0;
			}
			NextPartitionIndexToTry = CurrentPartitionIndex;
		}

		const FIoStoreWriterSettings& WriterSettings = WriterContext->WriterSettings;
		bHasMemoryMappedEntry |= Entry->Options.bIsMemoryMapped;
		const uint64 ChunkAlignment = Entry->Options.bIsMemoryMapped ? WriterSettings.MemoryMappingAlignment : 0;
		const uint64 PartitionSizeLimit = MaxPartitionSize > 0 ? MaxPartitionSize : MAX_uint64;
		if (Entry->DiskSize > PartitionSizeLimit)
		{
			// Someone added some super large content, give a detailed log so they know what to fix.
			const TCHAR* DebugName = Entry->Options.DebugName ? Entry->Options.DebugName : TEXT("NoDebugName");

			// This must be fatal as the loop below doesn't handle partition assignment of chunks larger than the partition size.
			UE_LOGF(LogIoStore, Fatal, "Chunk is too large: DiskSize = %llu CompressedSize = %llu UncompressedSize = %ls ChunkId = %ls DebugName = %ls PartitionSize = %llu",
				Entry->DiskSize, Entry->CompressedSize, Entry->UncompressedSize.IsSet() ? *WriteToString<32>(Entry->UncompressedSize.GetValue()) : TEXT("(Unknown)"),
				*WriteToString<32>(Entry->ChunkId), DebugName, PartitionSizeLimit);
		}

		for (;;)
		{
			uint64 OffsetBeforePadding = TargetPartition->Offset;
			if (ChunkAlignment)
			{
				TargetPartition->Offset = Align(TargetPartition->Offset, ChunkAlignment);
			}
			if (WriterSettings.CompressionBlockAlignment)
			{
				// This is mostly to support patching systems that can't handle arbitrary insertion sizes. We want to
				// keep things on the alignment so that they have a chance to match vs the previous release. We also
				// want to pack small things into a single block so we don't waste a ton of space.
				bool bCrossesBlockBoundary = Align(TargetPartition->Offset, WriterSettings.CompressionBlockAlignment) != Align(TargetPartition->Offset + Entry->DiskSize - 1, WriterSettings.CompressionBlockAlignment);
				if (bCrossesBlockBoundary)
				{
					TargetPartition->Offset = Align(TargetPartition->Offset, WriterSettings.CompressionBlockAlignment);
				}
			}

			if (TargetPartition->Offset + Entry->DiskSize + TargetPartition->ReservedSpace > PartitionSizeLimit)
			{
				TargetPartition->Offset = OffsetBeforePadding;
				while (Partitions.Num() <= NextPartitionIndexToTry)
				{
					FPartition& NewPartition = Partitions.AddDefaulted_GetRef();
					NewPartition.Index = Partitions.Num() - 1;
				}
				CurrentPartitionIndex = NextPartitionIndexToTry;
				TargetPartition = &Partitions[CurrentPartitionIndex];
				++NextPartitionIndexToTry;
			}
			else
			{
				Entry->Padding = TargetPartition->Offset - OffsetBeforePadding;
				TotalPaddingSize += Entry->Padding;
				break;
			}
		}

		if (!TargetPartition->ContainerFileHandle)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CreatePartitionContainerFile);
			CreatePartitionContainerFile(*TargetPartition);
		}
		Entry->Offset = TargetPartition->Offset;

		FIoOffsetAndLength OffsetLength;
		OffsetLength.SetOffset(UncompressedFileOffset);
		OffsetLength.SetLength(Entry->UncompressedSize.GetValue());

		FIoStoreTocEntryMeta ChunkMeta{ Entry->ChunkHash, FIoStoreTocEntryMetaFlags::None };
		if (Entry->Options.bIsMemoryMapped)
		{
			ChunkMeta.Flags |= FIoStoreTocEntryMetaFlags::MemoryMapped;
		}

		uint64 OffsetInChunk = 0;
		for (const FChunkBlock& ChunkBlock : Entry->ChunkBlocks)
		{
			FIoStoreTocCompressedBlockEntry& BlockEntry = TocBuilder.AddCompressionBlockEntry();
			BlockEntry.SetOffset(TargetPartition->Index * MaxPartitionSize + TargetPartition->Offset + OffsetInChunk);
			OffsetInChunk += ChunkBlock.DiskSize;
			BlockEntry.SetCompressedSize(uint32(ChunkBlock.CompressedSize));
			BlockEntry.SetUncompressedSize(uint32(ChunkBlock.UncompressedSize));
			BlockEntry.SetCompressionMethodIndex(TocBuilder.AddCompressionMethodEntry(ChunkBlock.CompressionMethod));

			// We do this here so that we get the total size of data excluding the encryption alignment
			TotalEntryCompressedSize += ChunkBlock.CompressedSize;
			if (Entry->bCouldBeFromReferenceDb && !Entry->bLoadingFromReferenceDb)
			{
				ReferenceCacheMissBytes += ChunkBlock.CompressedSize;
			}

			if (!ChunkBlock.CompressionMethod.IsNone())
			{
				ChunkMeta.Flags |= FIoStoreTocEntryMetaFlags::Compressed;
			}

			if (ContainerSettings.IsSigned())
			{
				FSHAHash& Signature = TocBuilder.AddBlockSignatureEntry();
				Signature = ChunkBlock.Signature;
			}

		}

		const int32 TocEntryIndex = TocBuilder.AddChunkEntry(Entry->ChunkId, OffsetLength, ChunkMeta);
		check(TocEntryIndex != INDEX_NONE);

		if (ContainerSettings.IsIndexed() && Entry->Options.FileName.Len() > 0)
		{
			TocBuilder.AddToFileIndex(Entry->ChunkId, MoveTemp(Entry->Options.FileName));
		}

		const uint64 RegionStartOffset = TargetPartition->Offset;
		TargetPartition->Offset += Entry->DiskSize;
		UncompressedFileOffset += Align(Entry->UncompressedSize.GetValue(), WriterSettings.CompressionBlockSize);
		TotalEntryUncompressedSize += Entry->UncompressedSize.GetValue();

		if (WriterSettings.bEnableFileRegions)
		{
			if (WriterSettings.bUsePatchInPlaceLayout)
			{
				TConstArrayView<FFileRegion> FileRegions = Entry->Request->GetRegions();
				if (!FileRegions.IsEmpty())
				{
					// make a copy as entry is deleted after this method returns
					PatchInPlaceFileRegions.Emplace(Entry->ChunkId, FileRegions);
				}
			}
			else
			{
				FFileRegion::AccumulateFileRegions(TargetPartition->AllFileRegions, RegionStartOffset, RegionStartOffset, TargetPartition->Offset, Entry->Request->GetRegions());
			}
		}
		uint64 WriteStartCycles = FPlatformTime::Cycles64();
		uint64 WriteBytes = 0;
		if (Entry->Padding > 0)
		{
			if (PaddingBuffer.Num() < Entry->Padding)
			{
				PaddingBuffer.SetNumZeroed(int32(Entry->Padding));
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WritePaddingToContainer);
				TargetPartition->ContainerFileHandle->Serialize(PaddingBuffer.GetData(), Entry->Padding);
				WriteBytes += Entry->Padding;
			}
		}
		check(Entry->Offset == TargetPartition->ContainerFileHandle->Tell());
		for (FChunkBlock& ChunkBlock : Entry->ChunkBlocks)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WriteBlockToContainer);
			TargetPartition->ContainerFileHandle->Serialize(ChunkBlock.IoBuffer->Data(), ChunkBlock.DiskSize);
			WriteBytes += ChunkBlock.DiskSize;
		}
		uint64 WriteEndCycles = FPlatformTime::Cycles64();
		WriterContext->WriteCycleCount.AddExchange(WriteEndCycles - WriteStartCycles);
		WriterContext->WriteByteCount.AddExchange(WriteBytes);
		WriterContext->SerializedChunksCount.IncrementExchange();
	}

	const FString				ContainerPathAndBaseFileName;
	FIoStoreWriterContextImpl*	WriterContext = nullptr;
	FIoContainerSettings		ContainerSettings;
	FString						TocFilePath;
	FIoStoreTocBuilder			TocBuilder;
	TArray<uint8>				PaddingBuffer;
	TArray<FPartition>			Partitions;
	TArray<FIoStoreWriteQueueEntry*> Entries;
	TArray<FIoStoreDeferredDuplicate> DeferredDuplicates;
	TArray<FLayoutEntry*>		LayoutEntries;
	FLayoutEntry*				LayoutEntriesHead = nullptr;
	FLayoutEntry*				LayoutEntriesTail = nullptr;
	TMap<FIoChunkId, FLayoutEntry*> PreviousBuildLayoutEntryByChunkId;
	TMap<FIoChunkId, TArray<FFileRegion>> PatchInPlaceFileRegions;
	TUniquePtr<FArchive>		CsvArchive;
	FIoStoreWriterResult		Result;
	uint64						UncompressedFileOffset = 0;
	uint64						TotalEntryUncompressedSize = 0; // sum of all entry source buffer sizes
	uint64						TotalEntryCompressedSize = 0; // entry compressed size excluding encryption alignment
	uint64						ReferenceCacheMissBytes = 0; // number of compressed bytes excluding alignment that could have been from refcache but weren't.
	uint64						TotalPaddingSize = 0;
	uint64						UncompressedContainerSize = 0; // this is the size the container would be if it were uncompressed.
	uint64						CompressedContainerSize = 0; // this is the size of the container with the given compression (which may be none).
	uint64						PatchInPlaceSizeBeforeLayout = 0;
	uint64						PatchInPlacePinnedSizeBeforeDefrag = 0;
	uint64						PatchInPlacePinnedSizeAfterDefrag = 0;
	int64						PatchInPlaceLocalityChange = 0;
	uint64						MaxPartitionSize = 0;
	int32						CurrentPartitionIndex = 0;
	bool						bHasMemoryMappedEntry = false;
	bool						bHasFlushed = false;
	bool						bHasResult = false;
	TSharedPtr<IIoStoreWriterReferenceChunkDatabase> ReferenceChunkDatabase;
	TSharedPtr<IIoStoreWriterReferenceChunkDatabase> PatchInPlaceReferenceChunkDatabase;


	friend class FIoStoreWriterContextImpl;
};

// InContainerPathAndBaseFileName: the utoc file will just be this with .utoc appended.
// The base filename ends up getting returned as the container name in the writer results.
TSharedPtr<IIoStoreWriter> FIoStoreWriterContextImpl::CreateContainer(const TCHAR* InContainerPathAndBaseFileName, const FIoContainerSettings& InContainerSettings)
{
	TSharedPtr<FIoStoreWriter> IoStoreWriter = MakeShared<FIoStoreWriter>(InContainerPathAndBaseFileName);
	FIoStatus IoStatus = IoStoreWriter->Initialize(*this, InContainerSettings);
	check(IoStatus.IsOk());
	IoStoreWriters.Add(IoStoreWriter);
	return IoStoreWriter;
}

void FIoStoreWriterContextImpl::FinalizeLayout()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreWriterContext::FinalizeLayout);
	TArray<FIoStoreWriteQueueEntry*> AllEntries;
	for (TSharedPtr<FIoStoreWriter> IoStoreWriter : IoStoreWriters)
	{
		IoStoreWriter->bHasFlushed = true;
		AllEntries.Append(IoStoreWriter->Entries);
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(WaitForChunkHashes);
		for (int32 EntryIndex = AllEntries.Num() - 1; EntryIndex >= 0; --EntryIndex)
		{
			AllEntries[EntryIndex]->HashTask.Wait();
		}
	}
	for (TSharedPtr<FIoStoreWriter> IoStoreWriter : IoStoreWriters)
	{
		if (IoStoreWriter->LayoutEntriesHead)
		{
			IoStoreWriter->FinalizeLayout();
		}
	}
}

void FIoStoreWriterContextImpl::FinalizeWrites()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreWriterContext::FinalizeWrites);
	TArray<FIoStoreWriteQueueEntry*> AllEntries;
	for (TSharedPtr<FIoStoreWriter> IoStoreWriter : IoStoreWriters)
	{
		AllEntries.Append(IoStoreWriter->Entries);
	}

	// Start scheduler threads, enqueue all entries, and wait for them to finish
	{
		double WritesStart = FPlatformTime::Seconds();

		BeginCompressionThread = Async(EAsyncExecution::Thread, [this]() { BeginCompressionThreadFunc(); });
		BeginEncryptionAndSigningThread = Async(EAsyncExecution::Thread, [this]() { BeginEncryptionAndSigningThreadFunc(); });
		WriterThread = Async(EAsyncExecution::Thread, [this]() { WriterThreadFunc(); });

		ScheduleAllEntries(AllEntries);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForWritesToComplete);
			WriterThread.Wait();
		}

		double WritesEnd = FPlatformTime::Seconds();
		double WritesSeconds = FPlatformTime::ToSeconds64(WriteCycleCount.Load());
		UE_LOGF(LogIoStore, Display, "Writing and compressing took %.2lf seconds, writes to disk took %.2lf seconds for %ls bytes @ %ls bytes per second.", 
			WritesEnd - WritesStart,
			WritesSeconds,
			*FText::AsNumber(WriteByteCount.Load()).ToString(),
			*FText::AsNumber((int64)((double)WriteByteCount.Load() / FMath::Max(.0001f, WritesSeconds))).ToString()
			);
	}

	// Classically there were so few writers that this didn't need to be multi threaded, but it
	// involves writing files, and with content on demand this ends up being thousands of iterations. 
	double FinalizeStart = FPlatformTime::Seconds();	
	ParallelFor(TEXT("IoStoreWriter::Finalize.PF"), IoStoreWriters.Num(), 1, [this](int Index)
	{ 
		IoStoreWriters[Index]->Finalize(); 
	}, EParallelForFlags::Unbalanced);
	double FinalizeEnd = FPlatformTime::Seconds();
	int64 TotalTocSize = 0;
	for (TSharedPtr<IIoStoreWriter> Writer : IoStoreWriters)
	{
		if (Writer->GetResult().IsOk())
		{
			TotalTocSize += Writer->GetResult().ValueOrDie().TocSize;
		}
	}

	UE_LOGF(LogIoStore, Display, "Finalize took %.1f seconds for %d writers to write %ls bytes, %ls bytes per second",
		FinalizeEnd - FinalizeStart,
		IoStoreWriters.Num(),
		*FText::AsNumber(TotalTocSize).ToString(),
		*FText::AsNumber((int64)((double)TotalTocSize / FMath::Max(.0001f, FinalizeEnd - FinalizeStart))).ToString()
		);
}


UE::DerivedData::FCacheKey FIoStoreWriterContextImpl::MakeDDCKey(FIoStoreWriteQueueEntry* Entry) const
{
	TStringBuilder<256> CacheKeySuffix;
	CacheKeySuffix << IoStoreDDCVersion;
	CacheKeySuffix << Entry->ChunkHash;
	CacheKeySuffix.Append(FCompression::GetCompressorDDCSuffix(Entry->CompressionMethod));
	CacheKeySuffix.Appendf(TEXT("%llu_%llu_%d_%d"),
		WriterSettings.CompressionBlockSize,
		CompressionBufferSize,
		WriterSettings.CompressionMinBytesSaved,
		WriterSettings.CompressionMinPercentSaved);

	return { IoStoreDDCBucket, FIoHash::HashBuffer(MakeMemoryView(FTCHARToUTF8(CacheKeySuffix.ToString()))) };
}

void FIoStoreWriterContextImpl::ScheduleAllEntries(TArrayView<FIoStoreWriteQueueEntry*> AllEntries)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ScheduleAllEntries);

	const auto HandleDDCGetResult = [this](FIoStoreWriteQueueEntry* Entry, FSharedBuffer Result)
	{
		bool bFoundInDDC = false;
		uint64 CompressedSize = 0;
		if (!Result.IsNull())
		{
			FLargeMemoryReader DDCDataReader((uint8*)Result.GetData(), Result.GetSize());
			bFoundInDDC = Entry->Writer->SerializeCompressedDDCData(Entry, DDCDataReader, &CompressedSize);

			UE_CLOGF(!bFoundInDDC, LogIoStore, Warning,
				"Ignoring invalid DDC data for ChunkId=%ls, DDCKey=%ls, UncompressedSize=%llu, NumChunkBlocks=%d",
				*LexToString(Entry->ChunkId),
				*WriteToString<96>(Entry->DDCKey),
				Entry->UncompressedSize.Get(0),
				Entry->NumChunkBlocks);
		}
		if (bFoundInDDC)
		{
			Entry->bFoundInDDC = true;
			CompressionDDCHitsByType[(int8)Entry->ChunkId.GetChunkType()].IncrementExchange();
			CompressionDDCGetBytes.AddExchange(CompressedSize);
			TRACE_COUNTER_INCREMENT(IoStoreDDCHitCount);
			Entry->BeginCompressionBarrier.Trigger();
		}
		else
		{
			Entry->bStoreCompressedDataInDDC = true;
			CompressionDDCMissCount.IncrementExchange();
			TRACE_COUNTER_INCREMENT(IoStoreDDCMissCount);
			// kick off source buffer read, and proceed to begin compression
			Entry->Request->PrepareSourceBufferAsync(Entry->BeginCompressionBarrier);
		}
	};

	FIoStoreDDCGetRequestDispatcher DDCGetRequestDispatcher(FIoStoreDDCRequestDispatcherParams{});

	for (FIoStoreWriteQueueEntry* Entry : AllEntries)
	{
		uint64 ScheduledMemory = ScheduledCompressionMemory.Load();
		const uint64 EntryMemoryEstimate = Entry->Request->GetSourceBufferSizeEstimate();
		
		while (ScheduledMemory > 0 && ScheduledMemory + EntryMemoryEstimate > MaxCompressionBufferMemory)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(WaitForCompressionMemory);
			if (!CompressionMemoryReleasedEvent->Wait(100.f))
			{
				// if the event timed out,
				// make sure we are not waiting for unsubmitted ddc requests
				DDCGetRequestDispatcher.DispatchGetRequests(HandleDDCGetResult);
			}
			ScheduledMemory = ScheduledCompressionMemory.Load();
		}

		ScheduledMemory = ScheduledCompressionMemory.AddExchange(EntryMemoryEstimate);
		TRACE_COUNTER_SET(IoStoreCompressionMemoryScheduled, ScheduledMemory + EntryMemoryEstimate);

		if (Entry->bLoadingFromReferenceDb)
		{
			Entry->Writer->LoadFromReferenceDb(Entry);
		}
		else if (Entry->bUseDDCForCompression)
		{
			Entry->DDCKey = MakeDDCKey(Entry);
			DDCGetRequestDispatcher.EnqueueGetRequest(Entry);
		}
		else
		{
			Entry->Request->PrepareSourceBufferAsync(Entry->BeginCompressionBarrier);
		}

		DDCGetRequestDispatcher.DispatchGetRequests(HandleDDCGetResult);
		BeginCompressionQueue.Enqueue(Entry);
	}

	DDCGetRequestDispatcher.FlushGetRequests(HandleDDCGetResult);
	BeginCompressionQueue.CompleteAdding();
}

void FIoStoreWriterContextImpl::BeginCompressionThreadFunc()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BeginCompressionThread);
	for (;;)
	{
		FIoStoreWriteQueueEntry* Entry = BeginCompressionQueue.DequeueOrWait();
		if (!Entry)
		{
			break;
		}
		while (Entry)
		{
			FIoStoreWriteQueueEntry* Next = Entry->Next;
			Entry->BeginCompressionBarrier.Wait();
			TRACE_COUNTER_INCREMENT(IoStoreBeginCompressionCount);
			Entry->Writer->BeginCompress(Entry);
			BeginEncryptionAndSigningQueue.Enqueue(Entry);
			Entry = Next;
		}
	}
	BeginEncryptionAndSigningQueue.CompleteAdding();
}

void FIoStoreWriterContextImpl::BeginEncryptionAndSigningThreadFunc()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BeginEncryptionAndSigningThread);

	const auto HandleDDCPutResult = [this](FIoStoreWriteQueueEntry* Entry, bool bSuccess)
	{
		if (bSuccess)
		{
			TRACE_COUNTER_INCREMENT(IoStoreDDCPutCount);
			CompressionDDCPutsByType[(int8)Entry->ChunkId.GetChunkType()].IncrementExchange();
			CompressionDDCPutBytes.AddExchange(Entry->CompressedSize);
		}
		else
		{
			CompressionDDCPutErrorCount.IncrementExchange();
		}
	};

	FIoStoreDDCRequestDispatcherParams PutRequestDispatcherParams;
	PutRequestDispatcherParams.QueueTimeLimitMs = 1000.f;
	FIoStoreDDCPutRequestDispatcher DDCPutRequestDispatcher(PutRequestDispatcherParams);

	for (;;)
	{
		FIoStoreWriteQueueEntry* Entry = BeginEncryptionAndSigningQueue.DequeueOrWait();
		if (!Entry)
		{
			break;
		}
		while (Entry)
		{
			FIoStoreWriteQueueEntry* Next = Entry->Next;
			Entry->FinishCompressionBarrier.Wait();
			
			if (Entry->bStoreCompressedDataInDDC)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(AddDDCPutRequest);
				TArray64<uint8> DDCData;
				FMemoryWriter64 DDCDataWriter(DDCData, true);
				DDCData.Reserve(16 + 8 * Entry->ChunkBlocks.Num() + Entry->CompressedSize);
				if (Entry->Writer->SerializeCompressedDDCData(Entry, DDCDataWriter))
				{
					DDCPutRequestDispatcher.EnqueuePutRequest(Entry, MakeSharedBufferFromArray(MoveTemp(DDCData)));
				}
				else
				{
					CompressionDDCPutErrorCount.IncrementExchange();
				}
			}
			DDCPutRequestDispatcher.DispatchPutRequests(HandleDDCPutResult);

			// Must be done after we have serialized the compressed data for DDC as it can potentially modify the
			// data stored by Entry!
			TRACE_COUNTER_INCREMENT(IoStoreBeginEncryptionAndSigningCount);
			Entry->Writer->BeginEncryptAndSign(Entry);

			WriterQueue.Enqueue(Entry);
			Entry = Next;
		}
	}
	DDCPutRequestDispatcher.FlushPutRequests(HandleDDCPutResult);
	WriterQueue.CompleteAdding();
}

void FIoStoreWriterContextImpl::WriterThreadFunc()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(WriterThread);
	for (;;)
	{
		FIoStoreWriteQueueEntry* Entry = WriterQueue.DequeueOrWait();
		if (!Entry)
		{
			return;
		}
		while (Entry)
		{
			FIoStoreWriteQueueEntry* Next = Entry->Next;
			Entry->BeginWriteBarrier.Wait();
			TRACE_COUNTER_INCREMENT(IoStoreBeginWriteCount);
			Entry->Writer->WriteEntry(Entry);
			Entry = Next;
		}
	}
}

