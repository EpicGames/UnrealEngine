// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IO/IoContainerId.h"
#include "IO/IoDispatcher.h"
#include "UObject/NameTypes.h"

class FIoStoreWriterContextImpl;

struct FIoStoreWriterSettings
{
	FName CompressionMethod = NAME_None;
	uint64 CompressionBlockSize = 64 << 10;

	// This does not align every entry - it tries to prevent excess crossings of this boundary by inserting padding.
	// and happens whether or not the entry is compressed.
	uint64 CompressionBlockAlignment = 0;
	int32 CompressionMinBytesSaved = 0;
	int32 CompressionMinPercentSaved = 0;
	int32 CompressionMinSizeToConsiderDDC = 0;
	uint64 MemoryMappingAlignment = 0;
	uint64 MaxPartitionSize = 0;
	uint64 PatchInPlaceMaximumTotalGrowth = 0;        // fail build if total overhead goes over this value and bUsePatchInPlaceLayout is enabled, 0 to ignore
	uint64 PatchInPlaceMaximumPerContainerGrowth = 0; // defragment container if it's overhead goes above this value by moving some entries, 0 to ignore
	uint64 PatchInPlaceParetoFrontCoarseSamples = 0;  // amount of coarse samples to use for pareto front probing during defrag
	uint64 PatchInPlaceParetoFrontFineSamples = 0;    // amount of fine samples to fine tune pareto front during defrag
	uint64 PatchInPlaceReadGranularity = 0;           // group inserted entries with-in read granularity bins based on offsets from file open order, 0 to ignore
	uint64 PatchInPlaceMaxChainSize = 0;              // largest possible insertion chain size, 0 to ignore
	int32 PatchInPlaceMaxChainLength = 0;             // largest possible insertion chain length, 0 to ignore
	uint8 PatchInPlaceParetoFrontHeuristic = 0;       // heuristic to choose for pareto front, see FPatchInPlaceLayout::BuildUnpinCandidates
	TMap<FString, uint64> MaxPartitionSizeOverride;
	bool bEnableFileRegions = false;
	bool bContainerMetaEnabled = false;
	bool bPerContainerMeta = false;
	bool bCompressionEnableDDC = false;
	bool bValidateChunkHashes = false;
	bool bUsePatchInPlaceLayout = false;
	bool bDeduplicateChunks = false;
};

struct FIoStoreWriterResult
{
	FIoContainerId ContainerId;
	FString ContainerName; // This is the base filename of the utoc used for output.
	int64 TocSize = 0;
	int64 TocEntryCount = 0;
	int64 PaddingSize = 0;
	int64 UncompressedContainerSize = 0; // this is the size the container would be if it were uncompressed.
	int64 CompressedContainerSize = 0; // this is the size of the container with the given compression (which may be none). Should be the sum of all partition file sizes.
	int64 DirectoryIndexSize = 0;
	uint64 TotalEntryCompressedSize = 0; // sum of the compressed size of entries excluding encryption alignment.
	uint64 ReferenceCacheMissBytes = 0; // number of compressed bytes excluding alignment that could have been from refcache but weren't.
	uint64 PatchInPlaceSizeBeforeLayout = 0;
	uint64 PatchInPlacePinnedSizeBeforeDefrag = 0;
	uint64 PatchInPlacePinnedSizeAfterDefrag = 0;
	int64 PatchInPlaceLocalityChange = 0;
	uint64 AddedChunksCount = 0;
	uint64 AddedChunksSize = 0;
	uint64 ModifiedChunksCount = 0;
	uint64 ModifiedChunksSize = 0;
	uint64 DeduplicatedSavedBytes = 0;
	FName CompressionMethod = NAME_None;
	EIoContainerFlags ContainerFlags = EIoContainerFlags::None;
};

struct FIoWriteOptions
{
	FString FileName;
	const TCHAR* DebugName = nullptr;
	// If valid, this chunk is byte-identical to specified chunk.
	FIoChunkId DuplicateOf = {};
	bool bForceUncompressed = false;
	bool bIsMemoryMapped = false;
};

class FIoStoreWriterContext
{
public:
	struct FProgress
	{
		uint64 TotalChunksCount = 0;
		uint64 HashedChunksCount = 0;
		// Number of chunks where we avoided reading and hashing, and instead used the result from the hashdb, and their types
		uint64 HashDbChunksCount = 0;
		uint64 HashDbChunksByType[(int8)EIoChunkType::MAX] = { 0 };
		// Number of chunks that were passed to the compressor (i.e. passed the various opt-outs), and their types
		uint64 CompressedChunksCount = 0;
		uint64 CompressedChunksByType[(int8)EIoChunkType::MAX] = { 0 };
		uint64 SerializedChunksCount = 0;
		uint64 ScheduledCompressionTasksCount = 0;
		uint64 ScheduledCompressionMemoryBytes = 0;
		uint64 CompressionDDCHitsByType[(int8)EIoChunkType::MAX] = { 0 };
		uint64 CompressionDDCPutsByType[(int8)EIoChunkType::MAX] = { 0 };
		uint64 CompressionDDCHitCount = 0;
		uint64 CompressionDDCMissCount = 0;
		uint64 CompressionDDCPutCount = 0;
		uint64 CompressionDDCPutErrorCount = 0;
		uint64 CompressionDDCGetBytes = 0;
		uint64 CompressionDDCPutBytes = 0;

		// The number of chunk retrieved from the reference cache database, and their types.
		uint64 RefDbChunksCount{ 0 };
		uint64 RefDbChunksByType[(int8)EIoChunkType::MAX] = { 0 };
		
		// The type of chunk that landed in BeginCompress before any opt-outs.
		uint64 BeginCompressChunksByType[(int8)EIoChunkType::MAX] = { 0 };
	};

	FIoStoreWriterContext();
	~FIoStoreWriterContext();

	[[nodiscard]] FIoStatus Initialize(const FIoStoreWriterSettings& InWriterSettings);
	TSharedPtr<class IIoStoreWriter> CreateContainer(const TCHAR* InContainerPath, const FIoContainerSettings& InContainerSettings);
	void FinalizeLayout();
	void FinalizeWrites();
	FProgress GetProgress() const;
	uint32 GetErrors() const;
	void SaveContainerMeta(const FString& FilePath);

private:
	FIoStoreWriterContextImpl* Impl;
};

class IIoStoreWriteRequest
{
public:
	virtual ~IIoStoreWriteRequest() = default;

	virtual uint64 GetOrderHint() = 0;
	virtual TArrayView<const FFileRegion> GetRegions() = 0;
	virtual const FIoHash* GetChunkHash() = 0;

	// Launches any async operations necessary in order to access the buffer. CompletionEvent is set once it's ready, which may be immediate.
	virtual void PrepareSourceBufferAsync(UE::Tasks::FTaskEvent& CompletionEvent) = 0;

	// Only valid after the completion event passed to PrepareSourceBufferAsync has fired.
	virtual const FIoBuffer* GetSourceBuffer() = 0;

	// Can't be called between PrepareSourceBufferAsync and its completion!
	virtual void FreeSourceBuffer() = 0;

	virtual uint64 GetSourceBufferSizeEstimate() = 0;

	// Return an identifier for the repository that is providing the source data, e.g. loosefile, zenserver.
	virtual const TCHAR* DebugNameOfRepository() const = 0;
};

class IIoStoreWriterReferenceChunkDatabase
{
public:
	virtual ~IIoStoreWriterReferenceChunkDatabase() = default;

	/*
	* Used by IIoStoreWriter to check and see if there's a reference chunk that matches the data that
	* IoStoreWriter wants to compress and write. Validity checks must be synchronous - if a chunk can't be
	* used for some reason (no matching chunk exists or otherwise), this function must return false and not 
	* call InCompletionCallback.
	* 
	* Once a matching chunk is found, it is read from the source iostore container asynchronously, and upon
	* completion InCompletionCallback is called with the raw output from FIoStoreReader::ReadCompressed (i.e.
	* FIoStoreCompressedReadResult). Failures once the async read process has started are currently fatal due to
	* difficulties in rekicking a read.
	* 
	* For the moment, changes in compression method are allowed.
	* 
	* RetrieveChunk is not currently thread safe and must be called from a single thread.
	* 
	* Chunks provided *MUST* decompress to bits that hash to the exact value provided in InChunkKey (i.e. be exactly the same bits),
	* and also be the same number of blocks (i.e. same CompressionBlockSize)
	*/
	virtual UE::Tasks::FTask RetrieveChunk(const FIoContainerId& InContainerId, const FIoHash& InChunkHash, const FIoChunkId& InChunkId, TUniqueFunction<void(TIoStatusOr<FIoStoreCompressedReadResult>)> InCompletionCallback) = 0;

	/* 
	* Quick synchronous existence check that returns the number of blocks for the chunk. This is used to set up
	* the necessary structures without needing to read the source data for the chunk. This might be called from
	* multiple threads as it has to happen after we have the source hash computed.
	*/
	virtual bool ChunkExists(const FIoContainerId& InContainerId, const FIoHash& InChunkHash, const FIoChunkId& InChunkId, int32& OutNumChunkBlocks) = 0;

	/*
	* Returns the compression block size that was used to break up the IoChunks in the source containers. If this is different than what we want, 
	* then none of the chunks will ever match. Knowing this up front allows us to only match on hash
	*/
	virtual uint32 GetCompressionBlockSize() const = 0;

	/*
	* Called by an iostore writer implementation to notify the ref cache it's been added
	*/
	virtual void NotifyAddedToWriter(const FIoContainerId& InContainerId, const FString& InContainerName) = 0;

	/*
	* Returns true if ContainerId was found and OutContainerFilePath is set.
	*/
	virtual bool GetContainerFilePath(const FIoContainerId& InContainerId, FString& OutContainerFilePath) = 0;
};

class IIoStoreWriter
{
public:
	virtual ~IIoStoreWriter() = default;

	/**
	*	If a reference database is provided, the IoStoreWriter implementation may elect to reuse compressed blocks
	*	from previous containers instead of recompressing input data. This must be set before any writes are appended.
	*/
	virtual void SetReferenceChunkDatabase(TSharedPtr<IIoStoreWriterReferenceChunkDatabase> ReferenceChunkDatabase, TSharedPtr<IIoStoreWriterReferenceChunkDatabase> PatchInPlaceReferenceChunkDatabase) = 0;
	virtual void EnableDiskLayoutOrdering(const TArray<TUniquePtr<FIoStoreReader>>& PatchSourceReaders = TArray<TUniquePtr<FIoStoreReader>>()) = 0;
	virtual void Append(const FIoChunkId& ChunkId, FIoBuffer Chunk, const FIoWriteOptions& WriteOptions, uint64 OrderHint = MAX_uint64) = 0;
	virtual void Append(const FIoChunkId& ChunkId, IIoStoreWriteRequest* Request, const FIoWriteOptions& WriteOptions) = 0;
	virtual TIoStatusOr<FIoStoreWriterResult> GetResult() = 0;
	virtual void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const = 0;
};

