// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayeredCookArtifactReader.h"

#include "HAL/CriticalSection.h"
#include "Misc/ScopeRWLock.h"

void FLayeredCookArtifactReader::AddLayer(TSharedRef<UE::Cook::ICookArtifactReader> InLayer)
{
	Layers.AddUnique(InLayer);
}

bool FLayeredCookArtifactReader::RemoveLayer(TSharedRef<UE::Cook::ICookArtifactReader> InLayer)
{
	return Layers.Remove(InLayer) > 0;
}

void FLayeredCookArtifactReader::EmptyLayers()
{
	Layers.Empty();
}

bool FLayeredCookArtifactReader::FileExists(const TCHAR* Filename)
{
	for (const TSharedRef<UE::Cook::ICookArtifactReader>& Layer : Layers)
	{
		if (Layer->FileExists(Filename))
		{
			return true;
		}
	}
	return false;
}

int64 FLayeredCookArtifactReader::FileSize(const TCHAR* Filename)
{
	for (const TSharedRef<UE::Cook::ICookArtifactReader>& Layer : Layers)
	{
		int64 Size = Layer->FileSize(Filename);
		if (Size >= 0)
		{
			return Size;
		}
	}

	return -1;
}

IFileHandle* FLayeredCookArtifactReader::OpenRead(const TCHAR* Filename)
{
	for (const TSharedRef<UE::Cook::ICookArtifactReader>& Layer : Layers)
	{
		if (IFileHandle* File = Layer->OpenRead(Filename))
		{
			return File;
		}
	}
	return nullptr;
}

struct FDirectoryEntry
{
	FString Name;
	bool bIsDirectory;
};

class FMergingVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	FRWLock FoundEntriesLock;
	TMap<FString, bool>& FoundEntries;
	FMergingVisitor(TMap<FString, bool>& InFoundEntries)
		: IPlatformFile::FDirectoryVisitor(EDirectoryVisitorFlags::ThreadSafe)
		, FoundEntries(InFoundEntries)
	{
	}

	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		FString FileName(FilenameOrDirectory);
		FRWScopeLock ScopeLock(FoundEntriesLock, SLT_Write);
		FoundEntries.FindOrAdd(MoveTemp(FileName), bIsDirectory);
		return true;
	}
};

bool FLayeredCookArtifactReader::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	TMap<FString, bool> Entries;
	FMergingVisitor MergingVisitor(Entries);
	bool bRetVal = false;

	for (const TSharedRef<UE::Cook::ICookArtifactReader>& Layer : Layers)
	{
		bRetVal |= Layer->IterateDirectory(Directory, MergingVisitor);
	}

	for (const TPair<FString, bool>& Entry : Entries)
	{
		if (!Visitor.CallShouldVisitAndVisit(*Entry.Key, Entry.Value))
		{
			return  bRetVal;
		}
	}
	return bRetVal;
}
