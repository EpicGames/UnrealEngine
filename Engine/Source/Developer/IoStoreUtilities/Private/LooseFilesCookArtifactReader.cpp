// Copyright Epic Games, Inc. All Rights Reserved.

#include "LooseFilesCookArtifactReader.h"
#include "HAL/PlatformFileManager.h"

bool FLooseFilesCookArtifactReader::FileExists(const TCHAR* Filename)
{
	return FPlatformFileManager::Get().GetPlatformFile().FileExists(Filename);
}

int64 FLooseFilesCookArtifactReader::FileSize(const TCHAR* Filename)
{
	return FPlatformFileManager::Get().GetPlatformFile().FileSize(Filename);
}

IFileHandle* FLooseFilesCookArtifactReader::OpenRead(const TCHAR* Filename)
{
	return FPlatformFileManager::Get().GetPlatformFile().OpenRead(Filename);
}

bool FLooseFilesCookArtifactReader::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	return FPlatformFileManager::Get().GetPlatformFile().IterateDirectory(Directory, Visitor);
}
