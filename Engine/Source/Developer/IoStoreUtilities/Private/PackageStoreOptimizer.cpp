// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageStoreOptimizer.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/BufferWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UObject/NameBatchSerialization.h"
#include "Containers/Map.h"
#include "UObject/UObjectHash.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/Object.h"
#include "Serialization/MemoryReader.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "Serialization/LargeMemoryReader.h"
#include "VerseVM/VVMVerse.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMContext.h"
#include "VerseVM/VVMCppClassInfo.h"
#include "VerseVM/VVMGlobalProgram.h"
#include "VerseVM/VVMUniqueString.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogPackageStoreOptimizer, Log, All);

// modified copy from SavePackage
EObjectMark GetExcludedObjectMarksForTargetPlatform(const ITargetPlatform* TargetPlatform)
{
	EObjectMark Marks = OBJECTMARK_NotForTargetPlatform;
	if (!TargetPlatform->AllowsEditorObjects())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_EditorOnly);
	}
	if (TargetPlatform->IsServerOnly())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForServer);
	}
	if (TargetPlatform->IsClientOnly())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForClient);
	}
	return Marks;
}

// modified copy from SavePackage
EObjectMark GetExcludedObjectMarksForObject(const UObject* Object, const ITargetPlatform* TargetPlatform)
{
	EObjectMark Marks = OBJECTMARK_NOMARKS;
	if (!Object->NeedsLoadForClient())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForClient);
	}
	if (!Object->NeedsLoadForServer())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForServer);
	}
#if WITH_ENGINE
	// NotForServer && NotForClient implies EditorOnly
	const bool bIsEditorOnlyObject = (Marks & OBJECTMARK_NotForServer) && (Marks & OBJECTMARK_NotForClient);
	const bool bTargetAllowsEditorObjects = TargetPlatform->AllowsEditorObjects();

	// no need to query the target platform if the object is editoronly and the targetplatform doesn't allow editor objects 
	const bool bCheckTargetPlatform = !bIsEditorOnlyObject || bTargetAllowsEditorObjects;
	if (bCheckTargetPlatform && (!Object->NeedsLoadForTargetPlatform(TargetPlatform) || !TargetPlatform->AllowObject(Object)))
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForTargetPlatform);
	}
#endif
	if (Object->IsEditorOnly())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_EditorOnly);
	}
	if ((Marks & OBJECTMARK_NotForClient) && (Marks & OBJECTMARK_NotForServer))
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_EditorOnly);
	}
	return Marks;
}

void FPackageStoreOptimizer::Initialize()
{
	FindScriptObjects();
}

void FPackageStoreOptimizer::Initialize(const FIoBuffer& ScriptObjectsBuffer)
{
	LoadScriptObjectsBuffer(ScriptObjectsBuffer);
}

FPackageStorePackage* FPackageStoreOptimizer::CreateMissingPackage(const FName& Name) const
{
	FPackageStorePackage* Package = new FPackageStorePackage();
	Package->Name = Name;
	Package->Id = FPackageId::FromName(Name);
	return Package;
}

FPackageStorePackage* FPackageStoreOptimizer::CreatePackageFromCookedHeader(const FName& Name, const FIoBuffer& CookedHeaderBuffer) const
{
	FPackageStorePackage* Package = new FPackageStorePackage();
	Package->Id = FPackageId::FromName(Name);
	Package->Name = Name;

	FCookedHeaderData CookedHeaderData = LoadCookedHeader(CookedHeaderBuffer);
	if (!CookedHeaderData.Summary.bUnversioned)
	{
		FZenPackageVersioningInfo& VersioningInfo = Package->VersioningInfo.Emplace();
		VersioningInfo.ZenVersion = EZenPackageVersion::Latest;
		VersioningInfo.PackageVersion = CookedHeaderData.Summary.GetFileVersionUE();
		VersioningInfo.LicenseeVersion = CookedHeaderData.Summary.GetFileVersionLicenseeUE();
		VersioningInfo.CustomVersions = CookedHeaderData.Summary.GetCustomVersionContainer();
	}
	Package->PackageFlags = CookedHeaderData.Summary.GetPackageFlags();
	Package->CookedHeaderSize = CookedHeaderData.Summary.TotalHeaderSize;
	for (int32 I = 0; I < CookedHeaderData.Summary.NamesReferencedFromExportDataCount; ++I)
	{
		Package->NameMapBuilder.AddName(CookedHeaderData.SummaryNames[I]);
	}

	Package->SoftPackageReferences = MoveTemp(CookedHeaderData.SoftPackageReferences);

	TArray<FPackageStorePackage::FUnresolvedImport> UnresolvedImports;
	TArray<FPackageStorePackage::FUnresolvedExport> UnresolvedExports;

	const int32 ImportCount = CookedHeaderData.ObjectImports.Num();
	const int32 ExportCount = CookedHeaderData.ObjectExports.Num();
	const int32 CellImportCount = CookedHeaderData.CellImports.Num();	
	const int32 CellExportCount = CookedHeaderData.CellExports.Num();

	UnresolvedImports.SetNum(ImportCount);
	UnresolvedExports.SetNum(ExportCount + CellExportCount);

	Package->Imports.SetNum(ImportCount);
	Package->Exports.SetNum(ExportCount);
	Package->CellImports.SetNum(CellImportCount);
	Package->CellExports.SetNum(CellExportCount);
	Package->ExportGraphNodes.Reserve((ExportCount + CellExportCount) * 2);

	ProcessImports(CookedHeaderData, Package, UnresolvedImports, UnresolvedExports);
	ProcessExports(CookedHeaderData, Package, UnresolvedImports, UnresolvedExports);
	ProcessPreloadDependencies(CookedHeaderData, Package);
	ProcessDataResources(CookedHeaderData, Package);

	CreateExportBundle(Package);

	FinalizePackageHeader(Package);

	return Package;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FPackageStoreOptimizer::FCookedHeaderData FPackageStoreOptimizer::LoadCookedHeader(const FIoBuffer& CookedHeaderBuffer) const
{
	FCookedHeaderData CookedHeaderData;
	TArrayView<const uint8> MemView(CookedHeaderBuffer.Data(), CookedHeaderBuffer.DataSize());
	FMemoryReaderView Ar(MemView);

	FPackageFileSummary& Summary = CookedHeaderData.Summary;
	{
		TGuardValue<int32> GuardAllowUnversionedContentInEditor(GAllowUnversionedContentInEditor, 1);
		Ar << Summary;
	}

	Ar.SetUseUnversionedPropertySerialization((CookedHeaderData.Summary.GetPackageFlags() & EPackageFlags::PKG_UnversionedProperties) != 0);
	Ar.SetFilterEditorOnly((CookedHeaderData.Summary.GetPackageFlags() & EPackageFlags::PKG_FilterEditorOnly) != 0);

	if (Summary.NameCount > 0)
	{
		Ar.Seek(Summary.NameOffset);

		FNameEntrySerialized NameEntry(ENAME_LinkerConstructor);

		CookedHeaderData.SummaryNames.Reserve(Summary.NameCount);
		for (int32 I = 0; I < Summary.NameCount; ++I)
		{
			Ar << NameEntry;
			CookedHeaderData.SummaryNames.Add(NameEntry);
		}
	}

	class FNameReaderProxyArchive
		: public FArchiveProxy
	{
	public:
		using FArchiveProxy::FArchiveProxy;

		FNameReaderProxyArchive(FArchive& InAr, const TArray<FName>& InNameMap)
			: FArchiveProxy(InAr)
			, NameMap(InNameMap)
		{
			// Replicate the filter editor only state of the InnerArchive as FArchiveProxy will
			// not intercept it.
			FArchive::SetFilterEditorOnly(InAr.IsFilterEditorOnly());
		}

		FArchive& operator<<(FName& Name)
		{
			int32 NameIndex = 0;
			int32 Number = 0;
			InnerArchive << NameIndex << Number;

			if (!NameMap.IsValidIndex(NameIndex))
			{
				UE_LOGF(LogPackageStoreOptimizer, Fatal, "Bad name index %i/%i", NameIndex, NameMap.Num());
			}

			const FName& MappedName = NameMap[NameIndex];
			Name = FName::CreateFromDisplayId(MappedName.GetDisplayIndex(), Number);

			return *this;
		}

	private:
		const TArray<FName>& NameMap;
	};
	FNameReaderProxyArchive ProxyAr(Ar, CookedHeaderData.SummaryNames);

	if (Summary.ImportCount > 0)
	{
		CookedHeaderData.ObjectImports.Reserve(Summary.ImportCount);
		ProxyAr.Seek(Summary.ImportOffset);
		for (int32 I = 0; I < Summary.ImportCount; ++I)
		{
			ProxyAr << CookedHeaderData.ObjectImports.AddDefaulted_GetRef();
		}
	}

	if (Summary.PreloadDependencyCount > 0)
	{
		CookedHeaderData.PreloadDependencies.Reserve(Summary.PreloadDependencyCount);
		ProxyAr.Seek(Summary.PreloadDependencyOffset);
		for (int32 I = 0; I < Summary.PreloadDependencyCount; ++I)
		{
			ProxyAr << CookedHeaderData.PreloadDependencies.AddDefaulted_GetRef();
		}
	}

	if (Summary.ExportCount > 0)
	{
		CookedHeaderData.ObjectExports.Reserve(Summary.ExportCount);
		ProxyAr.Seek(Summary.ExportOffset);
		for (int32 I = 0; I < Summary.ExportCount; ++I)
		{
			FObjectExport& ObjectExport = CookedHeaderData.ObjectExports.AddDefaulted_GetRef();
			ProxyAr << ObjectExport;
		}
	}

	if (Summary.CellImportCount > 0)
	{
		CookedHeaderData.CellImports.Reserve(Summary.CellImportCount);
		ProxyAr.Seek(Summary.CellImportOffset);
		for (int32 I = 0; I < Summary.CellImportCount; ++I)
		{
			FCellImport& CellImport = CookedHeaderData.CellImports.AddDefaulted_GetRef();
			ProxyAr << CellImport;
		}
	}

	if (Summary.CellExportCount > 0)
	{
		CookedHeaderData.CellExports.Reserve(Summary.CellExportCount);
		ProxyAr.Seek(Summary.CellExportOffset);
		for (int32 I = 0; I < Summary.CellExportCount; ++I)
		{
			FCellExport& CellExport = CookedHeaderData.CellExports.AddDefaulted_GetRef();
			ProxyAr << CellExport;
		}
	}

	if (Summary.SoftPackageReferencesCount > 0)
	{
		ProxyAr.Seek(Summary.SoftPackageReferencesOffset);
		CookedHeaderData.SoftPackageReferences.Reserve(Summary.SoftPackageReferencesCount);
		for (int32 I = 0; I < Summary.SoftPackageReferencesCount; ++I)
		{
			FName& SoftReference = CookedHeaderData.SoftPackageReferences.Emplace_GetRef();
			ProxyAr << SoftReference;
		}
	}

	if (Summary.DataResourceOffset > 0)
	{
		ProxyAr.Seek(Summary.DataResourceOffset);
		FObjectDataResource::Serialize(ProxyAr, CookedHeaderData.DataResources);
	}

	return CookedHeaderData;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FPackageStoreOptimizer::ResolveImport(FPackageStorePackage::FUnresolvedImport* Imports, FPackageStorePackage::FUnresolvedExport* Exports, const FObjectImport* ObjectImports, const FObjectExport* ObjectExports, const FName& PackageName, int32 LocalImportIndex) const
{
	FPackageStorePackage::FUnresolvedImport* Import = Imports + LocalImportIndex;
	if (Import->FullName.Len() == 0)
	{
		Import->FullName.Reserve(256);

		const FObjectImport* ObjectImport = ObjectImports + LocalImportIndex;
		if (ObjectImport->OuterIndex.IsNull())
		{
			FName ImportPackageName = ObjectImport->ObjectName;
			ImportPackageName.AppendString(Import->FullName);
			Import->FullName.ToLowerInline();
			Import->FullPath = Import->FullName;
			Import->FromPackageName = ImportPackageName;
			Import->FromPackageNameLen = Import->FullName.Len();
			Import->bIsScriptImport = Import->FullName.StartsWith(TEXT("/Script/"));
			Import->bIsVerseVNIImport = !Import->bIsScriptImport && Import->FullName.Contains(TEXT("/_Verse/VNI"));
			Import->bIsImportOfPackage = true;
		}
		else if (ObjectImport->OuterIndex.IsImport())
		{
			const int32 OuterImportIndex = ObjectImport->OuterIndex.ToImport();
			ResolveImport(Imports, Exports, ObjectImports, ObjectExports, PackageName, OuterImportIndex);
			FPackageStorePackage::FUnresolvedImport* OuterImport = Imports + OuterImportIndex;
			check(OuterImport->FullName.Len() > 0);
			Import->bIsScriptImport = OuterImport->bIsScriptImport;
			Import->bIsVerseVNIImport = OuterImport->bIsVerseVNIImport;
			Import->FullName.Append(OuterImport->FullName);
			Import->FullName.AppendChar(TEXT('/'));
			ObjectImport->ObjectName.AppendString(Import->FullName);
			Import->FullName.ToLowerInline();
			if (ObjectImport->HasPackageName())
			{
				ObjectImport->GetPackageName().AppendString(Import->FullPath);
				Import->FullPath.AppendChar(TEXT('/'));
				ObjectImport->ObjectName.AppendString(Import->FullPath);
				Import->FullPath.ToLowerInline();
				Import->FromPackageName = ObjectImport->GetPackageName();
				Import->FromPackageNameLen = ObjectImport->GetPackageName().ToString().Len();
			}
			else
			{
				Import->FullPath.Append(OuterImport->FullPath);
				Import->FullPath.AppendChar(TEXT('/'));
				ObjectImport->ObjectName.AppendString(Import->FullPath);
				Import->FullPath.ToLowerInline();
				Import->FromPackageName = OuterImport->FromPackageName;
				Import->FromPackageNameLen = OuterImport->FromPackageNameLen;
			}
		}
		else
		{
			check(ObjectImport->HasPackageName());
			int32 OuterExportIndex = ObjectImport->OuterIndex.ToExport();
			ResolveExport(Imports, Exports, ObjectImports, ObjectExports, PackageName, OuterExportIndex);
			FPackageStorePackage::FUnresolvedExport* OuterExport = Exports + OuterExportIndex;
			Import->FullName.Append(OuterExport->FullName);
			Import->FullName.AppendChar(TEXT('/'));
			ObjectImport->ObjectName.AppendString(Import->FullName);
			Import->FullName.ToLowerInline();
			ObjectImport->GetPackageName().AppendString(Import->FullPath);
			Import->FullPath.AppendChar(TEXT('/'));
			ObjectImport->ObjectName.AppendString(Import->FullPath);
			Import->FullPath.ToLowerInline();
			Import->FromPackageName = ObjectImport->GetPackageName();
			Import->FromPackageNameLen = ObjectImport->GetPackageName().ToString().Len();
		}
	}
}

void FPackageStoreOptimizer::AppendPathForPublicExportHash(UObject* Object, FStringBuilderBase& OutPath)
{
	if (!Object)
	{
		return;
	}
	if (!Object->GetOuter())
	{
		// Outermost package objects do not have a PublicExportHash Name; they use 0 for PublicExportHash
		// Write nothing for them.
		return;
	}

	constexpr int32 DirectorySeparatorLen = 1;
	const TCHAR DirectorySeparatorChar = '/';

	TArray<FName, TInlineAllocator<10>> PathNames;
	int32 PathNameLen = 0;
	for (UObject* Iter = Object; Iter->GetOuter(); Iter = Iter->GetOuter())
	{
		FName PathName = Iter->GetFName();
		PathNames.Add(PathName);
		PathNameLen += PathName.GetStringLength() + DirectorySeparatorLen;
	}
	if (PathNameLen <= 1)
	{
		// We should be writing at least 2 characters: /<LeafName>. Write nothing if we find an empty LeafName.
		return;
	}

	int32 InitialLength = OutPath.Len();
	for (FName PathName : ReverseIterate(PathNames))
	{
		OutPath << DirectorySeparatorChar << PathName;
	}

	// Down-case the characters we wrote; public export hash is based on lower-case
	TCHAR* OutPathData = OutPath.GetData();
	int32 NewLength = OutPath.Len();
	for (int32 N = InitialLength; N < NewLength; ++N)
	{
		OutPathData[N] = FChar::ToLower(OutPathData[N]);
	}
}

bool FPackageStoreOptimizer::TryGetPublicExportHash(FStringView PackageRelativeExportPath, uint64& OutPublicExportHash)
{
	OutPublicExportHash = 0;

	// PackageRelativeExportHash should have been generated by GetNameForPublicExportHash or an equivalent function
	if (PackageRelativeExportPath.Len() == 0 || // PublicExportHash is not defined for the UPackage object
		PackageRelativeExportPath.Len() == 1 || // An object with an empty leaf name is an error, but we ignore it and treat it the same as a package
		PackageRelativeExportPath[0] != '/') // Invalid string; GetNameForPublicExportHash starts every relative path with /
	{
		return false;
	}

	OutPublicExportHash = GetPublicExportHash(PackageRelativeExportPath);
	return true;
}

uint64 FPackageStoreOptimizer::GetPublicExportHash(FStringView PackageRelativeExportPath)
{
	check(PackageRelativeExportPath.Len() > 1);
	check(PackageRelativeExportPath[0] == '/');
	return CityHash64(reinterpret_cast<const char*>(PackageRelativeExportPath.GetData() + 1), (PackageRelativeExportPath.Len() - 1) * sizeof(TCHAR));
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
uint64 FPackageStoreOptimizer::GetCellExportHash(Verse::VUniqueString* VersePath)
{
	return GetCellExportHash(VersePath->AsStringView());
}
#endif
uint64 FPackageStoreOptimizer::GetCellExportHash(FUtf8StringView VersePath)
{
	return CityHash64(reinterpret_cast<const char*>(VersePath.GetData()), VersePath.Len());
}

void FPackageStoreOptimizer::ProcessImports(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package, TArray<FPackageStorePackage::FUnresolvedImport>& UnresolvedImports, TArray<FPackageStorePackage::FUnresolvedExport>& UnresolvedExports) const
{
	const int32 ImportCount = CookedHeaderData.ObjectImports.Num();
	const int32 CellImportCount = CookedHeaderData.CellImports.Num();

	TSet<FName> ImportedPackageNames;
	for (int32 ImportIndex = 0; ImportIndex < ImportCount; ++ImportIndex)
	{
		ResolveImport(UnresolvedImports.GetData(), UnresolvedExports.GetData(), CookedHeaderData.ObjectImports.GetData(), CookedHeaderData.ObjectExports.GetData(), Package->Name, ImportIndex);
		FPackageStorePackage::FUnresolvedImport& UnresolvedImport = UnresolvedImports[ImportIndex];
		// For VerseVNI imports, we still need to add the package name to the imported package names for when the
		// type being referenced wasn't generated by UHT
		if (!UnresolvedImport.bIsScriptImport)
		{
			if (UnresolvedImport.bIsImportOfPackage)
			{
				ImportedPackageNames.Add(UnresolvedImport.FromPackageName);
			}
		}
	}
	for (int32 CellImportIndex = 0; CellImportIndex < CellImportCount; ++CellImportIndex)
	{
		const FCellImport& CellImport = CookedHeaderData.CellImports[CellImportIndex];
		int32 ImportPackageIndex = CellImport.PackageIndex.ToImport();
		ResolveImport(UnresolvedImports.GetData(), UnresolvedExports.GetData(), CookedHeaderData.ObjectImports.GetData(), CookedHeaderData.ObjectExports.GetData(), Package->Name, ImportPackageIndex);
		FPackageStorePackage::FUnresolvedImport& PackageImport = UnresolvedImports[ImportPackageIndex];
		if (!PackageImport.bIsScriptImport)
		{
			ImportedPackageNames.Add(PackageImport.FromPackageName);
		}
	}
	Package->ImportedPackages.Reserve(ImportedPackageNames.Num());
	for (FName ImportedPackageName : ImportedPackageNames)
	{
		Package->ImportedPackages.Emplace(ImportedPackageName);
	}
	Algo::Sort(Package->ImportedPackages);

	for (int32 ImportIndex = 0; ImportIndex < ImportCount; ++ImportIndex)
	{
		FPackageStorePackage::FUnresolvedImport& UnresolvedImport = UnresolvedImports[ImportIndex];

		bool bImportFromPackage = !UnresolvedImport.bIsImportOfPackage;

		if (UnresolvedImport.bIsScriptImport)
		{
			FPackageObjectIndex ScriptObjectIndex = FPackageObjectIndex::FromScriptPath(UnresolvedImport.FullName);
			if (!ScriptObjectsMap.Contains(ScriptObjectIndex))
			{
				UE_LOGF(LogPackageStoreOptimizer, Warning, "Package '%ls' is referencing missing script import '%ls'", *Package->Name.ToString(), *UnresolvedImport.FullName);
			}
			Package->Imports[ImportIndex] = ScriptObjectIndex;
			bImportFromPackage = false;
		}
		else if (UnresolvedImport.bIsVerseVNIImport)
		{
			FPackageObjectIndex ScriptObjectIndex = FPackageObjectIndex::FromScriptPath(UnresolvedImport.FullName);
			bool bIsUHTGeneratedVerseVNIObject = ScriptObjectsMap.Contains(ScriptObjectIndex);
			if (bIsUHTGeneratedVerseVNIObject)
			{
				Package->Imports[ImportIndex] = ScriptObjectIndex;
				bImportFromPackage = false;
			}
		}

		if (bImportFromPackage)
		{
			bool bFoundPackageIndex = false;
			for (uint32 PackageIndex = 0, PackageCount = static_cast<uint32>(Package->ImportedPackages.Num()); PackageIndex < PackageCount; ++PackageIndex)
			{
				if (UnresolvedImport.FromPackageName == Package->ImportedPackages[PackageIndex].Name)
				{
					FStringView PackageRelativeName = FStringView(UnresolvedImport.FullPath).RightChop(UnresolvedImport.FromPackageNameLen);
					check(PackageRelativeName.Len());
					FPackageImportReference PackageImportRef(PackageIndex, Package->ImportedPublicExportHashes.Num());
					Package->Imports[ImportIndex] = FPackageObjectIndex::FromPackageImportRef(PackageImportRef);
					uint64 ExportHash = GetPublicExportHash(PackageRelativeName);
					Package->ImportedPublicExportHashes.Add(ExportHash);
					bFoundPackageIndex = true;
					break;
				}
			}
			check(bFoundPackageIndex);
		}
	}

	for (int32 CellImportIndex = 0; CellImportIndex < CellImportCount; ++CellImportIndex)
	{
		const FCellImport& CellImport = CookedHeaderData.CellImports[CellImportIndex];
		int32 ImportPackageIndex = CellImport.PackageIndex.ToImport();
		FPackageStorePackage::FUnresolvedImport& PackageImport = UnresolvedImports[ImportPackageIndex];
		if (PackageImport.bIsScriptImport)
		{
			FPackageObjectIndex ScriptCellIndex = FPackageObjectIndex::FromVersePath(CellImport.VersePath);
			if (!ScriptCellsMap.Contains(ScriptCellIndex))
			{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
				FUtf8StringView VersePathUtf8 = CellImport.VersePath->AsStringView();
#else
				FUtf8StringView VersePathUtf8 = CellImport.VersePath;
#endif
				FString VersePath = FString(VersePathUtf8);
				UE_LOGF(LogPackageStoreOptimizer, Warning, "Package '%ls' is referencing missing script import '%ls'", *Package->Name.ToString(), *VersePath);
			}
			Package->CellImports[CellImportIndex] = ScriptCellIndex;
		}
		else
		{
			bool bFoundPackageIndex = false;
			for (uint32 PackageIndex = 0, PackageCount = static_cast<uint32>(Package->ImportedPackages.Num()); PackageIndex < PackageCount; ++PackageIndex)
			{
				if (PackageImport.FromPackageName == Package->ImportedPackages[PackageIndex].Name)
				{
					FPackageImportReference PackageImportRef(PackageIndex, Package->ImportedPublicExportHashes.Num());
					Package->CellImports[CellImportIndex] = FPackageObjectIndex::FromPackageImportRef(PackageImportRef);
					uint64 CellExportHash = GetCellExportHash(CellImport.VersePath);
					Package->ImportedPublicExportHashes.Add(CellExportHash);
					bFoundPackageIndex = true;
					break;
				}
			}
			check(bFoundPackageIndex);
		}
	}
}

void FPackageStoreOptimizer::ResolveExport(FPackageStorePackage::FUnresolvedImport* Imports, FPackageStorePackage::FUnresolvedExport* Exports, const FObjectImport* ObjectImports, const FObjectExport* ObjectExports, const FName& PackageName, int32 LocalExportIndex) const
{
	FPackageStorePackage::FUnresolvedExport* Export = Exports + LocalExportIndex;
	if (Export->FullName.Len() == 0)
	{
		Export->FullName.Reserve(256);
		Export->FullPath.Reserve(256);
		const FObjectExport* ObjectExport = ObjectExports + LocalExportIndex;
		if (ObjectExport->OuterIndex.IsNull())
		{
			PackageName.AppendString(Export->FullName);
			Export->FullName.AppendChar(TEXT('/'));
			ObjectExport->ObjectName.AppendString(Export->FullName);
			Export->FullName.ToLowerInline();
			check(Export->FullName.Len() > 0);
			Export->FullPath = Export->FullName;
		}
		else
		{
			FString* OuterName = nullptr;
			if (ObjectExport->OuterIndex.IsExport())
			{
				int32 OuterExportIndex = ObjectExport->OuterIndex.ToExport();
				ResolveExport(Imports, Exports, ObjectImports, ObjectExports, PackageName, OuterExportIndex);
				OuterName = &Exports[OuterExportIndex].FullName;
				Export->FullPath.Append(*Exports[OuterExportIndex].FullPath);
				Export->FullPath.AppendChar(TEXT('/'));
				ObjectExport->ObjectName.AppendString(Export->FullPath);
				Export->FullPath.ToLowerInline();
			}
			else
			{
				check(Imports && ObjectImports);
				int32 OuterImportIndex = ObjectExport->OuterIndex.ToImport();
				ResolveImport(Imports, Exports, ObjectImports, ObjectExports, PackageName, OuterImportIndex);
				OuterName = &Imports[OuterImportIndex].FullName;
				Export->FullPath.Append(PackageName.ToString());
				if (OuterName->StartsWith(PackageName.ToString()))
				{
					FStringView PackageRelativeName = FStringView(*OuterName).RightChop(PackageName.ToString().Len());
					Export->FullPath += PackageRelativeName;
				}
				Export->FullPath.AppendChar(TEXT('/'));
				ObjectExport->ObjectName.AppendString(Export->FullPath);
				Export->FullPath.ToLowerInline();
			}

			check(OuterName && OuterName->Len() > 0);
			Export->FullName.Append(*OuterName);
			Export->FullName.AppendChar(TEXT('/'));
			ObjectExport->ObjectName.AppendString(Export->FullName);
			Export->FullName.ToLowerInline();
		}
	}
}

void FPackageStoreOptimizer::ProcessExports(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package, TArray<FPackageStorePackage::FUnresolvedImport>& UnresolvedImports, TArray<FPackageStorePackage::FUnresolvedExport>& UnresolvedExports) const
{
	const int32 ExportCount = CookedHeaderData.ObjectExports.Num();
	const int32 CellExportCount = CookedHeaderData.CellExports.Num();

	auto PackageObjectIdFromPackageIndex =
		[](const TArray<FPackageObjectIndex>& Imports, const FPackageIndex& PackageIndex) -> FPackageObjectIndex
	{
		if (PackageIndex.IsImport())
		{
			return Imports[PackageIndex.ToImport()];
		}
		if (PackageIndex.IsExport())
		{
			return FPackageObjectIndex::FromExportIndex(PackageIndex.ToExport());
		}
		return FPackageObjectIndex();
	};

	FString PackageNameStr = Package->Name.ToString();
	TMap<uint64, const FPackageStorePackage::FUnresolvedExport*> SeenPublicExportHashes;
	for (int32 ExportIndex = 0; ExportIndex < ExportCount; ++ExportIndex)
	{
		const FObjectExport& ObjectExport = CookedHeaderData.ObjectExports[ExportIndex];

		FPackageStorePackage::FExport& Export = Package->Exports[ExportIndex];
		FPackageStorePackage::FUnresolvedExport& UnresolvedExport = UnresolvedExports[ExportIndex];
		Export.ObjectName = ObjectExport.ObjectName;
		Export.ObjectFlags = ObjectExport.ObjectFlags;
		check(ObjectExport.SerialOffset >= Package->CookedHeaderSize);
		Export.SerialOffset = ObjectExport.SerialOffset - Package->CookedHeaderSize;
		Export.SerialSize = ObjectExport.SerialSize;
		Export.bNotForClient = ObjectExport.bNotForClient;
		Export.bNotForServer = ObjectExport.bNotForServer;
		Export.bIsPublic = (Export.ObjectFlags & (RF_Public | RF_GeneratePublicHash)) > 0 || ObjectExport.bGeneratePublicHash;
		ResolveExport(UnresolvedImports.GetData(), UnresolvedExports.GetData(), CookedHeaderData.ObjectImports.GetData(), CookedHeaderData.ObjectExports.GetData(), Package->Name, ExportIndex);
		if (Export.bIsPublic)
		{
			check(UnresolvedExport.FullPath.Len() > 0);
			check(UnresolvedExport.FullPath.StartsWith(PackageNameStr));
			FStringView PackageRelativeName = FStringView(UnresolvedExport.FullPath).RightChop(PackageNameStr.Len());
			check(PackageRelativeName.Len());
			Export.PublicExportHash = GetPublicExportHash(PackageRelativeName);
			const FPackageStorePackage::FUnresolvedExport* FindCollidingExport = SeenPublicExportHashes.FindRef(Export.PublicExportHash);
			if (FindCollidingExport)
			{
				UE_LOGF(LogPackageStoreOptimizer, Fatal, "Export hash collision in package \"%ls\": \"%ls\" and \"%ls", *PackageNameStr, PackageRelativeName.GetData(), *FindCollidingExport->FullPath);
			}
			SeenPublicExportHashes.Add(Export.PublicExportHash, &UnresolvedExport);
		}

		Export.OuterIndex = PackageObjectIdFromPackageIndex(Package->Imports, ObjectExport.OuterIndex);		
		Export.ClassIndex = PackageObjectIdFromPackageIndex(Package->Imports, ObjectExport.ClassIndex);
		Export.SuperIndex = PackageObjectIdFromPackageIndex(Package->Imports, ObjectExport.SuperIndex);
		Export.TemplateIndex = PackageObjectIdFromPackageIndex(Package->Imports, ObjectExport.TemplateIndex);

		for (uint8 CommandType = 0; CommandType < FExportBundleEntry::ExportCommandType_Count; ++CommandType)
		{
			FPackageStorePackage::FExportGraphNode& Node = Package->ExportGraphNodes.AddDefaulted_GetRef();
			Node.BundleEntry.CommandType = FExportBundleEntry::EExportCommandType(CommandType);
			Node.BundleEntry.LocalExportIndex = ExportIndex;
			Node.bIsPublic = Export.bIsPublic;
			Export.Nodes[CommandType] = &Node;
		}
	}

	TArrayView<FPackageStorePackage::FUnresolvedExport> UnresolvedCellExports = UnresolvedExports;
	UnresolvedCellExports.RightChopInline(ExportCount);
	for (int32 CellExportIndex = 0; CellExportIndex < CellExportCount; ++CellExportIndex)
	{
		const FCellExport& CellExport = CookedHeaderData.CellExports[CellExportIndex];

		FPackageStorePackage::FCellExport& Export = Package->CellExports[CellExportIndex];
		FPackageStorePackage::FUnresolvedExport& UnresolvedExport = UnresolvedCellExports[CellExportIndex];
		check(CellExport.SerialOffset >= Package->CookedHeaderSize);
		Export.SerialOffset = CellExport.SerialOffset - Package->CookedHeaderSize;
		Export.SerialLayoutSize = CellExport.SerialLayoutSize;
		Export.SerialSize = CellExport.SerialSize;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		FUtf8StringView VersePathUtf8 = CellExport.VersePath->AsStringView();
#else
		FUtf8StringView VersePathUtf8 = CellExport.VersePath;
#endif
		if (!VersePathUtf8.IsEmpty())
		{
			UnresolvedExport.FullName = FString(VersePathUtf8);
			Export.PublicExportHash = GetCellExportHash(CellExport.VersePath);
			if (const FPackageStorePackage::FUnresolvedExport* FindCollidingExport = SeenPublicExportHashes.FindRef(Export.PublicExportHash))
			{
				UE_LOGF(LogPackageStoreOptimizer, Fatal, "Export hash collision in package \"%ls\": \"%ls\" and \"%ls\"", *PackageNameStr, *UnresolvedExport.FullName, *FindCollidingExport->FullName);
			}
			SeenPublicExportHashes.Add(Export.PublicExportHash, &UnresolvedExport);
		}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		Export.CppClassInfo = CellExport.CppClassInfo->Name;
#else
		Export.CppClassInfo = CellExport.CppClassInfo;
#endif

		for (uint8 CommandType = 0; CommandType < FExportBundleEntry::ExportCommandType_Count; ++CommandType)
		{
			FPackageStorePackage::FExportGraphNode& Node = Package->ExportGraphNodes.AddDefaulted_GetRef();
			Node.BundleEntry.CommandType = FExportBundleEntry::EExportCommandType(CommandType);
			Node.BundleEntry.LocalExportIndex = ExportCount + CellExportIndex;
			Export.Nodes[CommandType] = &Node;
		}
	}
}

void FPackageStoreOptimizer::ProcessPreloadDependencies(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessPreloadDependencies);

	auto AddNodeDependency = [](FPackageStorePackage* Package, int32 FromExportIndex, FExportBundleEntry::EExportCommandType FromExportBundleCommandType, FPackageStorePackage::FExportGraphNode* ToNode)
	{
		FPackageStorePackage::FExportGraphNode* FromNode;
		if (FromExportIndex < Package->Exports.Num())
		{
			FPackageStorePackage::FExport& FromExport = Package->Exports[FromExportIndex];
			FromNode = FromExport.Nodes[FromExportBundleCommandType];
		}
		else
		{
			int32 CellExportIndex = FromExportIndex - Package->Exports.Num();
			FPackageStorePackage::FCellExport& FromExport = Package->CellExports[CellExportIndex];
			FromNode = FromExport.Nodes[FromExportBundleCommandType];
		}
		ToNode->InternalDependencies.Add(FromNode);
	};

	auto IsScriptImport = [](FPackageStorePackage* Package, int32 ImportIndex)
	{
		if (ImportIndex < Package->Imports.Num())
		{
			return Package->Imports[ImportIndex].IsScriptImport();
		}
		else
		{
			int32 CellImportIndex = ImportIndex - Package->Imports.Num();
			return Package->CellImports[CellImportIndex].IsScriptImport();
		}
	};

	TArray<FDependencyBundleHeader>& DependencyBundleHeaders = Package->GraphData.DependencyBundleHeaders;
	TArray<FDependencyBundleEntry>& DependencyBundleEntries = Package->GraphData.DependencyBundleEntries;

	for (int32 ExportIndex = 0; ExportIndex < Package->Exports.Num(); ++ExportIndex)
	{
		FPackageStorePackage::FExport& Export = Package->Exports[ExportIndex];
		const FObjectExport& ObjectExport = CookedHeaderData.ObjectExports[ExportIndex];

		AddNodeDependency(Package, ExportIndex, FExportBundleEntry::ExportCommandType_Create, Export.Nodes[FExportBundleEntry::ExportCommandType_Serialize]);

		FDependencyBundleHeader& DependencyBundleHeader = DependencyBundleHeaders.AddDefaulted_GetRef();
		FMemory::Memzero(&DependencyBundleHeader, sizeof(FDependencyBundleHeader));
		
		if (ObjectExport.FirstExportDependency >= 0)
		{
			DependencyBundleHeader.FirstEntryIndex = DependencyBundleEntries.Num();
			int32 StartIndex = ObjectExport.FirstExportDependency +
				ObjectExport.SerializationBeforeSerializationDependencies +
				ObjectExport.CreateBeforeSerializationDependencies +
				ObjectExport.SerializationBeforeCreateDependencies;
			for (int32 Index = StartIndex; Index < StartIndex + ObjectExport.CreateBeforeCreateDependencies; ++Index)
			{
				FPackageIndex Dep = CookedHeaderData.PreloadDependencies[Index];
				if (Dep.IsExport())
				{
					AddNodeDependency(Package, Dep.ToExport(), FExportBundleEntry::ExportCommandType_Create, Export.Nodes[FExportBundleEntry::ExportCommandType_Create]);
				}
				if (Dep.IsExport() || !IsScriptImport(Package, Dep.ToImport()))
				{
					DependencyBundleEntries.AddDefaulted_GetRef().LocalImportOrExportIndex = Dep;
					++DependencyBundleHeader.EntryCount[FExportBundleEntry::ExportCommandType_Create][FExportBundleEntry::ExportCommandType_Create];
				}
			}

			StartIndex = ObjectExport.FirstExportDependency +
				ObjectExport.SerializationBeforeSerializationDependencies +
				ObjectExport.CreateBeforeSerializationDependencies;
			for (int32 Index = StartIndex; Index < StartIndex + ObjectExport.SerializationBeforeCreateDependencies; ++Index)
			{
				FPackageIndex Dep = CookedHeaderData.PreloadDependencies[Index];
				if (Dep.IsExport())
				{
					AddNodeDependency(Package, Dep.ToExport(), FExportBundleEntry::ExportCommandType_Serialize, Export.Nodes[FExportBundleEntry::ExportCommandType_Create]);
				}
				if (Dep.IsExport() || !IsScriptImport(Package, Dep.ToImport()))
				{
					DependencyBundleEntries.AddDefaulted_GetRef().LocalImportOrExportIndex = Dep;
					++DependencyBundleHeader.EntryCount[FExportBundleEntry::ExportCommandType_Create][FExportBundleEntry::ExportCommandType_Serialize];
				}
			}
			
			StartIndex = ObjectExport.FirstExportDependency +
				ObjectExport.SerializationBeforeSerializationDependencies;
			for (int32 Index = StartIndex; Index < StartIndex + ObjectExport.CreateBeforeSerializationDependencies; ++Index)
			{
				FPackageIndex Dep = CookedHeaderData.PreloadDependencies[Index];
				if (Dep.IsExport())
				{
					AddNodeDependency(Package, Dep.ToExport(), FExportBundleEntry::ExportCommandType_Create, Export.Nodes[FExportBundleEntry::ExportCommandType_Serialize]);
				}
				if (Dep.IsExport() || !IsScriptImport(Package, Dep.ToImport()))
				{
					DependencyBundleEntries.AddDefaulted_GetRef().LocalImportOrExportIndex = Dep;
					++DependencyBundleHeader.EntryCount[FExportBundleEntry::ExportCommandType_Serialize][FExportBundleEntry::ExportCommandType_Create];
				}
			}

			StartIndex = ObjectExport.FirstExportDependency;
			for (int32 Index = StartIndex; Index < StartIndex + ObjectExport.SerializationBeforeSerializationDependencies; ++Index)
			{
				FPackageIndex Dep = CookedHeaderData.PreloadDependencies[Index];
				if (Dep.IsExport())
				{
					AddNodeDependency(Package, Dep.ToExport(), FExportBundleEntry::ExportCommandType_Serialize, Export.Nodes[FExportBundleEntry::ExportCommandType_Serialize]);
				}
				if (Dep.IsExport() || !IsScriptImport(Package, Dep.ToImport()))
				{
					DependencyBundleEntries.AddDefaulted_GetRef().LocalImportOrExportIndex = Dep;
					++DependencyBundleHeader.EntryCount[FExportBundleEntry::ExportCommandType_Serialize][FExportBundleEntry::ExportCommandType_Serialize];
				}
			}
		}
		else
		{
			DependencyBundleHeader.FirstEntryIndex = -1;
		}
	}

	for (int32 CellExportIndex = 0; CellExportIndex < Package->CellExports.Num(); ++CellExportIndex)
	{
		FPackageStorePackage::FCellExport& Export = Package->CellExports[CellExportIndex];
		const FCellExport& CellExport = CookedHeaderData.CellExports[CellExportIndex];

		int ExportIndex = Package->Exports.Num() + CellExportIndex;
		AddNodeDependency(Package, ExportIndex, FExportBundleEntry::ExportCommandType_Create, Export.Nodes[FExportBundleEntry::ExportCommandType_Serialize]);

		FDependencyBundleHeader& DependencyBundleHeader = DependencyBundleHeaders.AddDefaulted_GetRef();
		FMemory::Memzero(&DependencyBundleHeader, sizeof(FDependencyBundleHeader));

		if (CellExport.FirstExportDependency >= 0)
		{
			DependencyBundleHeader.FirstEntryIndex = DependencyBundleEntries.Num();

			int32 StartIndex = CellExport.FirstExportDependency +
				CellExport.SerializationBeforeSerializationDependencies;
			for (int32 Index = StartIndex; Index < StartIndex + CellExport.CreateBeforeSerializationDependencies; ++Index)
			{
				FPackageIndex Dep = CookedHeaderData.PreloadDependencies[Index];
				if (Dep.IsExport())
				{
					AddNodeDependency(Package, Dep.ToExport(), FExportBundleEntry::ExportCommandType_Create, Export.Nodes[FExportBundleEntry::ExportCommandType_Serialize]);
				}
				if (Dep.IsExport() || !IsScriptImport(Package, Dep.ToImport()))
				{
					DependencyBundleEntries.AddDefaulted_GetRef().LocalImportOrExportIndex = Dep;
					++DependencyBundleHeader.EntryCount[FExportBundleEntry::ExportCommandType_Serialize][FExportBundleEntry::ExportCommandType_Create];
				}
			}

			StartIndex = CellExport.FirstExportDependency;
			for (int32 Index = StartIndex; Index < StartIndex + CellExport.SerializationBeforeSerializationDependencies; ++Index)
			{
				FPackageIndex Dep = CookedHeaderData.PreloadDependencies[Index];
				if (Dep.IsExport())
				{
					AddNodeDependency(Package, Dep.ToExport(), FExportBundleEntry::ExportCommandType_Serialize, Export.Nodes[FExportBundleEntry::ExportCommandType_Serialize]);
				}
				if (Dep.IsExport() || !IsScriptImport(Package, Dep.ToImport()))
				{
					DependencyBundleEntries.AddDefaulted_GetRef().LocalImportOrExportIndex = Dep;
					++DependencyBundleHeader.EntryCount[FExportBundleEntry::ExportCommandType_Serialize][FExportBundleEntry::ExportCommandType_Serialize];
				}
			}
		}
		else
		{
			DependencyBundleHeader.FirstEntryIndex = -1;
		}
	}
}

void FPackageStoreOptimizer::ProcessDataResources(const FCookedHeaderData& CookedHeaderData, FPackageStorePackage* Package) const
{
	for (const FObjectDataResource& DataResource : CookedHeaderData.DataResources)
	{
		FBulkDataMapEntry& Entry = Package->BulkDataEntries.AddDefaulted_GetRef();
		checkf(DataResource.SerialSize == DataResource.RawSize, TEXT("Compressed bulk data is not supported in cooked builds"));

		Entry.SerialOffset = DataResource.SerialOffset;
		Entry.DuplicateSerialOffset = DataResource.DuplicateSerialOffset;
		Entry.SerialSize = DataResource.SerialSize;
		Entry.Flags = DataResource.LegacyBulkDataFlags;
		Entry.CookedIndex = DataResource.CookedIndex;
	}
}

TArray<FPackageStorePackage::FExportGraphNode*> FPackageStoreOptimizer::SortExportGraphNodesInLoadOrder(FPackageStorePackage* Package, FExportGraphEdges& Edges) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SortExportGraphNodesInLoadOrder);
	int32 NodeCount = Package->ExportGraphNodes.Num();
	for (auto& KV : Edges)
	{
		FPackageStorePackage::FExportGraphNode* ToNode = KV.Value;
		++ToNode->IncomingEdgeCount;
	}

	auto NodeSorter = [](const FPackageStorePackage::FExportGraphNode& A, const FPackageStorePackage::FExportGraphNode& B)
	{
		if (A.bIsPublic != B.bIsPublic)
		{
			return A.bIsPublic;
		}
		if (A.BundleEntry.CommandType != B.BundleEntry.CommandType)
		{
			return A.BundleEntry.CommandType < B.BundleEntry.CommandType;
		}
		return A.BundleEntry.LocalExportIndex < B.BundleEntry.LocalExportIndex;
	};

	TArray<FPackageStorePackage::FExportGraphNode*> NodesWithNoIncomingEdges;
	NodesWithNoIncomingEdges.Reserve(NodeCount);
	for (FPackageStorePackage::FExportGraphNode& Node : Package->ExportGraphNodes)
	{
		if (Node.IncomingEdgeCount == 0)
		{
			NodesWithNoIncomingEdges.HeapPush(&Node, NodeSorter);
		}
	}

	TArray<FPackageStorePackage::FExportGraphNode*> LoadOrder;
	LoadOrder.Reserve(NodeCount);
	while (NodesWithNoIncomingEdges.Num())
	{
		FPackageStorePackage::FExportGraphNode* RemovedNode;
		NodesWithNoIncomingEdges.HeapPop(RemovedNode, NodeSorter, EAllowShrinking::No);
		LoadOrder.Add(RemovedNode);
		for (auto EdgeIt = Edges.CreateKeyIterator(RemovedNode); EdgeIt; ++EdgeIt)
		{
			FPackageStorePackage::FExportGraphNode* ToNode = EdgeIt.Value();
			check(ToNode->IncomingEdgeCount > 0);
			if (--ToNode->IncomingEdgeCount == 0)
			{
				NodesWithNoIncomingEdges.HeapPush(ToNode, NodeSorter);
			}
		}
	}
	check(LoadOrder.Num() == NodeCount);
	return LoadOrder;
}

void FPackageStoreOptimizer::CreateExportBundle(FPackageStorePackage* Package) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateExportBundles);
	FExportGraphEdges Edges;
	for (FPackageStorePackage::FExportGraphNode& ExportGraphNode : Package->ExportGraphNodes)
	{
		for (FPackageStorePackage::FExportGraphNode* InternalDependency : ExportGraphNode.InternalDependencies)
		{
			Edges.Add(InternalDependency, &ExportGraphNode);
		}
	}
	TArray<FPackageStorePackage::FExportGraphNode*> LoadOrder = SortExportGraphNodesInLoadOrder(Package, Edges);
	for (FPackageStorePackage::FExportGraphNode* Node : LoadOrder)
	{
		Package->GraphData.ExportBundleEntries.Add(Node->BundleEntry);
	}
}

void FPackageStoreOptimizer::FinalizePackageHeader(FPackageStorePackage* Package) const
{
	FBufferWriter ImportedPublicExportHashesArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (uint64 ImportedPublicExportHash : Package->ImportedPublicExportHashes)
	{
		ImportedPublicExportHashesArchive << ImportedPublicExportHash;
	}
	uint64 ImportedPublicExportHashesSize = ImportedPublicExportHashesArchive.Tell();

	FBufferWriter ImportMapArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (FPackageObjectIndex Import : Package->Imports)
	{
		ImportMapArchive << Import;
	}
	uint64 ImportMapSize = ImportMapArchive.Tell();

	FBufferWriter ExportMapArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (const FPackageStorePackage::FExport& Export : Package->Exports)
	{
		FExportMapEntry ExportMapEntry;
		ExportMapEntry.CookedSerialOffset = Export.SerialOffset;
		ExportMapEntry.CookedSerialSize = Export.SerialSize;
		Package->NameMapBuilder.MarkNameAsReferenced(Export.ObjectName);
		ExportMapEntry.ObjectName = Package->NameMapBuilder.MapName(Export.ObjectName);
		ExportMapEntry.PublicExportHash = Export.PublicExportHash;
		ExportMapEntry.OuterIndex = Export.OuterIndex;
		ExportMapEntry.ClassIndex = Export.ClassIndex;
		ExportMapEntry.SuperIndex = Export.SuperIndex;
		ExportMapEntry.TemplateIndex = Export.TemplateIndex;
		ExportMapEntry.ObjectFlags = Export.ObjectFlags;
		ExportMapEntry.FilterFlags = EExportFilterFlags::None;
		if (Export.bNotForClient)
		{
			ExportMapEntry.FilterFlags = EExportFilterFlags::NotForClient;
		}
		else if (Export.bNotForServer)
		{
			ExportMapEntry.FilterFlags = EExportFilterFlags::NotForServer;
		}

		ExportMapArchive << ExportMapEntry;
	}
	uint64 ExportMapSize = ExportMapArchive.Tell();

	FBufferWriter CellImportMapArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (FPackageObjectIndex& CellImport : Package->CellImports)
	{
		CellImportMapArchive << CellImport;
	}
	uint64 CellImportMapSize = CellImportMapArchive.Tell();

	FBufferWriter CellExportMapArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (const FPackageStorePackage::FCellExport& CellExport : Package->CellExports)
	{
		FCellExportMapEntry CellExportMapEntry;
		CellExportMapEntry.CookedSerialOffset = CellExport.SerialOffset;
		CellExportMapEntry.CookedSerialLayoutSize = CellExport.SerialLayoutSize;
		CellExportMapEntry.CookedSerialSize = CellExport.SerialSize;
		CellExportMapEntry.PublicExportHash = CellExport.PublicExportHash;
		Package->NameMapBuilder.MarkNameAsReferenced(CellExport.CppClassInfo);
		CellExportMapEntry.CppClassInfo = Package->NameMapBuilder.MapName(CellExport.CppClassInfo);

		CellExportMapArchive << CellExportMapEntry;
	}
	uint64 CellExportMapSize = CellExportMapArchive.Tell();

	FBufferWriter ExportBundleEntriesArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (FExportBundleEntry BundleEntry : Package->GraphData.ExportBundleEntries)
	{
		ExportBundleEntriesArchive << BundleEntry;
	}
	uint64 ExportBundleEntriesSize = ExportBundleEntriesArchive.Tell();

	FBufferWriter DependencyBundleHeadersArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (FDependencyBundleHeader DependencyBundleHeader : Package->GraphData.DependencyBundleHeaders)
	{
		DependencyBundleHeadersArchive << DependencyBundleHeader;
	}

	FBufferWriter DependencyBundleEntriesArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (FDependencyBundleEntry DependencyBundleEntry : Package->GraphData.DependencyBundleEntries)
	{
		DependencyBundleEntriesArchive << DependencyBundleEntry;
	}

	uint64 GraphDataSize = DependencyBundleHeadersArchive.Tell() + DependencyBundleEntriesArchive.Tell();

	Package->NameMapBuilder.MarkNameAsReferenced(Package->Name);
	FMappedName MappedPackageName = Package->NameMapBuilder.MapName(Package->Name);

	FZenPackageImportedPackageNamesContainer ImportedPackageNamesContainer;
	ImportedPackageNamesContainer.Names.Reserve(Package->ImportedPackages.Num());
	for (const FPackageStorePackage::FImportedPackageRef& ImportedPackage : Package->ImportedPackages)
	{
		ImportedPackageNamesContainer.Names.Add(ImportedPackage.Name);
	}
	FBufferWriter ImportedPackagesArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	ImportedPackagesArchive << ImportedPackageNamesContainer;
	uint64 ImportedPackagesSize = ImportedPackagesArchive.Tell();

	FBufferWriter NameMapArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	SaveNameBatch(Package->NameMapBuilder.GetNameMap(), NameMapArchive);
	uint64 NameMapSize = NameMapArchive.Tell();

	FBufferWriter VersioningInfoArchive(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	if (Package->VersioningInfo.IsSet())
	{
		VersioningInfoArchive << Package->VersioningInfo.GetValue();
	}
	uint64 VersioningInfoSize = VersioningInfoArchive.Tell();

	
	FBufferWriter BulkDataMapAr(nullptr, 0, EBufferWriterFlags::AllowResize | EBufferWriterFlags::TakeOwnership);
	for (FBulkDataMapEntry& Entry : Package->BulkDataEntries)
	{
		BulkDataMapAr << Entry;
	}
	uint64 BulkDataMapSize = BulkDataMapAr.Tell();

	uint64 BulkDataPad = 0;
	uint64 OffsetBeforeBulkDataMap =
		sizeof(FZenPackageSummary)
		+ VersioningInfoSize
		+ sizeof(FZenPackageCellOffsets)
		+ NameMapSize
		+ sizeof(BulkDataPad);

	uint64 AlignedOffsetBeforeBulkDataMap = Align(OffsetBeforeBulkDataMap, sizeof(uint64));
	BulkDataPad = AlignedOffsetBeforeBulkDataMap - OffsetBeforeBulkDataMap;

	uint64 OffsetBeforePublicExportHashes =
		AlignedOffsetBeforeBulkDataMap
		+ BulkDataMapSize + sizeof(BulkDataMapSize);

	uint64 AlignedOffsetBeforePublicExportHashes = Align(OffsetBeforePublicExportHashes, sizeof(uint64));

	uint64 HeaderSize =
		AlignedOffsetBeforePublicExportHashes
		+ ImportedPublicExportHashesSize
		+ ImportMapSize
		+ ExportMapSize
		+ CellImportMapSize
		+ CellExportMapSize
		+ ExportBundleEntriesSize
		+ GraphDataSize
		+ ImportedPackagesSize;

	// NOTE: The CookedHeaderSize has been unused since EUnrealEngineObjectUE5Version::DATA_RESOURCES that added the bulk/data resource table.
	//       It should ultimately be removed, until then freeze the FPackageFileSummary baseline impact on serialized CookedHeaderSize at
	//       EUnrealEngineObjectUE5Version::OS_SUB_OBJECT_SHADOW_SERIALIZATION.
	// NOTE: EUnrealEngineObjectUE5Version::IMPORT_TYPE_HIERARCHIES adds two int32 fields to FPackageFileSummary. This increases the total header size by 8 bytes.
	// NOTE: operator<<(FStructuredArchive::FSlot Slot, FObjectImport& I) added unversioned cooked serialization of I.PackageName, i.e. two int32 fields for every import entry.
#if WITH_EDITORONLY_DATA
	constexpr size_t ExpectedPackageFileSummarySize = 360;
#else
	constexpr size_t ExpectedPackageFileSummarySize = 344;
#endif
	static_assert(sizeof(FPackageFileSummary) == ExpectedPackageFileSummarySize, "Any change to FPackageFileSummary serialization requires adjusting the offset below, transient changes can just update the size in this assert.");
	constexpr uint32 CookedHeaderSizeForImportTypeHierarchies = 8; // sizeof(ImportTypeHierarchiesCount) + sizeof(ImportTypeHierarchiesOffset) = 8
	const uint32 CookedHeaderSizeForImportPackageNames = 8 * Package->Imports.Num(); // sizeof(NameIndex) + sizeof(NameNumber) = 8
	const uint32 CookedHeaderSizeOffsetToMinimizePatchSize = CookedHeaderSizeForImportTypeHierarchies + CookedHeaderSizeForImportPackageNames;

	Package->HeaderBuffer = FIoBuffer(HeaderSize);
	uint8* HeaderData = Package->HeaderBuffer.Data();
	FMemory::Memzero(HeaderData, HeaderSize);
	FZenPackageSummary* PackageSummary = reinterpret_cast<FZenPackageSummary*>(HeaderData);
	PackageSummary->HeaderSize = HeaderSize;
	PackageSummary->Name = MappedPackageName;
	PackageSummary->PackageFlags = Package->PackageFlags;
	PackageSummary->_Unused = Package->CookedHeaderSize - CookedHeaderSizeOffsetToMinimizePatchSize; // Was CookedHeaderSize
	FBufferWriter HeaderArchive(HeaderData, HeaderSize);
	HeaderArchive.Seek(sizeof(FZenPackageSummary));

	if (Package->VersioningInfo.IsSet())
	{
		PackageSummary->bHasVersioningInfo = 1;
		HeaderArchive.Serialize(VersioningInfoArchive.GetWriterData(), VersioningInfoArchive.Tell());
	}
	else
	{
		PackageSummary->bHasVersioningInfo = 0;
	}

	FZenPackageCellOffsets* CellOffsets = HeaderArchive.SerializeUninitialized<FZenPackageCellOffsets>();

	HeaderArchive.Serialize(NameMapArchive.GetWriterData(), NameMapArchive.Tell());

	HeaderArchive << BulkDataPad;
	if (BulkDataPad > 0)
	{
		uint8 PadBytes[sizeof(uint64)] = {};
		HeaderArchive.Serialize(PadBytes, BulkDataPad);
	}
	check(HeaderArchive.Tell() == AlignedOffsetBeforeBulkDataMap);

	HeaderArchive << BulkDataMapSize;
	HeaderArchive.Serialize(BulkDataMapAr.GetWriterData(), BulkDataMapSize);

	if (uint64 Pad=AlignedOffsetBeforePublicExportHashes-OffsetBeforePublicExportHashes; Pad > 0)
	{
		uint8 PadBytes[sizeof(uint64)] = {};
		HeaderArchive.Serialize(PadBytes, Pad);
	}
	check(HeaderArchive.Tell() == AlignedOffsetBeforePublicExportHashes);

	// raw arrays of 8-byte aligned items
	PackageSummary->ImportedPublicExportHashesOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(ImportedPublicExportHashesArchive.GetWriterData(), ImportedPublicExportHashesArchive.Tell());
	PackageSummary->ImportMapOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(ImportMapArchive.GetWriterData(), ImportMapArchive.Tell());
	PackageSummary->ExportMapOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(ExportMapArchive.GetWriterData(), ExportMapArchive.Tell());
	CellOffsets->CellImportMapOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(CellImportMapArchive.GetWriterData(), CellImportMapArchive.Tell());
	CellOffsets->CellExportMapOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(CellExportMapArchive.GetWriterData(), CellExportMapArchive.Tell());
	// raw arrays of 4-byte aligned items
	PackageSummary->ExportBundleEntriesOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(ExportBundleEntriesArchive.GetWriterData(), ExportBundleEntriesArchive.Tell());
	PackageSummary->DependencyBundleHeadersOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(DependencyBundleHeadersArchive.GetWriterData(), DependencyBundleHeadersArchive.Tell());
	PackageSummary->DependencyBundleEntriesOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(DependencyBundleEntriesArchive.GetWriterData(), DependencyBundleEntriesArchive.Tell());
	PackageSummary->ImportedPackageNamesOffset = HeaderArchive.Tell();
	HeaderArchive.Serialize(ImportedPackagesArchive.GetWriterData(), ImportedPackagesArchive.Tell());
	check(HeaderArchive.Tell() == PackageSummary->HeaderSize)
}

FIoBuffer FPackageStoreOptimizer::CreatePackageBuffer(const FPackageStorePackage* Package, const FIoBuffer& CookedExportsDataBuffer) const
{
	check(Package->HeaderBuffer.DataSize() > 0);
	const uint64 BundleBufferSize = Package->HeaderBuffer.DataSize() + CookedExportsDataBuffer.DataSize();
	FIoBuffer BundleBuffer(BundleBufferSize);
	FMemory::Memcpy(BundleBuffer.Data(), Package->HeaderBuffer.Data(), Package->HeaderBuffer.DataSize());
	FMemory::Memcpy(BundleBuffer.Data() + Package->HeaderBuffer.DataSize(), CookedExportsDataBuffer.Data(), CookedExportsDataBuffer.DataSize());
	return BundleBuffer;
}

void FPackageStoreOptimizer::FindScriptObjectsRecursive(
	TMap<FPackageObjectIndex, FScriptObjectData>& OutScriptObjectsMap, FPackageObjectIndex OuterIndex, bool bOuterIsVerseVNI, UObject* Object)
{
	if (!Object->HasAllFlags(RF_Public))
	{
		UE_LOGF(LogPackageStoreOptimizer, Verbose, "Skipping script object: %ls (!RF_Public)", *Object->GetFullName());
		return;
	}

	FString OuterFullName;
	FPackageObjectIndex OuterCDOClassIndex;
	{
		const FScriptObjectData* Outer = OutScriptObjectsMap.Find(OuterIndex);
		check(Outer);
		OuterFullName = Outer->FullName;
		OuterCDOClassIndex = Outer->CDOClassIndex;
	}

	// Unlike things in /Scripts/, with Verse VNI objects, there is a mix of UHT generated types, which will always be 
	// available, and Verse compiler generated types which need to be cooked and packaged.  We don't want the compiler
	// generated types to be included in this collection.
	if (bOuterIsVerseVNI && !Verse::VerseVM::IsUHTGeneratedVerseVNIObject(Object))
	{
		return;
	}

	FName ObjectName = Object->GetFName();

	FString TempFullName = OuterFullName;
	TempFullName.AppendChar(TEXT('/'));
	ObjectName.AppendString(TempFullName);

	TempFullName.ToLowerInline();
	FPackageObjectIndex GlobalImportIndex = FPackageObjectIndex::FromScriptPath(TempFullName);

	FScriptObjectData* ScriptImport = OutScriptObjectsMap.Find(GlobalImportIndex);
	if (ScriptImport)
	{
		UE_LOGF(LogPackageStoreOptimizer, Fatal, "Import name hash collision \"%ls\" and \"%ls", *TempFullName, *ScriptImport->FullName);
	}

	FPackageObjectIndex CDOClassIndex = OuterCDOClassIndex;
	if (CDOClassIndex.IsNull())
	{
		TCHAR NameBuffer[FName::StringBufferSize];
		uint32 Len = ObjectName.ToString(NameBuffer);
		if (FCString::Strncmp(NameBuffer, TEXT("Default__"), 9) == 0)
		{
			FString CDOClassFullName = OuterFullName;
			CDOClassFullName.AppendChar(TEXT('/'));
			CDOClassFullName.AppendChars(NameBuffer + 9, Len - 9);
			CDOClassFullName.ToLowerInline();

			CDOClassIndex = FPackageObjectIndex::FromScriptPath(CDOClassFullName);
		}
	}

	ScriptImport = &OutScriptObjectsMap.Add(GlobalImportIndex);
	ScriptImport->GlobalIndex = GlobalImportIndex;
	ScriptImport->FullName = MoveTemp(TempFullName);
	ScriptImport->OuterIndex = OuterIndex;
	ScriptImport->ObjectName = ObjectName;
	ScriptImport->CDOClassIndex = CDOClassIndex;

	TArray<UObject*> InnerObjects;
	GetObjectsWithOuter(Object, InnerObjects, EGetObjectsFlags::None);
	for (UObject* InnerObject : InnerObjects)
	{
		FindScriptObjectsRecursive(OutScriptObjectsMap, GlobalImportIndex, bOuterIsVerseVNI, InnerObject);
	}
};

void FPackageStoreOptimizer::FindScriptObjects()
{
	FindScriptObjects(ScriptObjectsMap);
	TotalScriptObjectCount = ScriptObjectsMap.Num();

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	if (Verse::VPackage* BuiltInPackage = Verse::GlobalProgram->LookupPackage("$BuiltIn"))
	{
		for (int32 Index = 0; Index < BuiltInPackage->NumDefinitions(); ++Index)
		{
			Verse::VUniqueString& VersePath = BuiltInPackage->GetDefinitionName(Index);
			FPackageObjectIndex GlobalImportIndex = FPackageObjectIndex::FromVersePath(VersePath.AsStringView());

			FScriptCellData* ScriptImport = ScriptCellsMap.Find(GlobalImportIndex);
			if (ScriptImport)
			{
				UE_LOGF(LogPackageStoreOptimizer, Fatal, "Import name hash collision \"%ls\" and \"%ls\"", *VersePath.AsString(), *FString(ScriptImport->VersePath));
			}

			ScriptImport = &ScriptCellsMap.Add(GlobalImportIndex);
			ScriptImport->GlobalIndex = GlobalImportIndex;
			ScriptImport->VersePath = VersePath.AsStringView();
		}
	}
#endif
}

void FPackageStoreOptimizer::FindScriptObjects(TMap<FPackageObjectIndex, FScriptObjectData>& OutScriptObjectsMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindScriptObjects);
	FRuntimeScriptPackages ScriptPackages;
	FindAllRuntimeScriptPackages(ScriptPackages);

	TArray<UObject*> InnerObjects;

	auto FindScriptObjectsInPackages = [&OutScriptObjectsMap, &InnerObjects](const TArray<UPackage*>& Packages, bool bIsVerseVNI)
		{
			for (UPackage* Package : Packages)
			{
				FName ObjectName = Package->GetFName();
				FString FullName = Package->GetName();

				FullName.ToLowerInline();
				FPackageObjectIndex GlobalImportIndex = FPackageObjectIndex::FromScriptPath(FullName);

				FScriptObjectData* ScriptImport = OutScriptObjectsMap.Find(GlobalImportIndex);
				checkf(!ScriptImport, TEXT("Import name hash collision \"%s\" and \"%s"), *FullName, *ScriptImport->FullName);

				ScriptImport = &OutScriptObjectsMap.Add(GlobalImportIndex);
				ScriptImport->GlobalIndex = GlobalImportIndex;
				ScriptImport->FullName = FullName;
				ScriptImport->OuterIndex = FPackageObjectIndex();
				ScriptImport->ObjectName = ObjectName;

				InnerObjects.Reset();
				GetObjectsWithOuter(Package, InnerObjects, EGetObjectsFlags::None);
				for (UObject* InnerObject : InnerObjects)
				{
					FindScriptObjectsRecursive(OutScriptObjectsMap, GlobalImportIndex, bIsVerseVNI, InnerObject);
				}
			}
		};

	FindScriptObjectsInPackages(ScriptPackages.Script, false);
	FindScriptObjectsInPackages(ScriptPackages.VerseVNI, true);
}

FIoBuffer FPackageStoreOptimizer::CreateScriptObjectsBuffer() const
{
	TArray<FScriptObjectData> ScriptObjectsAsArray;
	ScriptObjectsMap.GenerateValueArray(ScriptObjectsAsArray);
	Algo::Sort(ScriptObjectsAsArray, [](const FScriptObjectData& A, const FScriptObjectData& B)
	{
		return A.FullName < B.FullName;
	});
	
	TArray<FScriptObjectEntry> ScriptObjectEntries;
	ScriptObjectEntries.Reserve(ScriptObjectsAsArray.Num());
	FPackageStoreNameMapBuilder NameMapBuilder;
	NameMapBuilder.SetNameMapType(FMappedName::EType::Global);
	for (const FScriptObjectData& ImportData : ScriptObjectsAsArray)
	{
		NameMapBuilder.MarkNameAsReferenced(ImportData.ObjectName);
		FScriptObjectEntry& Entry = ScriptObjectEntries.AddDefaulted_GetRef();
		Entry.Mapped = NameMapBuilder.MapName(ImportData.ObjectName);
		Entry.GlobalIndex = ImportData.GlobalIndex;
		Entry.OuterIndex = ImportData.OuterIndex;
		Entry.CDOClassIndex = ImportData.CDOClassIndex;
	}

	FLargeMemoryWriter ScriptObjectsArchive(0, true);
	SaveNameBatch(NameMapBuilder.GetNameMap(), ScriptObjectsArchive);
	int32 NumScriptObjects = ScriptObjectEntries.Num();
	ScriptObjectsArchive << NumScriptObjects;
	for (FScriptObjectEntry& Entry : ScriptObjectEntries)
	{
		ScriptObjectsArchive << Entry;
	}

	int64 DataSize = ScriptObjectsArchive.TotalSize();
	return FIoBuffer(FIoBuffer::AssumeOwnership, ScriptObjectsArchive.ReleaseOwnership(), DataSize);
}

void FPackageStoreOptimizer::LoadScriptObjectsBuffer(const FIoBuffer& ScriptObjectsBuffer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LoadScriptObjectsBuffer);
	FLargeMemoryReader ScriptObjectsArchive(ScriptObjectsBuffer.Data(), ScriptObjectsBuffer.DataSize());
	TArray<FDisplayNameEntryId> NameMap = LoadNameBatch(ScriptObjectsArchive);
	int32 NumScriptObjects;
	ScriptObjectsArchive << NumScriptObjects;
	for (int32 Index = 0; Index < NumScriptObjects; ++Index)
	{
		FScriptObjectEntry Entry{};
		ScriptObjectsArchive << Entry;
		FScriptObjectData& ImportData = ScriptObjectsMap.Add(Entry.GlobalIndex);
		FMappedName MappedName = Entry.Mapped;
		ImportData.ObjectName = NameMap[MappedName.GetIndex()].ToName(MappedName.GetNumber());
		ImportData.GlobalIndex = Entry.GlobalIndex;
		ImportData.OuterIndex = Entry.OuterIndex;
		ImportData.CDOClassIndex = Entry.CDOClassIndex;
	}
}

FPackageStoreEntryResource FPackageStoreOptimizer::CreatePackageStoreEntry(const FPackageStorePackage* Package, const FPackageStorePackage* OptionalSegmentPackage) const
{
	FPackageStoreEntryResource Result;
	Result.Flags = EPackageStoreEntryFlags::HasPackageData;

	if (OptionalSegmentPackage)
	{
		Result.Flags |= EPackageStoreEntryFlags::OptionalSegment;
		if (OptionalSegmentPackage->HasEditorData())
		{
			// AutoOptional packages are saved with editor data included
			Result.Flags |= EPackageStoreEntryFlags::AutoOptional;
		}
	}

	if (Package->PackageFlags & PKG_LoadUncooked)
	{
		Result.Flags |= EPackageStoreEntryFlags::LoadUncooked;
	}
	
	Result.PackageName = Package->Name;
	Result.PackageId = FPackageId::FromName(Package->Name);
	Result.ImportedPackageIds.Reserve(Package->ImportedPackages.Num());
	for (const FPackageStorePackage::FImportedPackageRef& ImportedPackage : Package->ImportedPackages)
	{
		Result.ImportedPackageIds.Add(ImportedPackage.Id);
	}
	
	if (OptionalSegmentPackage)
	{
		Result.OptionalSegmentImportedPackageIds.Reserve(OptionalSegmentPackage->ImportedPackages.Num());
		for (const FPackageStorePackage::FImportedPackageRef& ImportedPackage : OptionalSegmentPackage->ImportedPackages)
		{
			Result.OptionalSegmentImportedPackageIds.Add(ImportedPackage.Id);
		}
	}

	Result.SoftPackageReferences.Reserve(Package->SoftPackageReferences.Num());
	for (const FName& SoftRefName : Package->SoftPackageReferences)
	{
		TCHAR NameStr[FName::StringBufferSize];
		const uint32 NameLen = SoftRefName.ToString(NameStr);
		const FStringView SoftRef(NameStr, NameLen);

		if (!FPackageName::IsScriptPackage(SoftRef))
		{
			if (!FPackageName::IsValidLongPackageName(SoftRef))
			{
				UE_LOGF(LogPackageStoreOptimizer, Display, "Skipping invalid soft package reference name '%ls'", *SoftRefName.ToString());
				continue;
			}

			Result.SoftPackageReferences.Add(FPackageId::FromName(SoftRefName));
		}
	}
	Result.SoftPackageReferences.Sort();

	return Result;
}
