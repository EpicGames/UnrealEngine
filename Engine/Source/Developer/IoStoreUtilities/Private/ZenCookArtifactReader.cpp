// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenCookArtifactReader.h"

#include "Experimental/ZenServerInterface.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "StorageServerClientModule.h"
#include "ZenStoreHttpClient.h"

FZenCookArtifactReader::FZenCookArtifactReader(
	const FString& InputPath, 
	const FString& InMetadataDirectoryPath, 
	const ITargetPlatform* InTargetPlatform
)
#if !UE_BUILD_SHIPPING
	: ScopeZenService(MakeUnique<UE::Zen::FScopeZenService>())
	, ZenRootPath(InputPath)
	, StorageServerPlatformFile(nullptr)
{
}
#else
{
}
#endif // !UE_BUILD_SHIPPING

FZenCookArtifactReader::~FZenCookArtifactReader()
{
}

void FZenCookArtifactReader::InitializeConnection()
{
#if !UE_BUILD_SHIPPING
	if (StorageServerPlatformFile)
	{
		return;
	}

	StorageServerPlatformFile.Reset(IStorageServerClientModule::Get().TryCreateCustomPlatformFile(*ZenRootPath,
		&FPlatformFileManager::Get().GetPlatformFile()));
	if (StorageServerPlatformFile)
	{
		StorageServerPlatformFile->SetLowerLevel(nullptr);
	}
#endif
}

void FZenCookArtifactReader::Initialize(bool bCleanBuild)
{
	InitializeConnection();
#if !UE_BUILD_SHIPPING
	if (bCleanBuild && StorageServerPlatformFile)
	{
		StorageServerPlatformFile->UpdateFileList();
	}
#endif
}

bool FZenCookArtifactReader::FileExists(const TCHAR* Filename)
{
#if !UE_BUILD_SHIPPING
	if (StorageServerPlatformFile)
	{
		FString StandardFilename;
		if (MakeStorageServerPath(Filename, StandardFilename) && (StorageServerPlatformFile->FileSize(*StandardFilename) >= 0))
		{
			return StorageServerPlatformFile->FileExists(*StandardFilename);
		}
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}

int64 FZenCookArtifactReader::FileSize(const TCHAR* Filename)
{
#if !UE_BUILD_SHIPPING
	if (StorageServerPlatformFile)
	{
		FString StandardFilename;
		if (MakeStorageServerPath(Filename, StandardFilename))
		{
			return StorageServerPlatformFile->FileSize(*StandardFilename);
		}
	}
#endif // !UE_BUILD_SHIPPING

	return -1;
}

IFileHandle* FZenCookArtifactReader::OpenRead(const TCHAR* Filename)
{
#if !UE_BUILD_SHIPPING
	if (StorageServerPlatformFile)
	{
		FString StandardFilename;
		if (MakeStorageServerPath(Filename, StandardFilename) && (StorageServerPlatformFile->FileSize(*StandardFilename) >= 0))
		{
			return StorageServerPlatformFile->OpenRead(*StandardFilename);
		}
	}
#endif // !UE_BUILD_SHIPPING

	return nullptr;
}

bool FZenCookArtifactReader::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
#if !UE_BUILD_SHIPPING
	if (StorageServerPlatformFile)
	{
		FString StandardDirectory;
		if (MakeStorageServerPath(Directory, StandardDirectory))
		{
			return StorageServerPlatformFile->IterateDirectory(*StandardDirectory, Visitor);
		}
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}

#if !UE_BUILD_SHIPPING
bool FZenCookArtifactReader::MakeStorageServerPath(const TCHAR* Filename, FString& OutFilename) const
{
	OutFilename = Filename;
	if (FPaths::IsUnderDirectory(OutFilename, ZenRootPath) && FPaths::MakePathRelativeTo(OutFilename, *ZenRootPath))
	{
		static FString ProjectPrefix = FString::Printf(TEXT("%s/"), FApp::GetProjectName());
		if (OutFilename.StartsWith(*ProjectPrefix))
		{
			OutFilename.RightChopInline(ProjectPrefix.Len());
			OutFilename.InsertAt(0, *FPaths::ProjectDir());
		}
		else
		{
			OutFilename.InsertAt(0, "../../../");
		}
		return true;
	}
	return false;
}
#endif // !UE_BUILD_SHIPPING