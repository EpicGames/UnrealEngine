// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenFileSystemManifest.h"

#include "Algo/Sort.h"
#include "Cooker/ICookSandbox.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/PathViews.h"
#include "Settings/ProjectPackagingSettings.h"
#include "UObject/ICookInfo.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenFileSystemManifest, Display, All);

namespace UE::ZenFileSystemManifest
{

/** Reports whether files should pass a filter based on extension include/exclude and directory exclude properties. */
class FFileFilter
{
public:
	/** Mark that directories with the given leafname are excluded. Returns *this for chained function calls. */
	FFileFilter& ExcludeDirectoryLeafName(const TCHAR* LeafName);
	/**
	 * Mark that filenames with the given extension (which must be provided without a leading dot) are excluded.
	 * Returns *this for chained function calls.
	 */
	FFileFilter& ExcludeExtension(const TCHAR* NoDotExtension);
	/**
	 * Mark that filenames with the given extension  (which must be provided without a leading dot) are included, if
	 * not excluded. Overruled by ExcludeExtension.
	 * If no extensions are explicitly included then all non-excluded extensions are included.
	 * Returns *this for chained function calls.
	 */
	FFileFilter& IncludeExtension(const TCHAR* NoDotExtension);

	/** Report whether a directory with the given LeafName is kept by the filter. */
	bool IsDirectoryLeafNameKept(FStringView LeafName) const;
	/** Report whether the given extension (which must be provided without a leading dot) is kept by the filter. */
	bool IsFileExtensionKept(FStringView NoDotExtension);

private:
	TArray<FString> DirectoryExclusionFilter;
	TArray<FString> ExtensionExclusionFilter;
	TArray<FString> ExtensionInclusionFilter;
};

/**
 * A visitor used by FManifestGenerator::AddFilesFromDirectory to receive the results from
 * IPlatformFile::IterateDirectory.
 */
struct FAddFilesFromDirectoryVisitor
{
public:
	FAddFilesFromDirectoryVisitor(FManifestGenerator& InGenerator, const FString& InClientRoot,
		const FString& InLocalRoot, bool bInIncludeSubDirs, FFileFilter* InAdditionalFilter);

	/**
	 * Callback passed to IPlatformFile::IterateDirectory. Filters and adds files
	 * and subdirectories. Kept files are added to Generator.Manifest. Kept subdirectories are added to
	 * this->DirectoryVisitQueue. Return value is whether IterateDirectory should continue.
	 */
	bool VisitorFunction(const TCHAR* InnerFileNameOrDirectory, bool bIsDirectory);

public:
	FManifestGenerator& Generator;
	TArray<FString> DirectoryVisitQueue;
	const FString& ClientRoot;
	const FString& LocalRoot;
	FFileFilter* AdditionalFilter = nullptr;
	bool bIncludeSubDirs = false;
};

/**
 * Helper class for FManifestGenerator::Generate. Has functions that support collecting files from directories in the
 * uncooked folders used by or the cooked folders created by the cook. Specifications of which files to collect from
 * which kinds of folders are driven by the top-level code in Generate.
 */
class FManifestGenerator
{
public:
	FManifestGenerator(FZenFileSystemManifest& InManifest,
		const TOptional<ICookedPackageWriter::FReferencedPluginsInfo>& InReferencedPlugins);

	/** FPaths::RootDir, the root of the UnrealEngine tree, parent directory of EngineDir. */
	const FString& GetUnrealEngineRootDir() const;
	/** FPaths::EngineDir, the root of assets and source code shared across all Unreal projects. */
	const FString& GetEngineDir() const;
	/**
	 * FPaths::ProjectDir, the root of the project's project-specific assets and source code. Contains the .uproject
	 * file. Not necessarily the parent directory of all of the project's assets; some assets can be present in plugin
	 * folders outside of the ProjectDir.
	 */
	const FString& GetProjectDir() const;
	/**
	 * FPaths::ProjectName, the name of the project, which we expect to find as a direct subdirectory of the cooked
	 * output's root directory.
	 */
	const FString& GetProjectName() const;
	/** <CookDirectory>/Engine */
	FString GetCookedEngineDir() const;
	/** <CookDirectory>/<ProjectName> */
	FString GetCookedProjectDir() const;
	/** <CookDirectory>/<ProjectName>/Metadata */
	FString GetCookedMetadataDir() const;
	/** TargetPlatform's PlatformInfo's UBTPlatformString. */
	const FString& GetUBTPlatformString() const;
	/** Read/Write the BaseFilter the Generator uses for adding files from directories. */
	FFileFilter& GetBaseFilter();

	/**
	 * Add the given file to the manifest, stored relative to the given PathMappingPair of ClientRoot,LocalRoot. A
	 * PathMappingPair represents the two possible formats for the same root directory (the format for loading the path
	 * on the UnrealGame client and the format for loading the path on the cooker's machine) and the relative path
	 * to a file under the mapping pair is the same for each root. LocalRoot should be a parent directory of
	 * LocalFilePath; if it is not, the stored path falls back to absolute path on the cooker's machine and will not
	 * be loadable on the client.
	 */
	void AddSingleFile(const FString& ClientRoot, const FString& LocalRoot, FStringView LocalFilePath);
	/** If file exists at LocalRoot\RelPathFromRoot, AddSingleFile on it. Return whether it was added. */
	bool TryAddSingleRelpath(const FString& ClientRoot, const FString& LocalRoot, const FString& RelPathFromRoot);

	/**
	 * Add to the manifest all (optionally recursive) files under LocalAddRoot, if they pass the Generator's basefilter
	 * and the given Additional Filter. ClientRoot,LocalRoot are a PathMappingPair, with LocalRoot being a parent of
	 * LocalAddRoot, @see AddSingleFile. If LocalAddRoot is not provided it is set equal to LocalRoot.
	 */
	void AddFilesFromDirectory(const FString& ClientRoot, const FString& LocalRoot, bool bIncludeSubdirs,
		FFileFilter* AdditionalFilter = nullptr);
	void AddFilesFromDirectory(const FString& ClientRoot, const FString& LocalRoot, const FString& LocalAddRoot,
		bool bIncludeSubdirs, FFileFilter* AdditionalFilter = nullptr);
	/**
	 * If the given path is a subpath of one of the known client roots (EngineDir, ProjectDir), call
	 * AddFilesFromDirectory on it and return true, else return false.
	 */
	bool TryAddFilesFromFlexDirectory(const FString& FullOrProjectRelativePath, bool bIncludeSubDirs,
		FFileFilter* AdditionalFilter = nullptr);

	/**
	 * A collector AddFiles function. It looks in all GetExtensionDirectories() folders for a direct subdirectory
	 * with leafname equal to ExtensionSubDir, and calls AddFilesFromDirectory on each of those ExtensionSubDir
	 * directories, with bRecursive==true and with the given filter.
	 */
	void AddFilesFromExtensionDirectories(const TCHAR* ExtensionSubDir, FFileFilter* AdditionalFilter = nullptr);

	/**
	 * Return all the plugins that could be used by the runtime the manifest will be used in - plugins are filtered by
	 * the current project and platform.
	 */
	TArray<TSharedRef<IPlugin>> GetApplicablePlugins();
	/**
	 * Return the ClientRoot,LocalRoot PathMappingPair that is one of the manifest's recognized root directories
	 * and is the parent directory of the plugin's uncooked directory.
	 * These roots allow the construction of the relative path used to store the plugin's files on the ZenServer.
	 */
	void GetPluginClientAndLocalRoots(TSharedRef<IPlugin>& Plugin, FString& OutClientRoot, FString& OutLocalRoot);
	/**
	 * For a given plugin, return full paths to all of the (possibly-platform-specific) ExtensionBaseDirs that are
	 * present in the plugin and are applicable to the current project and platform. These directories need the same
	 * set of relative paths added to the manifest that are added for the plugin's base dir.
	 */
	TArray<FString> GetApplicablePluginExtensionBaseDirs(TSharedRef<IPlugin>& Plugin);

	/** Return the subfolder used for localization assets based on the project's Internationalization settings. */
	FString GetInternationalizationPresetPath();

	/**
	 * Find the cooked path for the given source-domain path from the CookSandbox, and if found, return the
	 * cookedpath and the (Client,LocalRoot) pair the cookedpath is under.
	 */
	bool TryGetCookedPathOfSourceDomainPath(const FString& SourceDomainPath, FString& OutCookedPath,
		FString& OutClientRoot, FString& OutLocalRoot);

	/** Return bSSLCertificatesWillStage from network settings for the cook's targetplatform, false by default. */
	bool IsSSLCertificatesWillStage();

	/** Return true if the manifest should gather files from the Engine. This can be false during DLC cooks. */
	bool IsIncludeEngine();

	/** Return true if the manifest should gather files from the Project. This can be false during DLC cooks. */
	bool IsIncludeProject();

	/** Return the relative path from LocalRoot to LocalFilePath joined onto ClientRoot. */
	static FString ConvertLocalPathToClientPath(const FString& ClientRoot, const FString& LocalRoot, FStringView LocalFilePath);

private:
	void PopulatePlatformDirectoryNames();

	static void GetExtensionDirs(TArray<FString>& OutExtensionDirs, const TCHAR* BaseDir, const TCHAR* SubDir,
		const TArray<FString>& PlatformDirectoryNames);

private:
	FZenFileSystemManifest& Manifest;
	const TOptional<ICookedPackageWriter::FReferencedPluginsInfo>& ReferencedPlugins;
	IPlatformFile& PlatformFile;
	FString UnrealEngineRootDir;
	FString EngineDir;
	FString ProjectDir;
	FString ProjectName;
	FFileFilter BaseFilter;
	TArray<FString> PlatformDirectoryNames;
	TSet<FString> PlatformDirectoryNameSet;
	FString UBTPlatformString;

	friend FAddFilesFromDirectoryVisitor;
};

} // namespace UE::ZenFileSystemManifest

const FZenFileSystemManifestEntry FZenFileSystemManifest::InvalidEntry = FZenFileSystemManifestEntry();

FZenFileSystemManifest::FZenFileSystemManifest(const ITargetPlatform& InTargetPlatform, FString InCookDirectory)
	: TargetPlatform(InTargetPlatform)
	, CookDirectory(MoveTemp(InCookDirectory))
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	ServerRoot = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*FPaths::RootDir());
	FPaths::NormalizeDirectoryName(ServerRoot);
}

#if WITH_EDITOR
void FZenFileSystemManifest::ConstructReferencedSetClientPath(const FString& MetadataDirectoryPath)
{
	using namespace UE::ZenFileSystemManifest;
	FString ReferencedSetLocalPath = FPaths::Combine(MetadataDirectoryPath, UE::Cook::GetReferencedSetFilename());
	FString ProjectClientRoot = TEXT("/{project}");
	FString ProjectLocalRoot = FPaths::Combine(CookDirectory, FApp::GetProjectName());
	ReferencedSetClientPath = FManifestGenerator::ConvertLocalPathToClientPath(ProjectClientRoot,
		ProjectLocalRoot, ReferencedSetLocalPath);
	ReferencedSetClientPath.ReplaceInline(TEXT("\\"), TEXT("/"));
}
void FZenFileSystemManifest::SetCookSandbox(UE::Cook::ICookSandbox* InCookSandbox)
{
	CookSandbox = InCookSandbox;
}
#endif

int32 FZenFileSystemManifest::Generate(const FString& MetadataDirectoryPath,
	const TOptional<ICookedPackageWriter::FReferencedPluginsInfo>& ReferencedPlugins)
{
	using namespace UE::ZenFileSystemManifest;

	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateStorageServerFileSystemManifest);

	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();
	const int32 PreviousEntryCount = NumEntries();

	FManifestGenerator Generator(*this, ReferencedPlugins);

	Generator.GetBaseFilter()
		.ExcludeDirectoryLeafName(TEXT("Binaries"))
		.ExcludeDirectoryLeafName(TEXT("Intermediate"))
		.ExcludeDirectoryLeafName(TEXT("Saved"))
		.ExcludeDirectoryLeafName(TEXT("Source"));

	FFileFilter CookedFilter = FFileFilter()
		.ExcludeDirectoryLeafName(TEXT("Metadata"))
		.ExcludeExtension(TEXT("uasset"))
		.ExcludeExtension(TEXT("ubulk"))
		.ExcludeExtension(TEXT("uexp"))
		.ExcludeExtension(TEXT("umap"))
		.ExcludeExtension(TEXT("uregs"));
	Generator.AddFilesFromDirectory(TEXT("/{engine}"), Generator.GetCookedEngineDir(), true, &CookedFilter);
	Generator.AddFilesFromDirectory(TEXT("/{project}"), Generator.GetCookedProjectDir(), true, &CookedFilter);

	FFileFilter CookedMetadataFilter = FFileFilter()
		.ExcludeDirectoryLeafName(TEXT("ShaderLibrarySource"))
		.ExcludeExtension(TEXT("manifest"));
	Generator.AddFilesFromDirectory(TEXT("/{project}"), Generator.GetCookedProjectDir(), MetadataDirectoryPath,
		true, &CookedMetadataFilter);

	if (Generator.IsIncludeProject())
	{
		FFileFilter ProjectSourceFilter = FFileFilter()
			.IncludeExtension(TEXT("uproject"));
		Generator.AddFilesFromDirectory(TEXT("/{project}"), Generator.GetProjectDir(), false, &ProjectSourceFilter);
	}
	
	FFileFilter ConfigFilter = FFileFilter()
		.IncludeExtension(TEXT("ini"));
	Generator.AddFilesFromExtensionDirectories(TEXT("Config"), &ConfigFilter);

	FFileFilter LocalizationFilter = FFileFilter()
		.IncludeExtension(TEXT("locmeta"))
		.IncludeExtension(TEXT("locres"));

	FFileFilter PluginFilter = FFileFilter()
		.IncludeExtension(TEXT("uplugin"));

	for (TSharedRef<IPlugin>& Plugin : Generator.GetApplicablePlugins())
	{
		const FString& PluginName = Plugin->GetName();
		FString BaseDir = Plugin->GetBaseDir();
		const FString& UPluginFile = Plugin->GetDescriptorFileName();
		FString ContentDir = Plugin->GetContentDir();
		FString LocalizationDir = ContentDir / TEXT("Localization");
		FString ConfigDir = BaseDir / TEXT("Config");
		UE_LOGF(LogZenFileSystemManifest, Verbose, "Plugin '%ls': BaseDir: '%ls'", *PluginName, *BaseDir);

		FString ClientRoot;
		FString LocalRoot;
		FString CookedDir;
		FString CookedClientRoot;
		FString CookedLocalRoot;
		Generator.GetPluginClientAndLocalRoots(Plugin, ClientRoot, LocalRoot);

		Generator.AddSingleFile(ClientRoot, LocalRoot, UPluginFile);
		Generator.AddFilesFromDirectory(ClientRoot, LocalRoot, LocalizationDir, true, &LocalizationFilter);
		if (Generator.TryGetCookedPathOfSourceDomainPath(LocalizationDir, CookedDir, CookedClientRoot, CookedLocalRoot))
		{
			Generator.AddFilesFromDirectory(CookedClientRoot, CookedLocalRoot, CookedDir, true, &LocalizationFilter);
		}
		Generator.AddFilesFromDirectory(ClientRoot, LocalRoot, ConfigDir, true, &ConfigFilter);

		// Next add any valid plugin extension directories of this plugin.
		for (const FString& ExtensionBaseDir : Generator.GetApplicablePluginExtensionBaseDirs(Plugin))
		{
			FString ExtensionLocalizationDir = ExtensionBaseDir / TEXT("Content") / TEXT("Localization");
			FString ExtensionConfigDir = ExtensionBaseDir / TEXT("Config");
			UE_LOGF(LogZenFileSystemManifest, Verbose, "Plugin '%ls': ExtensionBaseDir: '%ls'", *PluginName, *ExtensionBaseDir);

			Generator.AddFilesFromDirectory(ClientRoot, LocalRoot, ExtensionBaseDir, false, &PluginFilter);
			Generator.AddFilesFromDirectory(ClientRoot, LocalRoot, ExtensionLocalizationDir, true, &LocalizationFilter);
			if (Generator.TryGetCookedPathOfSourceDomainPath(ExtensionLocalizationDir, CookedDir, CookedClientRoot, CookedLocalRoot))
			{
				Generator.AddFilesFromDirectory(CookedClientRoot, CookedLocalRoot, CookedDir, true, &LocalizationFilter);
			}
			Generator.AddFilesFromDirectory(ClientRoot, LocalRoot, ExtensionConfigDir, true, &ConfigFilter);
		}
	}

	if (Generator.IsIncludeEngine())
	{
		FString InternationalizationPresetPath = Generator.GetInternationalizationPresetPath();
		// Source the ICU data folder name from the cook target's DataDrivenPlatformInfo.ini 
		const FDataDrivenPlatformInfo& TargetDDPI = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(TargetPlatform.IniPlatformName());
		const FString ICUDataVersion = !TargetDDPI.ICUDataVersion.IsEmpty()
			? TargetDDPI.ICUDataVersion
			: FString(TEXT("icudt64l")); // Fallback during the migration period for platforms that have not yet declared ICUDataVersion.
		Generator.AddFilesFromDirectory(
			FPaths::Combine(TEXT("/{engine}"),
				TEXT("Content"), TEXT("Internationalization"), ICUDataVersion),
			FPaths::Combine(Generator.GetEngineDir(),
				TEXT("Content"), TEXT("Internationalization"), InternationalizationPresetPath, ICUDataVersion),
			true /* bRecursive */);
	}
	Generator.AddFilesFromExtensionDirectories(TEXT("Content/Localization"), &LocalizationFilter);

	if (Generator.IsIncludeEngine() || Generator.IsIncludeProject())
	{
		if (Generator.IsSSLCertificatesWillStage())
		{
			if (!Generator.TryAddSingleRelpath(TEXT("/{project}"), Generator.GetProjectDir(),
				FPaths::Combine(TEXT("Content"), TEXT("Certificates"), TEXT("cacert.pem"))))
			{
				Generator.TryAddSingleRelpath(TEXT("/{engine}"), Generator.GetEngineDir(),
					FPaths::Combine(TEXT("Content"), TEXT("Certificates"), TEXT("ThirdParty"), TEXT("cacert.pem")));
			}

			FFileFilter CertificateFilter = FFileFilter()
				.IncludeExtension(TEXT("pem"));
			Generator.AddFilesFromDirectory(TEXT("/{project}/Certificates"),
				FPaths::Combine(Generator.GetProjectDir(), TEXT("Certificates")), true, &CertificateFilter);
		}
	}

	FFileFilter ContentFilter = FFileFilter()
		.ExcludeExtension(TEXT("uasset"))
		.ExcludeExtension(TEXT("ubulk"))
		.ExcludeExtension(TEXT("uexp"))
		.ExcludeExtension(TEXT("umap"));
	if (Generator.IsIncludeEngine())
	{
		Generator.AddFilesFromDirectory(TEXT("/{engine}/Content/Slate"),
			FPaths::Combine(Generator.GetEngineDir(), TEXT("Content"), TEXT("Slate")),
			true, &ContentFilter);
		Generator.AddFilesFromDirectory(TEXT("/{engine}/Content/Movies"),
			FPaths::Combine(Generator.GetEngineDir(), TEXT("Content"), TEXT("Movies")),
			true, &ContentFilter);
	}
	if (Generator.IsIncludeProject())
	{
		Generator.AddFilesFromDirectory(TEXT("/{project}/Content/Slate"),
			FPaths::Combine(Generator.GetProjectDir(), TEXT("Content"), TEXT("Slate")),
			true, &ContentFilter);
		Generator.AddFilesFromDirectory(TEXT("/{project}/Content/Movies"),
			FPaths::Combine(Generator.GetProjectDir(), TEXT("Content"), TEXT("Movies")),
			true, &ContentFilter);
	}
	
	if (Generator.IsIncludeProject())
	{
		FFileFilter OoodleDictionaryFilter = FFileFilter()
			.IncludeExtension(TEXT("udic"));
		Generator.AddFilesFromDirectory(TEXT("/{project}/Content/Oodle"),
			FPaths::Combine(Generator.GetProjectDir(), TEXT("Content"), TEXT("Oodle")),
			false, &OoodleDictionaryFilter);
	}

	FFileFilter ShaderCacheFilter = FFileFilter()
		.IncludeExtension(TEXT("ushadercache"))
		.IncludeExtension(TEXT("upipelinecache"));
	if (Generator.IsIncludeProject())
	{
		Generator.AddFilesFromDirectory(TEXT("/{project}/Content"),
			FPaths::Combine(Generator.GetProjectDir(), TEXT("Content")),
			false, &ShaderCacheFilter);
		Generator.AddFilesFromDirectory(FPaths::Combine(TEXT("/{project}"), TEXT("Content"), TEXT("PipelineCaches"), TargetPlatform.IniPlatformName()),
			FPaths::Combine(Generator.GetProjectDir(), TEXT("Content"), TEXT("PipelineCaches"), TargetPlatform.IniPlatformName()),
			false, &ShaderCacheFilter);
	}

	if (Generator.IsIncludeProject())
	{
		for (const FDirectoryPath& AdditionalFolderToStage : PackagingSettings->DirectoriesToAlwaysStageAsUFS)
		{
			Generator.TryAddFilesFromFlexDirectory(AdditionalFolderToStage.Path, true, &ContentFilter);
		}
		for (const FDirectoryPath& AdditionalFolderToStage : PackagingSettings->DirectoriesToAlwaysStageAsUFSServer)
		{
			Generator.TryAddFilesFromFlexDirectory(AdditionalFolderToStage.Path, true, &ContentFilter);
		}
	}

	const int32 CurrentEntryCount = NumEntries();
	return CurrentEntryCount - PreviousEntryCount;
}

const FZenFileSystemManifestEntry& FZenFileSystemManifest::CreateManifestEntry(const FString& Filename)
{
	const FString FullFilename = FPaths::ConvertRelativePathToFull(Filename);

	FString CookedEngineDirectory = FPaths::Combine(CookDirectory, TEXT("Engine"));
	FString CookedEngineDirectoryTrailingSeparator;
	CookedEngineDirectoryTrailingSeparator.Reserve(CookedEngineDirectory.Len() + 1);
	CookedEngineDirectoryTrailingSeparator.Append(CookedEngineDirectory);
	CookedEngineDirectoryTrailingSeparator.AppendChar(TEXT('/'));

	auto AddEntry = [this, &FullFilename](const FString& ClientDirectory, const FString& LocalDirectory)
		-> const FZenFileSystemManifestEntry&
	{
		FStringView RelativePath = FullFilename;
		RelativePath.RightChopInline(LocalDirectory.Len() + 1);

		FString ServerRelativeDirectory = LocalDirectory;
		FPaths::MakePathRelativeTo(ServerRelativeDirectory, *FPaths::RootDir());
		ServerRelativeDirectory = TEXT("/") + ServerRelativeDirectory;

		FString ServerPath = FPaths::Combine(ServerRelativeDirectory, RelativePath.GetData());
		FString ClientPath = FPaths::Combine(ClientDirectory, RelativePath.GetData());
		const FIoChunkId FileChunkId = CreateExternalFileChunkId(ClientPath);

		return AddManifestEntry(FileChunkId, MoveTemp(ServerPath), MoveTemp(ClientPath));
	};

	if (FullFilename.StartsWith(CookedEngineDirectoryTrailingSeparator))
	{
		return AddEntry(TEXT("/{engine}"), CookedEngineDirectory);
	}

	FString CookedProjectDirectory = FPaths::Combine(CookDirectory, FApp::GetProjectName());
	FString CookedProjectDirectoryTrailingSeparator;
	CookedProjectDirectoryTrailingSeparator.Reserve(CookedProjectDirectory.Len() + 1);
	CookedProjectDirectoryTrailingSeparator.Append(CookedProjectDirectory);
	CookedProjectDirectoryTrailingSeparator.AppendChar(TEXT('/'));

	if (FullFilename.StartsWith(CookedProjectDirectoryTrailingSeparator))
	{
		return AddEntry(TEXT("/{project}"), CookedProjectDirectory);
	}

	return InvalidEntry;
}

const FZenFileSystemManifestEntry& FZenFileSystemManifest::AddManifestEntry(const FIoChunkId& FileChunkId,
	FString ServerPath, FString ClientPath)
{
	check(ServerPath.Len() > 0 && ClientPath.Len() > 0);

	ServerPath.ReplaceInline(TEXT("\\"), TEXT("/"));
	ClientPath.ReplaceInline(TEXT("\\"), TEXT("/"));

	// The server path is always relative to project root
	if (ServerPath[0] == '/')
	{
		ServerPath.RightChopInline(1);
	}

	int32& EntryIndex = ServerPathToEntry.FindOrAdd(ServerPath, INDEX_NONE);

	if (EntryIndex != INDEX_NONE)
	{
		return Entries[EntryIndex];
	}

	EntryIndex = Entries.Num();

	FZenFileSystemManifestEntry Entry;
	Entry.ServerPath = MoveTemp(ServerPath);
	Entry.ClientPath = MoveTemp(ClientPath);
	Entry.FileChunkId = FileChunkId;

#if WITH_EDITOR
	if (!ReferencedSetClientPath.IsEmpty() && Entry.ClientPath == ReferencedSetClientPath)
	{
		ReferencedSet.Emplace(MoveTemp(Entry));
		return *ReferencedSet;
	}
	else
#endif
	{
		Entries.Add(MoveTemp(Entry));
		return Entries[EntryIndex];
	}
}

bool FZenFileSystemManifest::Save(const TCHAR* Filename)
{
	check(Filename);

	TArray<FString> CsvLines;
	CsvLines.Add(FString::Printf(TEXT(";ServerRoot=%s, Platform=%s, CookDirectory=%s"), *ServerRoot, *TargetPlatform.PlatformName(), *CookDirectory));
	CsvLines.Add(TEXT("FileId, ServerPath, ClientPath"));

	TStringBuilder<2048> Sb;
	TArray<const FZenFileSystemManifestEntry*> SortedEntries;
	SortedEntries.Reserve(Entries.Num());
	for (const FZenFileSystemManifestEntry& Entry : Entries)
	{
		SortedEntries.Add(&Entry);
	}
	Algo::Sort(SortedEntries, [](const FZenFileSystemManifestEntry* A, const FZenFileSystemManifestEntry* B) { return A->ClientPath < B->ClientPath; });

	for (const FZenFileSystemManifestEntry* Entry : SortedEntries)
	{
		Sb.Reset();
		Sb << Entry->FileChunkId << TEXT(", ") << Entry->ServerPath << TEXT(", ") << *Entry->ClientPath;
		CsvLines.Add(Sb.ToString());
	}

	return FFileHelper::SaveStringArrayToFile(CsvLines, Filename);
}


namespace UE::ZenFileSystemManifest
{

FManifestGenerator::FManifestGenerator(FZenFileSystemManifest& InManifest,
	const TOptional<ICookedPackageWriter::FReferencedPluginsInfo>& InReferencedPlugins)
	: Manifest(InManifest)
	, ReferencedPlugins(InReferencedPlugins)
	, PlatformFile(FPlatformFileManager::Get().GetPlatformFile())
{
	UnrealEngineRootDir = FPaths::RootDir();
	EngineDir = FPaths::EngineDir();
	FPaths::NormalizeDirectoryName(EngineDir);
	ProjectDir = FPaths::ProjectDir();
	FPaths::NormalizeDirectoryName(ProjectDir);
	ProjectName = FApp::GetProjectName();

	PopulatePlatformDirectoryNames();
}

const FString& FManifestGenerator::GetUnrealEngineRootDir() const
{
	return UnrealEngineRootDir;
}

const FString& FManifestGenerator::GetEngineDir() const
{
	return EngineDir;
}

const FString& FManifestGenerator::GetProjectDir() const
{
	return ProjectDir;
}

const FString& FManifestGenerator::GetProjectName() const
{
	return ProjectName;
}

FString FManifestGenerator::GetCookedEngineDir() const
{
	return FPaths::Combine(Manifest.CookDirectory, TEXT("Engine"));
}

FString FManifestGenerator::GetCookedProjectDir() const
{
	return FPaths::Combine(Manifest.CookDirectory, GetProjectName());
}

FString FManifestGenerator::GetCookedMetadataDir() const
{
	return FPaths::Combine(Manifest.CookDirectory, GetProjectName(), TEXT("Metadata"));
}

const FString& FManifestGenerator::GetUBTPlatformString() const
{
	return UBTPlatformString;
}

FFileFilter& FManifestGenerator::GetBaseFilter()
{
	return BaseFilter;
}

FString FManifestGenerator::ConvertLocalPathToClientPath(const FString& ClientRoot, const FString& LocalRoot, FStringView LocalFilePath)
{
	FString ClientPath(LocalFilePath);
	// MakePathRelative calls GetPath (aka the parent path) on InRelativeTo argument before using it to look for the
	// common prefix. Since we want our root paths to be used directly, rather than their parent path, add a
	// terminating / to our root paths when passing them into MakePathRelativeTo; a terminating slash causes GetPath
	// to return the path before the terminating slash.
	FPaths::MakePathRelativeTo(ClientPath, *(LocalRoot / TEXT("")));
	ClientPath = FPaths::Combine(ClientRoot, ClientPath);
	return ClientPath;
}

void FManifestGenerator::AddSingleFile(const FString& ClientRoot, const FString& LocalRoot, FStringView LocalFilePath)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(AddManifestEntry);
	FString ClientPath(ConvertLocalPathToClientPath(ClientRoot, LocalRoot, LocalFilePath));

	FString ServerRelativePath(LocalFilePath);

	// MakePathRelative returns false if Target and Source are on different drives. If that's the case 
	// we use the full path. Otherwise, we make the path relative to the root dir. 
	// TODO: Absolute paths are not valid for use in paths on the zenserver , because they are supposed to be 
	// transferable. We will need to add an ExternalProjectRoot token to ZenServer paths, which will require 
	// some changes in zenserver to support.
	if (FPaths::MakePathRelativeTo(ServerRelativePath, *(UnrealEngineRootDir / TEXT(""))))
	{
		ServerRelativePath = FString(TEXT("/")) + ServerRelativePath;
	}

	const FIoChunkId FileChunkId = CreateExternalFileChunkId(ClientPath);
	Manifest.AddManifestEntry(
		FileChunkId,
		MoveTemp(ServerRelativePath),
		MoveTemp(ClientPath));
}

bool FManifestGenerator::TryAddSingleRelpath(const FString& ClientRoot, const FString& LocalRoot, const FString& RelPathFromRoot)
{
	FString LocalFilePath = FPaths::Combine(LocalRoot, RelPathFromRoot);
	if (!FPaths::FileExists(LocalFilePath))
	{
		return false;
	}
	AddSingleFile(ClientRoot, LocalRoot, LocalFilePath);
	return true;
}

void FManifestGenerator::AddFilesFromDirectory(const FString& ClientRoot, const FString& LocalRoot,
	bool bIncludeSubDirs, FFileFilter* AdditionalFilter)
{
	AddFilesFromDirectory(ClientRoot, LocalRoot, LocalRoot, bIncludeSubDirs, AdditionalFilter);
}

void FManifestGenerator::AddFilesFromDirectory(const FString& ClientRoot, const FString& LocalRoot,
	const FString& LocalAddRoot, bool bIncludeSubDirs, FFileFilter* AdditionalFilter)
{
	FAddFilesFromDirectoryVisitor Visitor(*this, ClientRoot, LocalRoot, bIncludeSubDirs, AdditionalFilter);

	Visitor.DirectoryVisitQueue.Push(LocalAddRoot);
	while (!Visitor.DirectoryVisitQueue.IsEmpty())
	{
		FString DirectoryToVisit = Visitor.DirectoryVisitQueue.Pop(EAllowShrinking::No);
		PlatformFile.IterateDirectory(*DirectoryToVisit,
			[&Visitor](const TCHAR* InnerFileNameOrDirectory, bool bIsDirectory) -> bool
			{
				return Visitor.VisitorFunction(InnerFileNameOrDirectory, bIsDirectory);
			});
	}
};

bool FManifestGenerator::TryAddFilesFromFlexDirectory(const FString& FullOrProjectRelativePath,
	bool bRecursive, FFileFilter* AdditionalFilter)
{
	FString AbsoluteDirToStage = FPaths::ConvertRelativePathToFull(FPaths::Combine(ProjectDir, TEXT("Content"), FullOrProjectRelativePath));
	FPaths::NormalizeDirectoryName(AbsoluteDirToStage);
	FString AbsoluteEngineDir = FPaths::ConvertRelativePathToFull(EngineDir);
	FString AbsoluteProjectDir = FPaths::ConvertRelativePathToFull(ProjectDir);
	FStringView RelativeToKnownRootView;
	if (FPathViews::TryMakeChildPathRelativeTo(AbsoluteDirToStage, AbsoluteProjectDir, RelativeToKnownRootView))
	{
		AddFilesFromDirectory(FPaths::Combine(TEXT("/{project}"), RelativeToKnownRootView.GetData()),
			FPaths::Combine(ProjectDir, RelativeToKnownRootView.GetData()),
			bRecursive, AdditionalFilter);
		return true;
	}
	else if (FPathViews::TryMakeChildPathRelativeTo(AbsoluteDirToStage, AbsoluteEngineDir, RelativeToKnownRootView))
	{
		AddFilesFromDirectory(FPaths::Combine(TEXT("/{engine}"), RelativeToKnownRootView.GetData()),
			FPaths::Combine(EngineDir, RelativeToKnownRootView.GetData()),
			bRecursive, AdditionalFilter);
		return true;
	}
	else
	{
		UE_LOGF(LogZenFileSystemManifest, Warning,
			"Ignoring additional folder to stage that is not relative to the engine or project directory: %ls",
			*FullOrProjectRelativePath);
		return false;
	}
}

void FManifestGenerator::AddFilesFromExtensionDirectories(const TCHAR* ExtensionSubDir, FFileFilter* AdditionalFilter)
{
	TArray<FString> ExtensionDirs;
	if (IsIncludeEngine())
	{
		GetExtensionDirs(ExtensionDirs, *EngineDir, ExtensionSubDir, PlatformDirectoryNames);
		for (const FString& LocalDir : ExtensionDirs)
		{
			if (!LocalDir.StartsWith(EngineDir))
			{
				UE_LOGF(LogZenFileSystemManifest, Warning,
					"GetExtensionDirs on root directory '%ls' unexpectedly returned folder '%ls' that is not a subdirectory of the root directory.",
					*EngineDir, *LocalDir);
				continue;
			}
			FString ClientDir = FPaths::Combine(TEXT("/{engine}"), LocalDir.RightChop(EngineDir.Len()));
			AddFilesFromDirectory(ClientDir, LocalDir, true, AdditionalFilter);
		}
	}
	if (IsIncludeProject())
	{
		ExtensionDirs.Reset();
		GetExtensionDirs(ExtensionDirs, *ProjectDir, ExtensionSubDir, PlatformDirectoryNames);
		for (const FString& LocalDir : ExtensionDirs)
		{
			if (!LocalDir.StartsWith(ProjectDir))
			{
				UE_LOGF(LogZenFileSystemManifest, Warning,
					"GetExtensionDirs on root directory '%ls' unexpectedly returned folder '%ls' that is not a subdirectory of the root directory.",
					*ProjectDir, *LocalDir);
				continue;
			}
			FString ClientDir = FPaths::Combine(TEXT("/{project}"), LocalDir.RightChop(ProjectDir.Len()));
			AddFilesFromDirectory(ClientDir, LocalDir, true, AdditionalFilter);
		}
	}
};

void FManifestGenerator::GetExtensionDirs(TArray<FString>& OutExtensionDirs, const TCHAR* BaseDir, const TCHAR* SubDir,
	const TArray<FString>& PlatformDirectoryNames)
{
	auto AddIfDirectoryExists = [&OutExtensionDirs](FString&& Dir)
	{
		if (FPaths::DirectoryExists(Dir))
		{
			OutExtensionDirs.Emplace(MoveTemp(Dir));
		}
	};

	AddIfDirectoryExists(FPaths::Combine(BaseDir, SubDir));

	FString PlatformExtensionBaseDir = FPaths::Combine(BaseDir, TEXT("Platforms"));
	for (const FString& PlatformDirectoryName : PlatformDirectoryNames)
	{
		AddIfDirectoryExists(FPaths::Combine(PlatformExtensionBaseDir, PlatformDirectoryName, SubDir));
	}

	FString RestrictedBaseDir = FPaths::Combine(BaseDir, TEXT("Restricted"));
	IFileManager::Get().IterateDirectory(*RestrictedBaseDir,
		[&OutExtensionDirs, SubDir, &PlatformDirectoryNames]
		(const TCHAR* FilenameOrDirectory, bool bIsDirectory) -> bool
		{
			if (bIsDirectory)
			{
				GetExtensionDirs(OutExtensionDirs, FilenameOrDirectory, SubDir, PlatformDirectoryNames);
			}
			return true;
		});
}

TArray<TSharedRef<IPlugin>> FManifestGenerator::GetApplicablePlugins()
{
	constexpr bool bSkipDisabledPlugins = false;

	IPluginManager& PluginManager = IPluginManager::Get();
	TArray<TSharedRef<IPlugin>> DiscoveredPlugins = PluginManager.GetDiscoveredPlugins();

	DiscoveredPlugins.RemoveAll([this](TSharedRef<IPlugin>& Plugin)
		{
			const FString& PluginName = Plugin->GetName();
			if (bSkipDisabledPlugins && !Plugin->IsEnabled())
			{
				UE_LOGF(LogZenFileSystemManifest, Verbose, "Skipping plugin '%ls' because it is disabled.", *PluginName);
				return true;
			}
			const FPluginDescriptor& Descriptor = Plugin->GetDescriptor();
			if (!Descriptor.SupportsTargetPlatform(UBTPlatformString))
			{
				UE_LOGF(LogZenFileSystemManifest, Verbose, "Skipping plugin '%ls' because it is not supported on platform '%ls'.",
					*PluginName, *Manifest.TargetPlatform.PlatformName());
				for (const FString& SupportedTargetPlatform : Descriptor.SupportedTargetPlatforms)
				{
					UE_LOGF(LogZenFileSystemManifest, Verbose, "       '%ls' supports platform '%ls'",
						*PluginName, *SupportedTargetPlatform);
				}
				return true;
			}

			if (ReferencedPlugins.IsSet())
			{
				// Skip adding files to the manifest that are under plugins that were not cooked in the current BuildLayer.

				if (!Plugin->CanContainContent())
				{
					// Code-only plugins would not be found by looking for cooked packages, so do not filter them out based
					// on their presence in the ReferencedPlugins. Instead, add all code-only plugins from game or engine that
					// are enabled on the target, but only if we are cooking game or engine.
					if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Engine && !ReferencedPlugins->bReferencesEngine)
					{
						UE_LOGF(LogZenFileSystemManifest, Verbose,
							"Skipping plugin '%ls' because it is a code-only Engine plugin, and we did not cook /Engine.",
							*PluginName);
						return true;
					}
					if (Plugin->GetLoadedFrom() == EPluginLoadedFrom::Project && !ReferencedPlugins->bReferencesGame)
					{
						UE_LOGF(LogZenFileSystemManifest, Verbose,
							"Skipping plugin '%ls' because it is a code-only Project plugin, and we did not cook /Game.",
							*PluginName);
						return true;
					}
				}
				else
				{
					if (!ReferencedPlugins->ReferencedPlugins.Contains(Plugin->GetName()))
					{
						UE_LOGF(LogZenFileSystemManifest, Verbose,
							"Skipping plugin '%ls' because we did not cook any content from it.",
							*PluginName);
						return true;
					}
				}
			}

			return false;
		});
	return DiscoveredPlugins;
}

void FManifestGenerator::GetPluginClientAndLocalRoots(TSharedRef<IPlugin>& Plugin,
	FString& OutClientRoot, FString& OutLocalRoot)
{
	switch (Plugin->GetLoadedFrom())
	{
	case EPluginLoadedFrom::Engine:
		OutClientRoot = TEXT("/{engine}");
		OutLocalRoot = EngineDir;
		break;
	case EPluginLoadedFrom::Project:
		OutClientRoot = TEXT("/{project}");
		OutLocalRoot = ProjectDir;
		break;
	default:
		checkNoEntry();
		break;
	}
}

bool FManifestGenerator::TryGetCookedPathOfSourceDomainPath(const FString& SourceDomainPath, FString& OutCookedPath, 
	FString& OutClientRoot, FString& OutLocalRoot)
{
#if !WITH_EDITOR
	return false;
#else
	UE::Cook::ICookSandbox* CookSandbox = Manifest.CookSandbox;
	if (!CookSandbox)
	{
		return false;
	}
	const ITargetPlatform& TargetPlatform = Manifest.TargetPlatform;
	
	FString FullPath = FPaths::ConvertRelativePathToFull(SourceDomainPath);
	FString CookedPath = CookSandbox->ConvertToFullPlatformSandboxPath(FullPath, true /* bForWrite */, TargetPlatform.PlatformName());
	if (FullPath.Equals(CookedPath, ESearchCase::IgnoreCase))
	{
		return false;
	}

	FString CookedEngineRoot = GetCookedEngineDir();
	if (FPathViews::IsParentPathOf(CookedEngineRoot, CookedPath))
	{
		OutCookedPath = MoveTemp(CookedPath);
		OutClientRoot = TEXT("/{engine}");
		OutLocalRoot = MoveTemp(CookedEngineRoot);
		return true;
	}
	FString CookedProjectRoot = GetCookedProjectDir();
	if (FPathViews::IsParentPathOf(CookedProjectRoot, CookedPath))
	{
		OutCookedPath = MoveTemp(CookedPath);
		OutClientRoot = TEXT("/{project}");
		OutLocalRoot = MoveTemp(CookedProjectRoot);
		return true;
	}

	return false;
#endif
}

TArray<FString> FManifestGenerator::GetApplicablePluginExtensionBaseDirs(TSharedRef<IPlugin>& Plugin)
{
	TArray<FString> ExtensionBaseDirs = Plugin->GetExtensionBaseDirs();
	ExtensionBaseDirs.RemoveAll([this](const FString& ExtensionBaseDir)
		{
			// Scan the extension path for "Platforms/X" and include this extension if it is not platform specific at all,
			// or if X is found and it is a valid target platform
			bool bFoundPlatformsComponent = false;
			bool bDone = false;
			bool bIncludeExtension = true;
			FPathViews::IterateComponents(
				ExtensionBaseDir,
				[this, &bFoundPlatformsComponent, &bDone, &bIncludeExtension](FStringView CurrentPathComponent)
				{
					if (!bFoundPlatformsComponent)
					{
						if (CurrentPathComponent == TEXTVIEW("Platforms"))
						{
							bFoundPlatformsComponent = true;
						}
					}
					else if (!bDone)
					{
						uint32 CurrentPathComponentHash = GetTypeHash(CurrentPathComponent);
						const bool bIsValidPlatform = PlatformDirectoryNameSet.ContainsByHash(CurrentPathComponentHash, CurrentPathComponent);
						bIncludeExtension = bIsValidPlatform;
						bDone = true;
					}
					else
					{
						// Do nothing.
					}
				});
			return !bIncludeExtension;
		});
	return ExtensionBaseDirs;
}

FString FManifestGenerator::GetInternationalizationPresetPath()
{
	const UProjectPackagingSettings* const PackagingSettings = GetDefault<UProjectPackagingSettings>();

	FString InternationalizationPresetAsString = UEnum::GetValueAsString(PackagingSettings->InternationalizationPreset);
	const TCHAR* InternationalizationPresetPath = FCString::Strrchr(*InternationalizationPresetAsString, ':');
	if (InternationalizationPresetPath)
	{
		++InternationalizationPresetPath;
	}
	else
	{
		UE_LOGF(LogZenFileSystemManifest, Warning, "Failed reading internationalization preset setting, defaulting to English");
		InternationalizationPresetPath = TEXT("English");
	}
	return FString(InternationalizationPresetPath);
}

bool FManifestGenerator::IsSSLCertificatesWillStage()
{
	bool bSSLCertificatesWillStage = false;
	FConfigCacheIni* TargetPlatformConfig = Manifest.TargetPlatform.GetConfigSystem();
	if (TargetPlatformConfig)
	{
		GConfig->GetBool(TEXT("/Script/Engine.NetworkSettings"), TEXT("n.VerifyPeer"), bSSLCertificatesWillStage, GEngineIni);
	}
	return bSSLCertificatesWillStage;
}

bool FManifestGenerator::IsIncludeEngine()
{
	if (!ReferencedPlugins.IsSet())
	{
		return true;
	}

	return ReferencedPlugins->bReferencesEngine;
}

bool FManifestGenerator::IsIncludeProject()
{
	if (!ReferencedPlugins.IsSet())
	{
		return true;
	}
	return ReferencedPlugins->bReferencesGame;
}

void FManifestGenerator::PopulatePlatformDirectoryNames()
{
	FString IniPlatformName = Manifest.TargetPlatform.IniPlatformName();
	const FDataDrivenPlatformInfo& PlatformInfo = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(IniPlatformName);
	PlatformDirectoryNameSet.Reserve(PlatformInfo.IniParentChain.Num()
		+ PlatformInfo.AdditionalRestrictedFolders.Num() + 1);

	PlatformDirectoryNameSet.Add(IniPlatformName);
	for (const FString& PlatformName : PlatformInfo.AdditionalRestrictedFolders)
	{
		PlatformDirectoryNameSet.Add(PlatformName);
	}
	for (const FString& PlatformName : PlatformInfo.IniParentChain)
	{
		PlatformDirectoryNameSet.Add(PlatformName);
	}

	PlatformDirectoryNames = PlatformDirectoryNameSet.Array();

	UBTPlatformString = PlatformInfo.UBTPlatformString;
}

FFileFilter& FFileFilter::ExcludeDirectoryLeafName(const TCHAR* LeafName)
{
	DirectoryExclusionFilter.Emplace(LeafName);
	return *this;
}

FFileFilter& FFileFilter::ExcludeExtension(const TCHAR* Extension)
{
	ExtensionExclusionFilter.Emplace(Extension);
	return *this;
}

FFileFilter& FFileFilter::IncludeExtension(const TCHAR* Extension)
{
	ExtensionInclusionFilter.Emplace(Extension);
	return *this;
}

bool FFileFilter::IsDirectoryLeafNameKept(FStringView LeafName) const
{
	for (const FString& ExcludedDirectory : DirectoryExclusionFilter)
	{
		if (LeafName == ExcludedDirectory)
		{
			return false;
		}
	}
	return true;
}

bool FFileFilter::IsFileExtensionKept(FStringView Extension)
{
	if (!ExtensionExclusionFilter.IsEmpty())
	{
		for (const FString& ExcludedExtension : ExtensionExclusionFilter)
		{
			if (Extension == ExcludedExtension)
			{
				return false;
			}
		}
	}
	if (!ExtensionInclusionFilter.IsEmpty())
	{
		for (const FString& IncludedExtension : ExtensionInclusionFilter)
		{
			if (Extension == IncludedExtension)
			{
				return true;
			}
		}
		return false;
	}
	return true;
}

FAddFilesFromDirectoryVisitor::FAddFilesFromDirectoryVisitor(
	FManifestGenerator& InGenerator, const FString& InClientRoot, const FString& InLocalRoot, bool bInIncludeSubDirs,
	FFileFilter* InAdditionalFilter)
	: Generator(InGenerator), ClientRoot(InClientRoot), LocalRoot(InLocalRoot)
	, AdditionalFilter(InAdditionalFilter), bIncludeSubDirs(bInIncludeSubDirs)
{
}

bool FAddFilesFromDirectoryVisitor::VisitorFunction(const TCHAR* InnerFileNameOrDirectory, bool bIsDirectory)
{
	if (bIsDirectory)
	{
		if (!bIncludeSubDirs)
		{
			return true;
		}
		FStringView DirectoryName = FPathViews::GetPathLeaf(InnerFileNameOrDirectory);
		if (!Generator.BaseFilter.IsDirectoryLeafNameKept(DirectoryName))
		{
			return true;
		}
		if (AdditionalFilter && !AdditionalFilter->IsDirectoryLeafNameKept(DirectoryName))
		{
			return true;
		}
		DirectoryVisitQueue.Add(InnerFileNameOrDirectory);
		return true;
	}
	FStringView Extension = FPathViews::GetExtension(InnerFileNameOrDirectory);
	if (!Generator.BaseFilter.IsFileExtensionKept(Extension))
	{
		return true;
	}
	if (AdditionalFilter && !AdditionalFilter->IsFileExtensionKept(Extension))
	{
		return true;
	}
	Generator.AddSingleFile(ClientRoot, LocalRoot, InnerFileNameOrDirectory);
	return true;
}

} // namespace UE::ZenFileSystemManifest
