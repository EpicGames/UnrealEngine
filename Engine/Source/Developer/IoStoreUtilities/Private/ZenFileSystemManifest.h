// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "IO/IoDispatcher.h"
#include "Misc/Optional.h"
#include "Serialization/PackageWriter.h"

class ITargetPlatform;
class FArchive;
namespace UE::ZenFileSystemManifest { class FFileFilter; }
namespace UE::ZenFileSystemManifest { class FManifestGenerator; }
#if WITH_EDITOR
namespace UE::Cook { class ICookSandbox; }
#endif

struct FZenFileSystemManifestEntry
{
	FString ServerPath;
	FString ClientPath;
	FIoChunkId FileChunkId;
};

class FZenFileSystemManifest
{
public:
	FZenFileSystemManifest(const ITargetPlatform& InTargetPlatform, FString InCookDirectory);
	
#if WITH_EDITOR
	void ConstructReferencedSetClientPath(const FString& MetadataDirectoryPath);
	void SetCookSandbox(UE::Cook::ICookSandbox* InCookSandbox);
#endif

	/**
	 * Scan the local Cooked folder for files on disk, create and append Entries for any files not already
	 * present in the manifest. Files in the manifest do not contain any payload data, so no update is necessary
	 * for Entries that already exist. Returns the number of new entries.
	 */
	int32 Generate(const FString& MetadataDirectoryPath,
		const TOptional<ICookedPackageWriter::FReferencedPluginsInfo>& ReferencedPlugins
			= TOptional<ICookedPackageWriter::FReferencedPluginsInfo>());

	const FZenFileSystemManifestEntry& CreateManifestEntry(const FString& Filename);
	
	const FZenFileSystemManifestEntry& AddManifestEntry(const FIoChunkId& FileChunkId, FString ServerPath, FString ClientPath);

	TArrayView<const FZenFileSystemManifestEntry> ManifestEntries() const
	{
		return Entries;
	}

	bool Save(const TCHAR* Filename);

	int32 NumEntries() const
	{
		return Entries.Num();
	}

	const FString& ServerRootPath() const
	{
		return ServerRoot;
	}

	/**
	 * Return the ManifestEntry that was found for the ReferencedSet.txt artifact left by the cooker in the Metadata
	 * directory. The ReferencedSet reports which packages in the oplog were cooked in the most recent incremental
	 * cook. ZenServer needs to read it, so we report it separately to ZenServer rather than reporting it with the
	 * rest of the cook artifacts reported in the EndCook op.
	 */
	TOptional<FZenFileSystemManifestEntry> GetReferencedSet() const
	{
		return ReferencedSet;
	}

private:
	const ITargetPlatform& TargetPlatform;
	FString CookDirectory;
	FString ServerRoot;
	TMap<FString, int32> ServerPathToEntry;
	TArray<FZenFileSystemManifestEntry> Entries;
	TOptional<FZenFileSystemManifestEntry> ReferencedSet;
#if WITH_EDITOR
	FString ReferencedSetClientPath;
	UE::Cook::ICookSandbox* CookSandbox = nullptr;
#endif

	static const FZenFileSystemManifestEntry InvalidEntry;

	friend UE::ZenFileSystemManifest::FFileFilter;
	friend UE::ZenFileSystemManifest::FManifestGenerator;
};
