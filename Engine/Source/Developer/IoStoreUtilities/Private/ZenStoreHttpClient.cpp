// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenStoreHttpClient.h"

#include "Algo/BinarySearch.h"
#include "Algo/IsSorted.h"
#include "Algo/Sort.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "HAL/FileManager.h"

#if PLATFORM_DESKTOP

#include "Async/Async.h"
#include "HAL/PlatformFileManager.h"
#include "Http/HttpClient.h"
#include "IO/IoHash.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Tasks/Task.h"
#include "ZenAsyncCbPackageReceiver.h"
#include "ZenCbPackageReceiver.h"
#include "ZenSerialization.h"
#include "ZenServerHttp.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenStore, Log, All);

namespace UE
{
namespace Zen::Internal
{
struct FZenHttpClientState
{
	THttpUniquePtr<IHttpConnectionPool> ConnectionPool;
	TUniquePtr<FHttpRequestQueue> RequestQueue;
};
}

const uint32 FZenStoreHttpClient::PoolEntryCount = 32;

std::atomic<uint32> FZenStoreHttpClient::FSaltGenerator::GOpCounter(0);

FZenStoreHttpClient::FSaltGenerator::FSaltGenerator()
	: SaltBase(FGenericPlatformProcess::GetCurrentProcessId() + 0x9e3779b9u)
{
}

FZenStoreHttpClient::FZenStoreHttpClient()
{
	InitCommon();
}

FZenStoreHttpClient::FZenStoreHttpClient(FStringView HostName, uint16 Port)
: ZenService(HostName.IsEmpty() ? FStringView() : *WriteToString<64>(TEXT("http://"), HostName, TEXT(":"), Port))
{
	InitCommon();
}

FZenStoreHttpClient::FZenStoreHttpClient(UE::Zen::FServiceSettings&& InSettings)
: ZenService(MoveTemp(InSettings))
{
	InitCommon();
}

FZenStoreHttpClient::~FZenStoreHttpClient()
{
}

void
FZenStoreHttpClient::InitCommon()
{
	RequestPool = MakeUnique<Zen::FZenHttpRequestPool>(ZenService.GetInstance().GetEndpoint(), PoolEntryCount);
	ClientState = MakeUnique<Zen::Internal::FZenHttpClientState>();

	const uint32 MaxConnections = uint32(FMath::Clamp(FPlatformMisc::NumberOfCoresIncludingHyperthreads(), 8, 64));
	constexpr uint32 RequestPoolSize = 128;
	constexpr uint32 RequestPoolOverflowSize = 128;

	FHttpConnectionPoolParams ConnectionPoolParams;
	ConnectionPoolParams.MaxConnections = MaxConnections;
	ConnectionPoolParams.MinConnections = MaxConnections;
	ClientState->ConnectionPool = IHttpManager::Get().CreateConnectionPool(ConnectionPoolParams);

	FHttpClientParams ClientParams;
	ClientParams.MaxRequests = RequestPoolSize + RequestPoolOverflowSize;
	ClientParams.MinRequests = RequestPoolSize;
	ClientParams.LowSpeedLimit = 1;
	ClientParams.LowSpeedTime = 25;
	ClientParams.bBypassProxy = true;
	ClientState->RequestQueue = MakeUnique<FHttpRequestQueue>(*ClientState->ConnectionPool, ClientParams);
}

bool
FZenStoreHttpClient::TryCreateProject(FStringView InProjectId,
	FStringView InParentProjectId,
	FStringView InOplogId,
	FStringView ServerRoot,
	FStringView EngineRoot,
	FStringView ProjectRoot,
	FStringView ProjectFilePath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_Initialize);

	UE_LOGF(LogZenStore, Display, "Establishing oplog '%ls/%ls'", *FString(InProjectId), *FString(InOplogId));

	// Establish project

	{
		// Create the connection request with no logging of errors; our caller will handle logging the connection error
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

		TStringBuilder<128> ProjectUri;
		ProjectUri << "/prj/" << InProjectId;
		TArray64<uint8> GetBuffer;

		// TODO: how to handle failure here? This is probably the most likely point of failure
		// if the service is not up or not responding

		if (Download(*Request, ProjectUri, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
		{
			UE_LOGF(LogZenStore, Display, "Zen project '%ls' already exists", *FString(InProjectId));
			bConnectionSucceeded = true;
		}
		else
		{
			Request->Reset();

			FCbWriter ProjInfo;
			ProjInfo.BeginObject();
			ProjInfo << "id" << InProjectId;
			if (!InParentProjectId.IsEmpty())
			{
				ProjInfo << "parentid" << InParentProjectId;
			}
			ProjInfo << "root" << ServerRoot;
			ProjInfo << "engine" << EngineRoot;
			ProjInfo << "project" << ProjectRoot;
			ProjInfo << "projectfile" << ProjectFilePath;
			ProjInfo.EndObject();

			if (!Post(*Request, ProjectUri, ProjInfo.Save().AsObject()))
			{
				UE_LOGF(LogZenStore, Warning, "Zen project '%ls' creation FAILED: %ls", *FString(InProjectId), *Request->GetDetailedResultString(ProjectUri));
				bConnectionSucceeded = false;

				// TODO: how to recover / handle this?
			}
			else
			{
				const int ResponseCode = Request->GetResponseCode();
				if (ResponseCode == 201)
				{
					UE_LOGF(LogZenStore, Display, "Zen project '%ls' created", *FString(InProjectId));
					bConnectionSucceeded = true;
				}
				else if (200 <= ResponseCode && ResponseCode < 300)
				{
					UE_LOGF(LogZenStore, Warning, "Zen project '%ls' creation returned success but unexpected result: %ls", *FString(InProjectId), *Request->GetDetailedResultString(ProjectUri));
					bConnectionSucceeded = true;
				}
				else
				{
					UE_LOGF(LogZenStore, Warning, "Zen project '%ls' creation FAILED: %ls", *FString(InProjectId), *Request->GetDetailedResultString(ProjectUri));
					bConnectionSucceeded = false;
				}
			}
		}
	}

	ProjectPath = WriteToString<128>("/prj/", InProjectId);
	OplogPath = WriteToString<128>("/prj/", InProjectId, "/oplog/", InOplogId);
	OplogRpcPath = WriteToAnsiString<128>("/prj/", InProjectId, "/oplog/", InOplogId, "/rpc");
	OplogNewEntryPath = WriteToString<128>("/prj/", InProjectId, "/oplog/", InOplogId, "/new");
	OplogPrepNewEntryPath = WriteToString<128>("/prj/", InProjectId, "/oplog/", InOplogId, "/prep");

	bAllowRead = true;
	bAllowEdit = true;

	return bConnectionSucceeded;
}

bool FZenStoreHttpClient::TryDeleteOplog(FStringView InProjectId, FStringView InOplogId)
{
	if (!IsConnected())
	{
		return false;
	}
	UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

	UE_LOGF(LogZenStore, Display, "Deleting oplog '%ls/%ls' if it exists", *FString(InProjectId), *FString(InOplogId));
	bool Success = Delete(*Request, OplogPath);
	if (!Success && Request->GetResponseCode() != 404)
	{
		UE_LOGF(LogZenStore, Warning, "Deleting oplog FAILED: %ls", *Request->GetDetailedResultString(OplogPath));
	}
	return Success;
}

bool FZenStoreHttpClient::TryCreateOplog(FStringView InProjectId, FStringView InOplogId, FStringView InOplogLifetimeMarkerPath)
{
	if (!IsConnected())
	{
		return false;
	}

	UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

	TArray64<uint8> GetBuffer;
	FCbObjectView OplogInfo;
	if (Download(*Request, OplogPath, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
	{
		UE_LOGF(LogZenStore, Display, "Zen oplog '%ls/%ls' already exists", *FString(InProjectId), *FString(InOplogId));

		OplogInfo = FCbObjectView(GetBuffer.GetData());
	}
	else
	{
		FCbWriter Writer;
		Writer.BeginObject();
		Writer.AddString("gcpath", InOplogLifetimeMarkerPath);
		Writer.EndObject();

		FCbFieldIterator OplogCreateInfo = Writer.Save();

		Request->Reset();

		if (!Post(*Request, OplogPath, OplogCreateInfo.AsObjectView()))
		{
			UE_LOGF(LogZenStore, Warning, "Zen oplog '%ls/%ls' creation FAILED: %ls", *FString(InProjectId), *FString(InOplogId), *Request->GetDetailedResultString(OplogPath));
			// Demote the connection status back to not connected
			bConnectionSucceeded = false;
			return false;
		}
		else
		{
			const int ResponseCode = Request->GetResponseCode();
			if (ResponseCode == 201)
			{
				UE_LOGF(LogZenStore, Display, "Zen oplog '%ls/%ls' created", *FString(InProjectId), *FString(InOplogId));
			}
			else if (200 <= ResponseCode && ResponseCode < 300)
			{
				UE_LOGF(LogZenStore, Warning, "Zen oplog '%ls/%ls' creation returned success but unexpected result: %ls", *FString(InProjectId), *FString(InOplogId), *Request->GetDetailedResultString(OplogPath));
			}
			else
			{
				UE_LOGF(LogZenStore, Warning, "Zen oplog '%ls/%ls' creation FAILED: %ls", *FString(InProjectId), *FString(InOplogId), *Request->GetDetailedResultString(OplogPath));
				// Demote the connection status back to not connected
				bConnectionSucceeded = false;
				return false;
			}
		}

		// Issue another GET to retrieve information

		GetBuffer.Reset();
		Request->Reset();
		if (Download(*Request, OplogPath, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
		{
			OplogInfo = FCbObjectView(GetBuffer.GetData());
		}
		else
		{
			UE_LOGF(LogZenStore, Warning, "Zen oplog '%ls/%ls' get FAILED: %ls", *FString(InProjectId), *FString(InOplogId), *Request->GetDetailedResultString(OplogPath));
			bConnectionSucceeded = false;
			return false;
		}
	}

	TempDirPath = FUTF8ToTCHAR(OplogInfo["tempdir"].AsString());
	return true;
}

void FZenStoreHttpClient::InitializeReadOnly(FStringView InProjectId, FStringView InOplogId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_InitializeReadOnly);

	UE_LOGF(LogZenStore, Display, "Opening oplog %ls / %ls", *FString(InProjectId), *FString(InOplogId));

	// Establish project

	ProjectPath = WriteToString<128>("/prj/", InProjectId);
	{
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

		TArray64<uint8> GetBuffer;

		// TODO: how to handle failure here? This is probably the most likely point of failure
		// if the service is not up or not responding

		if (!Download(*Request, ProjectPath, &GetBuffer, Zen::EContentType::CbObject) || Request->GetResponseCode() != 200)
		{
			UE_LOGF(LogZenStore, Fatal, "Zen project '%ls' get FAILED: %ls", *FString(InProjectId), *Request->GetDetailedResultString(ProjectPath));
			bConnectionSucceeded = false;
		}
		else
		{
			bConnectionSucceeded = true;
		}
	}

	// Establish oplog
	check(IsConnected());

	{
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

		OplogPath = WriteToString<128>("/prj/", InProjectId, "/oplog/", InOplogId);

		TArray64<uint8> GetBuffer;
		if (!Download(*Request, OplogPath, &GetBuffer, Zen::EContentType::CbObject) || Request->GetResponseCode() != 200)
		{
			UE_LOGF(LogZenStore, Fatal, "Zen oplog '%ls/%ls' get FAILED: %ls", *FString(InProjectId), *FString(InOplogId), *Request->GetDetailedResultString(OplogPath));
			bConnectionSucceeded = false;
		}
	}

	bAllowRead = true;
}

bool FZenStoreHttpClient::Download(Zen::FZenHttpRequest& Request, FStringView Uri, TArray64<uint8>* Buffer, Zen::EContentType AcceptType)
{
	Zen::FZenHttpRequest::Result Res = Request.PerformBlockingDownload(Uri, Buffer, AcceptType);
	if (ShouldRetry(Request))
	{
		Request.Reset();
		Res = Request.PerformBlockingDownload(Uri, Buffer, AcceptType);
	}
	return Res == Zen::FZenHttpRequest::Result::Success;
}

bool FZenStoreHttpClient::Post(Zen::FZenHttpRequest& Request, FStringView Uri, FCbObjectView Obj)
{
	Zen::FZenHttpRequest::Result Res = Request.PerformBlockingPost(Uri, Obj);
	if (ShouldRetry(Request))
	{
		Request.Reset();
		Res = Request.PerformBlockingPost(Uri, Obj);
	}
	return Res == Zen::FZenHttpRequest::Result::Success;
}

bool FZenStoreHttpClient::Post(Zen::FZenHttpRequest& Request, FStringView Uri, FMemoryView Payload)
{
	Zen::FZenHttpRequest::Result Res = Request.PerformBlockingPost(Uri, Payload);
	if (ShouldRetry(Request))
	{
		Request.Reset();
		Res = Request.PerformBlockingPost(Uri, Payload);
	}
	return Res == Zen::FZenHttpRequest::Result::Success;
}

void FZenStoreHttpClient::SetRequestUri(IHttpRequest& Request, FAnsiStringView Path)
{
	using namespace Zen;

	check(Path.Len() && Path[0] == '/');

	const FZenServiceEndpoint& Endpoint = ZenService.GetInstance().GetEndpoint();

	TAnsiStringBuilder<128> Uri;
	Uri << Endpoint.GetURL();
	Uri << Path;
	Request.SetUri(Uri);

	if (Endpoint.GetSocketType() != FZenServiceEndpoint::ESocketType::Unix)
	{
		return;
	}

	FStringView UnixSocketPath = Endpoint.GetName();
	const auto& AsAnsi = StringCast<ANSICHAR>(UnixSocketPath.GetData(), UnixSocketPath.Len());
	Request.SetUnixSocketPath(AsAnsi);
}

void FZenStoreHttpClient::Rpc(FAnsiStringView Uri, FCbObject RpcRequestObject, FCbPackage& OutResponse)
{
	THttpUniquePtr<IHttpRequest> Request = ClientState->RequestQueue->CreateRequest({});
	if (!Request)
	{
		return;
	}

	constexpr int MaxAttempts = 2;
	int Attempt = 0;
	bool bRetry = false;
	SetRequestUri(*Request, Uri);
	Request->SetMethod(EHttpMethod::Post);
	Request->AddAcceptType(EHttpMediaType::CbPackage);
	Request->SetContentType(EHttpMediaType::CbObject);
	Request->SetBody(RpcRequestObject.GetBuffer().MakeOwned());
	do
	{
		Zen::FCbPackageReceiver Receiver(OutResponse);
		THttpUniquePtr<IHttpResponse> HttpResponse;
		Request->Send(&Receiver, HttpResponse);
		int StatusCode = HttpResponse->GetStatusCode();

		bRetry = Receiver.ShouldRetry(ZenService.GetInstance(), *HttpResponse, /*UseSoftRecovery*/ false) && (++Attempt < MaxAttempts);
	}
	while (bRetry);
}

void FZenStoreHttpClient::RpcAsync(FAnsiStringView Uri, FCbObject RpcRequestObject, TUniqueFunction<void(const FCbPackage& Response)>&& ResponseCallback)
{
	THttpUniquePtr<IHttpRequest> Request = ClientState->RequestQueue->CreateRequest({});
	if (!Request)
	{
		FCbPackage Response;
		ResponseCallback(Response);
		return;
	}

	SetRequestUri(*Request, Uri);
	Request->SetMethod(EHttpMethod::Post);
	Request->AddAcceptType(EHttpMediaType::CbPackage);
	Request->SetContentType(EHttpMediaType::CbObject);
	Request->SetBody(RpcRequestObject.GetBuffer().MakeOwned());
	// PackageReceiver will delete itself in an async task launched from the completion callback
	Zen::FAsyncCbPackageReceiver* PackageReceiver = new Zen::FAsyncCbPackageReceiver(MoveTemp(Request), ZenService.GetInstance(),
		[ResponseCallback = MoveTemp(ResponseCallback)]
		(Zen::FAsyncCbPackageReceiver* Receiver) mutable
		{
			// Launch a task for the completion function since it can execute arbitrary code.
			Tasks::Launch(TEXT("FZenStoreHttpClient::RpcAsync"), [Receiver, InnerResponseCallback = MoveTemp(ResponseCallback)]
			{
				InnerResponseCallback(Receiver->GetResponsePackage());
				delete Receiver;
			});
		},
		/*MaxAttempts*/ 2,
		/*UseSoftRecovery */ false);
	PackageReceiver->SendAsync();
} //-V773

bool FZenStoreHttpClient::Delete(Zen::FZenHttpRequest& Request, FStringView Uri)
{
	Zen::FZenHttpRequest::Result Res = Request.PerformBlockingDelete(Uri);
	if (ShouldRetry(Request))
	{
		Request.Reset();
		Res = Request.PerformBlockingDelete(Uri);
	}
	return Res == Zen::FZenHttpRequest::Result::Success;
}

bool FZenStoreHttpClient::ShouldRetry(Zen::FZenHttpRequest& Request)
{
	const int ConnectError = 7;
	const int SSLConnectError = 35;
	const int TimeoutError = 28;
	const int SendError = 55;
	const int ReceiveError = 56;

	if ((Request.GetErrorCode() == ConnectError) ||
		(Request.GetErrorCode() == SSLConnectError) ||
		(Request.GetErrorCode() == TimeoutError) ||
		(Request.GetErrorCode() == SendError) ||
		(Request.GetErrorCode() == ReceiveError))
	{
		return TryRecover();
	}
	else if (Request.GetResponseCode() == 404 && Request.GetResponseAsString().Contains("No suitable route found"))
	{
		// This is a special case if we happen to hit the port while zenserver is starting up - it is worth a retry
		return TryRecover();
	}

	return false;
}

bool FZenStoreHttpClient::TryRecover()
{
	Zen::FZenServiceInstance& ZenInstance = ZenService.GetInstance();
	if (ZenInstance.IsServiceRunningLocally())
	{
		return ZenInstance.TryRecovery(Zen::FZenServiceInstance::ERecoveryMode::Hard);
	}
	return true;
}

TIoStatusOr<uint64> FZenStoreHttpClient::AppendOp(FCbPackage OpEntry)
{
	check(bAllowEdit);

	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_AppendOp);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Zen_AppendOp_Async);

		TConstArrayView<FCbAttachment> Attachments = OpEntry.GetAttachments();

		{
			TSet<FIoHash> AttachmentHashes;
			for (const FCbAttachment& Attachment : Attachments)
			{
				AttachmentHashes.Add(Attachment.GetHash());
			}
			TStringBuilder<256> ErrorBuffer;
			OpEntry.GetObject().IterateAttachments([&AttachmentHashes, &ErrorBuffer](FCbFieldView Field)
				{
					const FIoHash Hash = Field.AsAttachment();
					if (!AttachmentHashes.Contains(Hash))
					{
						if (ErrorBuffer.Len() > 0)
						{
							ErrorBuffer << ", ";
						}
						ErrorBuffer << LexToString(Hash);
					}
				});

			if (ErrorBuffer.Len() > 0)
			{
				return TIoStatusOr<uint64>((FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Oplog body references attachments that are not included in the package: ") << ErrorBuffer.ToString()));
			}
		}

		FLargeMemoryWriter SerializedPackage;

		const int32 Salt = SaltGen.Next();
		bool bIsUsingTempFiles = false;

		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		TArray<FString> TempFiles;
		ON_SCOPE_EXIT
		{
			for (const FString& TempFile : TempFiles)
			{
				(void)PlatformFile.DeleteFile(*TempFile);
			}
		};

		if (TempDirPath.IsEmpty())
		{
			// Old-style with all attachments by value

			UE::Zen::OpLog::SaveCbPackage(OpEntry, SerializedPackage);
		}
		else
		{
			// Prep phase

			TSet<FIoHash> NeedChunks;

			if (!Attachments.IsEmpty())
			{
				FCbWriter Writer;
				Writer.BeginObject();
				Writer.BeginArray("have");

				for (const FCbAttachment& Attachment : Attachments)
				{
					Writer.AddHash(Attachment.GetHash());
				}

				Writer.EndArray();
				Writer.EndObject();

				FCbFieldIterator Prep = Writer.Save();

				if (Post(*Request, OplogPrepNewEntryPath, Prep.AsObjectView()) && Request->GetResponseCode() == 200)
				{
					FCbObjectView NeedObject = FCbObjectView(Request->GetResponseBuffer().GetData());

					for (auto& Entry : NeedObject["need"])
					{
						NeedChunks.Add(Entry.AsHash());
					}
				}
				else
				{
					UE_LOGF(LogZenStore, Warning, "Oplog prep FAILED: %ls", *Request->GetDetailedResultString(OplogPrepNewEntryPath));
					UE_LOGF(LogZenStore, Warning, "Unable to check for needed chunks, assuming all are needed");
					for (const FCbAttachment& Attachment : Attachments)
					{
						NeedChunks.Add(Attachment.GetHash());
					}
				}
			}

			// This uses a slight variation for package attachment serialization
			// by writing larger attachments to a file and referencing it in the
			// core object. Small attachments are serialized inline as normal

			FCbWriter Writer;

			FCbObject PackageObj = OpEntry.GetObject();
			const FIoHash PackageObjHash = PackageObj.GetHash();

			Writer.AddObject(PackageObj);
			Writer.AddObjectAttachment(PackageObjHash);

			// Send phase

			const bool bCanUseLocalTempFile = ZenService.GetInstance().IsServiceRunningLocally();

			for (const FCbAttachment& Attachment : Attachments)
			{
				if (!Attachment.IsCompressedBinary())
				{
					return TIoStatusOr<uint64>((FIoStatus)(FIoStatusBuilder(EIoErrorCode::CompressionError) << TEXT("Attachment is not compressed")));
				}

				const FIoHash AttachmentHash = Attachment.GetHash();
				bool bIsSerialized = false;

				if (NeedChunks.Contains(AttachmentHash))
				{
					FSharedBuffer AttachmentData = Attachment.AsCompressedBinary().GetCompressed().ToShared();
					if (bCanUseLocalTempFile && AttachmentData.GetSize() >= StandaloneThresholdBytes)
					{
						// Write to temporary file. To avoid race conditions we derive
						// the file name from a salt value and the attachment hash

						FIoHash AttachmentSpec[] { FIoHash::HashBuffer(&Salt, sizeof Salt), AttachmentHash };
						FIoHash AttachmentId = FIoHash::HashBuffer(MakeMemoryView(AttachmentSpec));

						FString TempFilePath = TempDirPath / LexToString(AttachmentId);

						if (PlatformFile.FileExists(*TempFilePath))
						{
							UE_LOGF(LogZenStore, Warning, "A temp file named '%ls' already exists, taking slow path for large attachment", *TempFilePath);
						}
						else
						{
							if (IFileHandle* FileHandle = PlatformFile.OpenWrite(*TempFilePath))
							{
								bool WriteOK = FileHandle->Write((const uint8*)AttachmentData.GetData(), AttachmentData.GetSize());
								delete FileHandle;
								if (WriteOK)
								{
									Writer.AddHash(AttachmentHash);
									TempFiles.Add(TempFilePath);
									bIsSerialized = true;
									bIsUsingTempFiles = true;
								}
								else
								{
									(void)PlatformFile.DeleteFile(*TempFilePath);
									// Take the slow path if we can't write the payload file in the large attachment directory
									UE_LOGF(LogZenStore, Warning, "Could not write to file '%ls', taking slow path for large attachment", *TempFilePath);
								}
							}
							else
							{
								// Take the slow path if we can't open the payload file in the large attachment directory
								UE_LOGF(LogZenStore, Warning, "Could not create file '%ls', taking slow path for large attachment", *TempFilePath);
							}
						}
					}

					if (!bIsSerialized)
					{
						UE::Zen::OpLog::SaveCbAttachment(Attachment, Writer);
					}
				}
				else
				{
					Writer.AddHash(AttachmentHash);
				}
			}
			Writer.AddNull();

			Writer.Save(SerializedPackage);
		}

		UE_LOG(LogZenStore, Verbose, TEXT("Package size: %" UINT64_FMT), SerializedPackage.TotalSize());

		TStringBuilder<64> NewOpPostUri;
		NewOpPostUri << OplogNewEntryPath;

		if (bIsUsingTempFiles)
		{
			NewOpPostUri << "?salt=" << Salt;
		}

		Request->Reset();

		if (Post(*Request, NewOpPostUri, SerializedPackage.GetView()) && (Request->GetResponseCode() == 200 || Request->GetResponseCode() == 201))
		{
			return TIoStatusOr<uint64>(SerializedPackage.TotalSize());
		}
		else
		{
			return TIoStatusOr<uint64>((FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Append OpLog failed, NewOpLogPath='") << OplogNewEntryPath << TEXT("' ") << Request->GetDetailedResultString(OplogPrepNewEntryPath)));
		}
	}
}

TIoStatusOr<uint64> FZenStoreHttpClient::GetChunkSize(const FIoChunkId& Id)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_GetChunkSize);

	check(bAllowRead);

	UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);
	TStringBuilder<128> ChunkUri;
	ChunkUri << OplogPath << '/' << Id << "/info";
	if (Download(*Request, ChunkUri, nullptr, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
	{
		FCbObjectView ResponseObj = Request->GetResponseAsObject();
		const uint64 ChunkSize = ResponseObj["size"].AsUInt64(0);
		return ChunkSize;
	}
	if (int ResponseCode = Request->GetResponseCode(); ResponseCode != 404)
	{
		UE_LOGF(LogZenStore, Verbose, "Get chunk request FAILED: %ls", *Request->GetDetailedResultString(ChunkUri));
	}
	return FIoStatus(EIoErrorCode::NotFound);
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadChunk(const FIoChunkId& Id, uint64 Offset, uint64 Size)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_ReadChunk);
	TStringBuilder<128> ChunkUri;
	ChunkUri << OplogPath << '/' << Id;
	return ReadOpLogUri(ChunkUri, Offset, Size);
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadChunk(const FIoHash& RawHash, uint64 Offset, uint64 Size)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_ReadChunk);
	TStringBuilder<128> ChunkUri;
	ChunkUri << OplogPath << '/' << RawHash;
	return ReadOpLogUri(ChunkUri, Offset, Size);
}

static void SerializeRequest(FCbWriter& Writer, const FIoChunkId& ChunkId)
{
	Writer << "Oid" << ChunkId;
}

static void SerializeRequest(FCbWriter& Writer, const FIoHash& RawHash)
{
	Writer << "RawHash" << RawHash;
}

static void DeserializeResponse(const FCbObjectView ChunkObject, FIoChunkId& ChunkId)
{
	FCbObjectId ObjectId = ChunkObject[ANSITEXTVIEW("Id")].AsObjectId();
	ChunkId.Set(ObjectId.GetView());
}

static void DeserializeResponse(const FCbObjectView ChunkObject, FIoHash& RawHash)
{
	RawHash = ChunkObject[ANSITEXTVIEW("Id")].AsHash();
}

static const TCHAR* DescribeRequest(const FIoChunkId& ChunkId)
{
	return TEXT("Oid");
}

static const TCHAR* DescribeRequest(const FIoHash& RawHash)
{
	return TEXT("RawHash");
}

template<bool bAsync, typename T>
void FZenStoreHttpClient::ReadChunks_Internal(TConstArrayView<T> Requests, TUniqueFunction<void(const T& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_ReadChunksAsync_Internal);
	FCbWriter Writer;
	Writer.BeginObject();
	Writer << "method" << "getchunks";

	Writer.BeginObject("Request");
	Writer.BeginArray("Chunks");

	TSet<T> InProgressRequests;
	for (const T& Request : Requests)
	{
		InProgressRequests.Add(Request);
	}
	for (const T& Request : InProgressRequests)
	{
		Writer.BeginObject();
		SerializeRequest(Writer, Request);
		Writer.EndObject();
	}

	Writer.EndArray();
	Writer.EndObject();
	Writer.EndObject();

	auto OnGetChunksResponse = [InProgressRequests = MoveTemp(InProgressRequests)]
	(TUniqueFunction<void(const T& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback, const FCbPackage& ResultPackage) mutable
	{
		if (!ResultPackage)
		{
			for (const T& InProgressRequest : InProgressRequests)
			{
				ChunkCallback(InProgressRequest, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::NotFound) << TEXT("Transport/request failure, ")
					<< DescribeRequest(InProgressRequest) << TEXT("=") << *WriteToString<32>(InProgressRequest)));
			}
			return;
		}

		const FCbObjectView ResultObjectView(ResultPackage.GetObject());
		for (FCbFieldView ChunkField : ResultObjectView[ANSITEXTVIEW("Chunks")])
		{
			const FCbObjectView ChunkObject(ChunkField.AsObjectView());
			T Response;
			DeserializeResponse(ChunkObject, Response);

			ON_SCOPE_EXIT
			{
				InProgressRequests.Remove(Response);
			};

			const FIoHash RawHash = ChunkObject[ANSITEXTVIEW("RawHash")].AsHash();
			if (const FCbAttachment* Attachment = ResultPackage.FindAttachment(RawHash))
			{
				if (const FCompressedBuffer& Compressed = Attachment->AsCompressedBinary(); !Compressed.IsNull())
				{
					FIoBuffer Decompressed(Compressed.GetRawSize());
					if (!FCompressedBufferReader(Compressed).TryDecompressTo(Decompressed.GetMutableView()))
					{
						ChunkCallback(Response, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::CompressionError) << TEXT("Failed to decompress, ")
							<< DescribeRequest(Response) << TEXT("=") << *WriteToString<32>(Response)));
					}
					else
					{
						// Completion status - success
						ChunkCallback(Response, Decompressed);
					}
				}
				else if (const FSharedBuffer& Uncompressed = Attachment->AsBinary(); !Uncompressed.IsNull())
				{
					// Completion status - success
					ChunkCallback(Response, FIoBuffer(FIoBuffer::Clone, Uncompressed.GetData(), Uncompressed.GetSize()));
				}
				else
				{
					ChunkCallback(Response, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::ReadError) << TEXT("Unexpected attachment type, ")
						<< DescribeRequest(Response) << TEXT("=") << *WriteToString<32>(Response)));
				}
			}
			else
			{
				ChunkCallback(Response, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::NotFound) << TEXT("Attachment not found, ")
					<< DescribeRequest(Response) << TEXT("=") << *WriteToString<32>(Response)));
			}
		}

		for (const T& InProgressRequest : InProgressRequests)
		{
			ChunkCallback(InProgressRequest, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::NotFound) << TEXT("Missing response, ")
				<< DescribeRequest(InProgressRequest) << TEXT("=") << *WriteToString<32>(InProgressRequest)));
		}
	};

	if constexpr (bAsync)
	{
		RpcAsync(OplogRpcPath, Writer.Save().AsObject(),
			[ChunkCallback = MoveTemp(ChunkCallback), OnGetChunksResponse = MoveTemp(OnGetChunksResponse)]
			(const FCbPackage& Response) mutable
			{
				OnGetChunksResponse(MoveTemp(ChunkCallback), Response);
			});
	}
	else
	{
		FCbPackage ResultPackage;
		Rpc(OplogRpcPath, Writer.Save().AsObject(), ResultPackage);
		OnGetChunksResponse(MoveTemp(ChunkCallback), MoveTemp(ResultPackage));
	}
}

void FZenStoreHttpClient::ReadChunks(TConstArrayView<FIoChunkId> Ids, TUniqueFunction<void(const FIoChunkId& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	return ReadChunks_Internal<false, FIoChunkId>(Ids, MoveTemp(ChunkCallback));
}

void FZenStoreHttpClient::ReadChunks(TConstArrayView<FIoHash> RawHashes, TUniqueFunction<void(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	return ReadChunks_Internal<false, FIoHash>(RawHashes, MoveTemp(ChunkCallback));
}

void FZenStoreHttpClient::ReadChunksAsync(TConstArrayView<FIoChunkId> Ids, TUniqueFunction<void(const FIoChunkId& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	return ReadChunks_Internal<true, FIoChunkId>(Ids, MoveTemp(ChunkCallback));
}

void FZenStoreHttpClient::ReadChunksAsync(TConstArrayView<FIoHash> RawHashes, TUniqueFunction<void(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	return ReadChunks_Internal<true, FIoHash>(RawHashes, MoveTemp(ChunkCallback));
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadOpLogAttachment(FStringView Id)
{
	TStringBuilder<128> ChunkUri;
	ChunkUri << OplogPath << '/' << Id;
	return ReadOpLogUri(ChunkUri);
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadOpLogUri(FStringBuilderBase& ChunkUri, uint64 Offset, uint64 Size)
{
	check(bAllowRead);

	UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);
	TArray64<uint8> GetBuffer;

	bool bHaveQuery = false;

	auto AppendQueryDelimiter = [&bHaveQuery, &ChunkUri]
	{
		if (bHaveQuery)
		{
			ChunkUri.AppendChar(TEXT('&'));
		}
		else
		{
			ChunkUri.AppendChar(TEXT('?'));
			bHaveQuery = true;
		}
	};

	if (Offset)
	{
		AppendQueryDelimiter();
		ChunkUri.Appendf(TEXT("offset=%" UINT64_FMT), Offset);
	}

	if (Size != ~uint64(0))
	{
		AppendQueryDelimiter();
		ChunkUri.Appendf(TEXT("size=%" UINT64_FMT), Size);
	}

	if (Download(*Request, ChunkUri, &GetBuffer, Zen::EContentType::CompressedBinary) && Request->GetResponseCode() == 200)
	{
		if (FCompressedBuffer Compressed = FCompressedBuffer::FromCompressed(FSharedBuffer::MakeView(GetBuffer.GetData(), GetBuffer.Num())))
		{
			uint64 CompressedOffset = 0;
			if (Offset > 0)
			{
				uint64 BlockSize = 0;
				ECompressedBufferCompressor Compressor;
				ECompressedBufferCompressionLevel CompressionLevel;
				if (!Compressed.TryGetCompressParameters(Compressor, CompressionLevel, BlockSize))
				{
					return FIoStatus(EIoErrorCode::CompressionError);
				}

				if (BlockSize > 0)
				{
					CompressedOffset = Offset % BlockSize;
				}
			}

			FIoBuffer Decompressed(Compressed.GetRawSize());
			if (!FCompressedBufferReader(Compressed).TryDecompressTo(Decompressed.GetMutableView(), CompressedOffset))
			{
				return FIoStatus(EIoErrorCode::CompressionError);
			}

			return Decompressed;
		}
		else
		{
			return FIoBuffer(FIoBuffer::Clone, GetBuffer.GetData(), GetBuffer.Num());
		}
	}
	else
	{
		int ResponseCode = Request->GetResponseCode();
		UE_LOGF(LogZenStore, Log, "Get chunk request FAILED: %ls", *Request->GetDetailedResultString(ChunkUri));
		if (ResponseCode == 404)
		{
			return FIoStatus(EIoErrorCode::NotFound);
		}
	}
	return FIoStatus(EIoErrorCode::ReadError);
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetProjectInfo()
{
#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
	return Async(ThreadPool, [this]
	{
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get());

		TArray64<uint8> GetBuffer;
		if (Download(*Request, ProjectPath, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
		{
			FCbObjectView Response(GetBuffer.GetData());
			return TIoStatusOr<FCbObject>(FCbObject::Clone(Response));
		}
		else
		{
			return TIoStatusOr<FCbObject>(FIoStatus(EIoErrorCode::NotFound));
		}
	});
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetOplog(bool bTrimByReferencedSet)
{
#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
	return Async(ThreadPool, [this, bTrimByReferencedSet]
	{
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get());
		
		TStringBuilder<128> Uri;
		Uri << OplogPath << TEXT("/entries");
		Uri << TEXT("?") << TEXT("trim_by_referencedset=") << (bTrimByReferencedSet ? TEXT("true") : TEXT("false"));

		TArray64<uint8> GetBuffer;
		if (Download(*Request, Uri, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
		{
			FCbObjectView Response(GetBuffer.GetData());
			return TIoStatusOr<FCbObject>(FCbObject::Clone(Response));
		}
		else
		{
			return TIoStatusOr<FCbObject>(FIoStatus(EIoErrorCode::NotFound));
		}
	});
}

TFuture<TArray<TIoStatusOr<FCbObject>>> FZenStoreHttpClient::GetOplogEntries(FStringView TargetProject, FStringView TargetOplog, TArray<FString> EntryKeys)
{
	FString TargetOplogPath;
	checkf(TargetProject.IsEmpty() == TargetOplog.IsEmpty(), TEXT("TargetProject and TargetOplog must be either both valid or both empty"));
	if (TargetProject.IsEmpty())
	{
		TargetOplogPath = OplogPath;
	}
	else
	{
		TargetOplogPath = WriteToString<128>("/prj/", TargetProject, "/oplog/", TargetOplog);
	}

#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
	return Async(ThreadPool, [this, TargetOplogPath = MoveTemp(TargetOplogPath), EntryKeys = MoveTemp(EntryKeys)]
		{
			TArray<TIoStatusOr<FCbObject>> Results;
			Results.Reserve(EntryKeys.Num());

			for (const FString& EntryKey : EntryKeys)
			{
				if (EntryKey.IsEmpty())
				{
					Results.Add(TIoStatusOr<FCbObject>(FIoStatus(EIoErrorCode::NotFound)));
					continue;
				}
				UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get());

				TStringBuilder<128> Uri;
				Uri << TargetOplogPath << TEXT("/entries");
				Uri << TEXT("?") << TEXT("opkey=") << *EntryKey;

				TArray64<uint8> GetBuffer;
				if (Download(*Request, Uri, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
				{
					FCbObjectView Response(GetBuffer.GetData());
					Results.Add(TIoStatusOr<FCbObject>(FCbObject::Clone(Response)));
				}
				else
				{
					Results.Add(TIoStatusOr<FCbObject>(FIoStatus(EIoErrorCode::NotFound)));
				}
			}
			return Results;
		});
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetFiles()
{
#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
	return Async(ThreadPool, [this]
	{
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get());
		
		TStringBuilder<128> Uri;
		Uri << OplogPath << "/files";

		TArray64<uint8> GetBuffer;
		if (Download(*Request, Uri, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
		{
			FCbObjectView Response(GetBuffer.GetData());
			return TIoStatusOr<FCbObject>(FCbObject::Clone(Response));
		}
		else
		{
			return TIoStatusOr<FCbObject>(FIoStatus(EIoErrorCode::NotFound));
		}
	});
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetChunkInfos()
{
#if WITH_EDITOR
	EAsyncExecution ThreadPool = EAsyncExecution::LargeThreadPool;
#else
	EAsyncExecution ThreadPool = EAsyncExecution::ThreadPool;
#endif
	return Async(ThreadPool, [this]
	{
		UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get());
		
		TStringBuilder<128> Uri;
		Uri << OplogPath << "/chunkinfos";

		TArray64<uint8> GetBuffer;
		if (Download(*Request, Uri, &GetBuffer, Zen::EContentType::CbObject) && Request->GetResponseCode() == 200)
		{
			FCbObjectView Response(GetBuffer.GetData());
			return TIoStatusOr<FCbObject>(FCbObject::Clone(Response));
		}
		else
		{
			return TIoStatusOr<FCbObject>(FIoStatus(EIoErrorCode::NotFound));
		}
	});
}

void 
FZenStoreHttpClient::StartBuildPass()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_StartBuildPass);

	check(bAllowEdit);
}

TIoStatusOr<uint64>
FZenStoreHttpClient::EndBuildPass(FCbPackage OpEntry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenStoreHttp_EndBuildPass);

	check(bAllowEdit);

	FLargeMemoryWriter SerializedPackage;
	UE::Zen::OpLog::SaveCbPackage(OpEntry, SerializedPackage);

	UE_LOGF(LogZenStore, Verbose, "Package size: %lld", SerializedPackage.TotalSize());

	UE::Zen::FZenScopedRequestPtr Request(RequestPool.Get(), false /* bLogErrors */);

	FMemoryView Payload { SerializedPackage.GetData(), (uint64)SerializedPackage.TotalSize()};
	if (Post(*Request, OplogNewEntryPath, Payload) && (Request->GetResponseCode() == 200 || Request->GetResponseCode() == 201))
	{
		return static_cast<uint64>(Payload.GetSize());
	}
	else
	{
		UE_LOGF(LogZenStore, Warning, "Oplog end FAILED: %ls", *Request->GetDetailedResultString(OplogNewEntryPath));
		return (FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("End build pass failed, NewOpLogPath='") << OplogNewEntryPath << TEXT("'"));
	}
}

} // UE

#else // not desktop platform

namespace UE {
namespace Zen {
	struct FZenHttpRequestPool
	{
	};
}

FZenStoreHttpClient::FZenStoreHttpClient()
{
}

FZenStoreHttpClient::FZenStoreHttpClient(const FStringView InHostName, uint16 InPort)
{
}

FZenStoreHttpClient::FZenStoreHttpClient(UE::Zen::FServiceSettings&& InSettings)
{
}

FZenStoreHttpClient::~FZenStoreHttpClient()
{
}

bool FZenStoreHttpClient::TryCreateProject(FStringView InProjectId, FStringView InOplogId, FStringView ServerRoot,
	FStringView EngineRoot,	FStringView ProjectRoot,
	FStringView ProjectFilePath)
{
	return false;
}

bool FZenStoreHttpClient::TryDeleteOplog(FStringView InProjectId, FStringView InOplogId)
{
	return false;
}

bool FZenStoreHttpClient::TryCreateOplog(FStringView InProjectId, FStringView InOplogId, FStringView InOplogLifetimeMarkerPath)
{
	return false;
}

void FZenStoreHttpClient::InitializeReadOnly(FStringView InProjectId, FStringView InOplogId)
{
}

TIoStatusOr<uint64> FZenStoreHttpClient::GetChunkSize(const FIoChunkId& Id)
{
	return 0;
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadChunk(const FIoChunkId& Id, uint64 Offset, uint64 Size)
{
	return FIoBuffer();
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadChunk(const FIoHash& RawHash, uint64 Offset, uint64 Size)
{
	return FIoBuffer();
}

void FZenStoreHttpClient::ReadChunks(TConstArrayView<FIoChunkId> Ids, TUniqueFunction<void(const FIoChunkId& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	for (const FIoChunkId& Id : Ids)
	{
		ChunkCallback(Id, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Unsupported, Oid=") << *WriteToString<32>(Id)));
	}
}

void FZenStoreHttpClient::ReadChunks(TConstArrayView<FIoHash> RawHashes, TUniqueFunction<void(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	for (const FIoHash& RawHash : RawHashes)
	{
		ChunkCallback(RawHash, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Unsupported, RawHash=") << *WriteToString<32>(RawHash)));
	}
}

void FZenStoreHttpClient::ReadChunksAsync(TConstArrayView<FIoChunkId> Ids, TUniqueFunction<void(const FIoChunkId& Id, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	for (const FIoChunkId& Id : Ids)
	{
		ChunkCallback(Id, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Unsupported, Oid=") << *WriteToString<32>(Id)));
	}
}

void FZenStoreHttpClient::ReadChunksAsync(TConstArrayView<FIoHash> RawHashes, TUniqueFunction<void(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)>&& ChunkCallback)
{
	for (const FIoHash& RawHash : RawHashes)
	{
		ChunkCallback(RawHash, (FIoStatus)(FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Unsupported, RawHash=") << *WriteToString<32>(RawHash)));
	}
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadOpLogAttachment(FStringView Id)
{
	return FIoBuffer();
}

TIoStatusOr<FIoBuffer> FZenStoreHttpClient::ReadOpLogUri(FStringBuilderBase& ChunkUri, uint64 Offset, uint64 Size)
{
	return FIoBuffer();
}

void FZenStoreHttpClient::StartBuildPass()
{
}

TIoStatusOr<uint64> FZenStoreHttpClient::EndBuildPass(FCbPackage OpEntry)
{
	return FIoStatus(EIoErrorCode::Unknown);
}

TIoStatusOr<uint64> FZenStoreHttpClient::AppendOp(FCbPackage OpEntry)
{
	return TIoStatusOr<uint64>();
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetProjectInfo()
{
	return TFuture<TIoStatusOr<FCbObject>>();
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetOplog(bool bTrimByReferencedSet)
{
	return TFuture<TIoStatusOr<FCbObject>>();
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetFiles()
{
	return TFuture<TIoStatusOr<FCbObject>>();
}

TFuture<TIoStatusOr<FCbObject>> FZenStoreHttpClient::GetChunkInfos()
{
	return TFuture<TIoStatusOr<FCbObject>>();
}

}

#endif // desktop platform

namespace UE
{

bool FZenStoreHttpClient::IsConnected() const
{
	return bConnectionSucceeded;
}

namespace Zen::Private
{

TArray<TUniquePtr<UTF8CHAR[]>> AttachmentIds;
FRWLock AttachmentIdsLock;

}

const UTF8CHAR* FZenStoreHttpClient::FindOrAddAttachmentId(FUtf8StringView AttachmentText)
{
	FRWScopeLock AttachmentIdScopeLock(Zen::Private::AttachmentIdsLock, SLT_ReadOnly);
	bool bReadOnly = true;

	for (;;)
	{
		int32 Index = Algo::LowerBound(Zen::Private::AttachmentIds, AttachmentText,
			[](const TUniquePtr<UTF8CHAR[]>& Existing, FUtf8StringView AttachmentText)
			{
				return FUtf8StringView(Existing.Get()).Compare(AttachmentText, ESearchCase::IgnoreCase) < 0;
			});
		UTF8CHAR* Existing = nullptr;
		if (Index != Zen::Private::AttachmentIds.Num())
		{
			Existing = Zen::Private::AttachmentIds[Index].Get();
		}
 		if (Existing == nullptr || !FUtf8StringView(Existing).Equals(AttachmentText, ESearchCase::IgnoreCase))
		{
			if (bReadOnly)
			{
				AttachmentIdScopeLock.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
				bReadOnly = false;
				continue;
			}

			auto StrDupNew = [](const UTF8CHAR* Data, int32 Len)
			{
				UTF8CHAR* Dup = new UTF8CHAR[Len + 1];
				FMemory::Memcpy(Dup, Data, Len * sizeof(UTF8CHAR));
				Dup[Len] = UTF8CHAR('\0');
				return Dup;
			};
			Existing = StrDupNew(AttachmentText.GetData(), AttachmentText.Len());
			Zen::Private::AttachmentIds.Insert(TUniquePtr<UTF8CHAR[]>(Existing), Index);
		}
		return Existing;
	}
}

const UTF8CHAR* FZenStoreHttpClient::FindAttachmentId(FUtf8StringView AttachmentText)
{
	FReadScopeLock AttachmentIdScopeLock(Zen::Private::AttachmentIdsLock);

	int32 Index = Algo::LowerBound(Zen::Private::AttachmentIds, AttachmentText,
		[](const TUniquePtr<UTF8CHAR[]>& Existing, FUtf8StringView AttachmentText)
		{
			return FUtf8StringView(Existing.Get()).Compare(AttachmentText, ESearchCase::IgnoreCase) < 0;
		});
	if (Index == Zen::Private::AttachmentIds.Num())
	{
		return nullptr;
	}
	const UTF8CHAR* Existing = Zen::Private::AttachmentIds[Index].Get();
	if (!FUtf8StringView(Existing).Equals(AttachmentText, ESearchCase::IgnoreCase))
	{
		return nullptr;
	}
	return Existing;
}

}
