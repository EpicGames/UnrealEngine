// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZenStoreWriter.h"

#include "Algo/BinarySearch.h"
#include "Algo/Find.h"
#include "Algo/IsSorted.h"
#include "Algo/Sort.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Async/Async.h"
#include "Containers/Queue.h"
#include "Cooker/CookArtifact.h"
#include "Experimental/ZenProjectStoreWriter.h"
#include "Experimental/ZenServerInterface.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoDispatcher.h"
#include "IPAddress.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "PackageStoreOptimizer.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/BulkData.h"
#include "Serialization/CompactBinaryContainerSerialization.h"
#include "Serialization/CompactBinaryPackage.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/LargeMemoryWriter.h" 
#include "SocketSubsystem.h"
#include "UObject/ICookInfo.h"
#include "UObject/SavePackage.h"
#include "ZenCookArtifactReader.h"
#include "ZenFileSystemManifest.h"
#include "ZenStoreHttpClient.h"

DEFINE_LOG_CATEGORY_STATIC(LogZenStoreWriter, Log, All);

using namespace UE;

static FString GZenStoreWriterProjectIdOverride;
static FAutoConsoleVariableRef CVar_ZenStoreWriterProjectIdOverride(
	TEXT("ZenStoreWriter.ProjectIdOverride"),
	GZenStoreWriterProjectIdOverride,
	TEXT("Overrides the project ID otherwise taken from FApp/-DLCNAME. Format: '[<ParentId>/]ProjectId'")
);

// Using a suffix instead of an override as we may be cooking for multiple target platforms in one session
static FString GZenStoreWriterOplogIdSuffix;
static FAutoConsoleVariableRef CVar_ZenStoreWriterOplogIdSuffix(
	TEXT("ZenStoreWriter.OplogIdSuffix"),
	GZenStoreWriterOplogIdSuffix,
	TEXT("Adds to the end of the oplog ID otherwise it is the solely target platform. Format: 'TargetPlatform[Suffix]'")
);

static bool GZenStoreWriterExperimentalOutputPayloadInfo = false;
static FAutoConsoleVariableRef CVar_ZenStoreWriterExperimentalOutputPayloadInfo(
	TEXT("ZenStoreWriter.Experimental.OutputPayloadInfo"),
	GZenStoreWriterExperimentalOutputPayloadInfo,
	TEXT("Output metadata about each bulkdata payload owned by a package data entry to the oplog")
);

static void GetZenStoreProjectId(FString& ProjectId, FString& ParentId)
{
	ParentId.Reset();

	if (GZenStoreWriterProjectIdOverride.IsEmpty())
	{
		FString DLCName;
		FParse::Value(FCommandLine::Get(), TEXT("DLCNAME="), DLCName);
		DLCName.ToLowerInline();

		ProjectId = FApp::GetZenStoreProjectId(DLCName);
		if (!DLCName.IsEmpty())
		{
			ParentId = FApp::GetZenStoreProjectId();
		}

		return;
	}

	FStringView Override = GZenStoreWriterProjectIdOverride;

	int32 SlashIndex;
	if (!Override.FindChar('/', SlashIndex))
	{
		ProjectId = Override;
		return;
	}

	ProjectId = Override.Left(SlashIndex);
	check(!ProjectId.IsEmpty());

	Override = Override.Mid(SlashIndex + 1);
	check(!Override.IsEmpty());
	ParentId = Override;
}

static void GetZenStoreOplogId(FString& OplogId, const ITargetPlatform* InTargetPlatform)
{
	if (!FParse::Value(FCommandLine::Get(), TEXT("-ZenStorePlatform="), OplogId))
	{
		OplogId = InTargetPlatform->PlatformName();
	}

	if (!GZenStoreWriterOplogIdSuffix.IsEmpty())
	{
		OplogId += GZenStoreWriterOplogIdSuffix;
	}
}

// Note that this is destructive - we yank out the buffer memory from the 
// IoBuffer into the FSharedBuffer
FSharedBuffer IoBufferToSharedBuffer(FIoBuffer& InBuffer)
{
	InBuffer.EnsureOwned();
	const uint64 DataSize = InBuffer.DataSize();
	uint8* DataPtr = InBuffer.Release().ValueOrDie();
	return FSharedBuffer{ FSharedBuffer::TakeOwnership(DataPtr, DataSize, FMemory::Free) };
};

FCbObjectId ToObjectId(const FIoChunkId& ChunkId)
{
	return FCbObjectId(MakeMemoryView(ChunkId.GetData(), ChunkId.GetSize()));
}

FMD5Hash IoHashToMD5(const FIoHash& IoHash)
{
	const FIoHash::ByteArray& Bytes = IoHash.GetBytes();
	
	FMD5 MD5Gen;
	MD5Gen.Update(Bytes, sizeof(FIoHash::ByteArray));
	
	FMD5Hash Hash;
	Hash.Set(MD5Gen);

	return Hash;
}

FZenStoreWriter::FPackageDataEntry::~FPackageDataEntry()
{

}

FZenStoreWriter::FPendingPackageState::~FPendingPackageState()
{

}

struct FZenStoreWriter::FZenCommitInfo
{
	IPackageWriter::FCommitPackageInfo CommitInfo;
	TUniquePtr<FPendingPackageState> PackageState;
};

TArray<const UTF8CHAR*> FZenStoreWriter::ReservedOplogKeys;

void FZenStoreWriter::StaticInit()
{
	if (ReservedOplogKeys.Num() > 0)
	{
		return;
	}

	ReservedOplogKeys.Append({ UTF8TEXT("files"), UTF8TEXT("key"), UTF8TEXT("packagedata"), UTF8TEXT("bulkdata"), UTF8TEXT("packagestoreentry") });
	Algo::Sort(ReservedOplogKeys, [](const UTF8CHAR* A, const UTF8CHAR* B)
		{
			return FUtf8StringView(A).Compare(FUtf8StringView(B), ESearchCase::IgnoreCase) < 0;
		});;
}

FZenStoreWriter::FZenStoreWriter(
	const FString& InOutputPath,
	const FString& InMetadataDirectoryPath,
	const ITargetPlatform* InTargetPlatform,
	TSharedRef<UE::Cook::ICookArtifactReader> InCookArtifactReader
)
	: CookArtifactReader(InCookArtifactReader)
	, TargetPlatform(*InTargetPlatform)
	, TargetPlatformFName(*InTargetPlatform->PlatformName())
	, OutputPath(InOutputPath)
	, MetadataDirectoryPath(InMetadataDirectoryPath)
	, PackageStoreOptimizer(new FPackageStoreOptimizer())
	, CookMode(ICookedPackageWriter::FCookInfo::CookByTheBookMode)
	, bInitialized(false)
	, bInitializedConnection(false)
	, bProvidePerPackageResults(false)
{
	StaticInit();

	GetZenStoreProjectId(ProjectId, ParentProjectId);

	GetZenStoreOplogId(OplogId, InTargetPlatform);

	HttpClient = MakeUnique<UE::FZenStoreHttpClient>();

	IsLocalConnection = HttpClient->GetZenServiceInstance().IsServiceRunningLocally();

	FString RootDir = FPaths::RootDir();
	FString EngineDir = FPaths::EngineDir();
	FPaths::NormalizeDirectoryName(EngineDir);
	FString ProjectDir = FPaths::ProjectDir();
	FPaths::NormalizeDirectoryName(ProjectDir);
	FString ProjectPath = FPaths::GetProjectFilePath();
	FPaths::NormalizeFilename(ProjectPath);

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FString AbsServerRoot = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*RootDir);
	FString AbsEngineDir = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*EngineDir);
	FString AbsProjectDir = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*ProjectDir);
	FString ProjectFilePath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*ProjectPath);

	HttpClient->TryCreateProject(ProjectId, ParentProjectId, OplogId, AbsServerRoot, AbsEngineDir, AbsProjectDir, IsLocalConnection ? ProjectFilePath : FStringView());

	PackageStoreOptimizer->Initialize();

	ZenFileSystemManifest = MakeUnique<FZenFileSystemManifest>(TargetPlatform, OutputPath);
#if WITH_EDITOR
	ZenFileSystemManifest->ConstructReferencedSetClientPath(MetadataDirectoryPath);
#endif

	Compressor = FOodleDataCompression::ECompressor::Mermaid;
	CompressionLevel = FOodleDataCompression::ECompressionLevel::VeryFast;
}

FZenStoreWriter::~FZenStoreWriter()
{
	FEventCountToken Token = CommitPendingEvent.PrepareWait();
	if (NumCommitPending.load())
	{
		UE_LOGF(LogZenStoreWriter, Display, "Flushing pending commits...");
		CommitPendingEvent.Wait(Token);
	}

	FScopeLock _(&PackagesCriticalSection);

	if (PendingPackages.Num())
	{
		UE_LOGF(LogZenStoreWriter, Warning, "Pending packages at shutdown!");
	}
}

void FZenStoreWriter::SetCooker(UE::PackageWriter::Private::ICookerInterface* CookerInterface)
{
	Cooker = CookerInterface;
#if WITH_EDITOR
	if (Cooker)
	{
		ZenFileSystemManifest->SetCookSandbox(Cooker->GetCookSandbox());
	}
#endif
}

bool FZenStoreWriter::CanUse()
{
	Zen::FZenServiceInstance& DefaultInstance = Zen::GetDefaultServiceInstance();
	return DefaultInstance.IsServiceReady();
}

void FZenStoreWriter::WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive, const TArray<FFileRegion>& FileRegions)
{
	check(Info.ChunkId.IsValid());
	FPendingPackageState& ExistingState = GetPendingPackage(Info.PackageName);
	FPackageDataEntry& Entry = ExistingState.PackageData.AddDefaulted_GetRef();

	TRACE_CPUPROFILER_EVENT_SCOPE(FZenStoreWriter::WritePackageData);

	FIoBuffer PackageBuffer;
	if (ExistingState.PreOptimizedPackage.IsValid())
	{
		// If we are writing output data after having done a diff operation, we may already have pre-optimized package data in memory and
		// we should use that instead of generating it again.
		Entry.OptimizedPackage = MoveTemp(ExistingState.PreOptimizedPackage);
		PackageBuffer = FIoBuffer(FIoBuffer::Clone, ExportsArchive.GetData(), Info.HeaderSize);
	}
	else
	{
		ExistingState.OriginalHeaderSize = Info.HeaderSize;

		int64 DataSize = ExportsArchive.TotalSize();
		FIoBuffer PackageData(FIoBuffer::AssumeOwnership, ExportsArchive.ReleaseOwnership(), DataSize);

		FIoBuffer CookedHeaderBuffer = FIoBuffer(PackageData.Data(), Info.HeaderSize, PackageData);
		FIoBuffer CookedExportsBuffer = FIoBuffer(PackageData.Data() + Info.HeaderSize, PackageData.DataSize() - Info.HeaderSize, PackageData);
		Entry.OptimizedPackage.Reset(PackageStoreOptimizer->CreatePackageFromCookedHeader(Info.PackageName, CookedHeaderBuffer));
		PackageBuffer = PackageStoreOptimizer->CreatePackageBuffer(Entry.OptimizedPackage.Get(), CookedExportsBuffer);
	}

	Entry.FileRegions = FileRegions;
	for (FFileRegion& Region : Entry.FileRegions)
	{
		// Adjust regions so they are relative to the start of the export bundle buffer
		Region.Offset -= ExistingState.OriginalHeaderSize;
		Region.Offset += Entry.OptimizedPackage->GetHeaderSize();
	}

	// Commit to Zen build store

	FCbObjectId ChunkOid = ToObjectId(Info.ChunkId);

	Entry.CompressedPayload = UE::Tasks::Launch(TEXT("CompressPayload"), [this, PackageBuffer]()
	{ 
		return FCompressedBuffer::Compress(FSharedBuffer::MakeView(PackageBuffer.GetView()), Compressor, CompressionLevel);
	});

	Entry.Info				= Info;
	Entry.ChunkId			= ChunkOid;
	Entry.IsValid			= true;
}

void FZenStoreWriter::WriteIoStorePackageData(const FPackageInfo& Info, const FIoBuffer& PackageData, const FPackageStoreEntryResource& PackageStoreEntry, const TArray<FFileRegion>& FileRegions)
{
	check(Info.ChunkId.IsValid());

	TRACE_CPUPROFILER_EVENT_SCOPE(WriteIoStorePackageData);

	//WriteFileRegions(*FPaths::ChangeExtension(Info.LooseFilePath, FString(".uexp") + FFileRegion::RegionsFileExtension), FileRegionsCopy);

	FCbObjectId ChunkOid = ToObjectId(Info.ChunkId);

	FPendingPackageState& ExistingState = GetPendingPackage(Info.PackageName);

	FPackageDataEntry& Entry = ExistingState.PackageData.AddDefaulted_GetRef();

	PackageData.EnsureOwned();

	Entry.CompressedPayload = UE::Tasks::Launch(TEXT("CompressPayload"), [this, PackageData]()
	{ 
		return FCompressedBuffer::Compress(FSharedBuffer::MakeView(PackageData.GetView()), Compressor, CompressionLevel);
	});

	Entry.Info				= Info;
	Entry.ChunkId			= ChunkOid;
	Entry.IsValid			= true;
}

void FZenStoreWriter::WriteBulkData(const FBulkDataInfo& Info, const FIoBuffer& BulkData, const TArray<FFileRegion>& FileRegions)
{
	check(Info.ChunkId.IsValid());

	FCbObjectId ChunkOid = ToObjectId(Info.ChunkId);

	FPendingPackageState& ExistingState = GetPendingPackage(Info.PackageName);

	FBulkDataEntry& BulkEntry = ExistingState.BulkData.AddDefaulted_GetRef(); 

	BulkData.EnsureOwned();

	BulkEntry.CompressedPayload = UE::Tasks::Launch(TEXT("CompressPayload"), [this, BulkData]()
	{ 
		return FCompressedBuffer::Compress(FSharedBuffer::MakeView(BulkData.GetView()), Compressor, CompressionLevel);
	});

	BulkEntry.Info		= Info;
	BulkEntry.ChunkId	= ChunkOid;
	BulkEntry.IsValid	= true;
	BulkEntry.FileRegions = FileRegions;
}

void FZenStoreWriter::WriteAdditionalFile(const FAdditionalFileInfo& Info, const FIoBuffer& FileData)
{
	FPendingPackageState& ExistingState = GetPendingPackage(Info.PackageName);

	FFileDataEntry& FileEntry = ExistingState.FileData.AddDefaulted_GetRef();

	// TODO: Change FileData into FIoBuffer&&.
	// SavePackage currently allows us to destroy not only the FileData structure but even the data it points to,
	// but it would be nice to indicate that contract in the function call.
	FSharedBuffer FileDataSharedBuffer = IoBufferToSharedBuffer(const_cast<FIoBuffer&>(FileData));

#if !WITH_EDITOR
	// ZenStoreWriter is only used on editor, and we've added some dependencies on editor-only functionality of
	// SavePackage.
	checkNoEntry();
#else
	UE::SavePackageUtilities::IncrementOutstandingAsyncWrites();
	FileEntry.CompressedPayload = UE::Tasks::Launch(TEXT("CompressPayLoad"),
		[this, FileDataSharedBuffer, Filename=Info.Filename]()
	{
		ON_SCOPE_EXIT
		{
			UE::SavePackageUtilities::DecrementOutstandingAsyncWrites();
		};

		UE::PackageWriter::Private::FWriteFileData WriteFileData;
		WriteFileData.Filename = Filename;
		WriteFileData.Buffer = FCompositeBuffer(FileDataSharedBuffer);
		WriteFileData.bIsSidecar = true;
		WriteFileData.bContributeToHash = false;
		// For robustness, we handle all AdditionalFiles reported by UObject::CookAdditionalFiles as if they might be
		// written by multiple packages, without regard for their filename.
		WriteFileData.bPackageSpecificFilename = false;

		// WriteFileOnCookDirector or HashAndWrite will calculate the hash for us if we pass in
		// EWriteOptions::WriteHash, but we need a different hash - the hash of the compressed file that we store
		// in ZenServer. So omit WriteHash and just pass in unused placeholders for the hash output.
		FMD5 UnusedHash;
		TRefCountPtr<FPackageHashes> UnusedHashes;
		if (Cooker)
		{
			Cooker->WriteFileOnCookDirector(WriteFileData, UnusedHash, UnusedHashes, EWriteOptions::Write);
		}
		else
		{
			UE::PackageWriter::Private::HashAndWrite(WriteFileData, UnusedHash, UnusedHashes, EWriteOptions::Write);
		}

		return FCompressedBuffer::Compress(FileDataSharedBuffer, Compressor, CompressionLevel);
	});
#endif

	{
		UE::TUniqueLock ScopeLock(ZenFileSystemManifestMutex);
		const FZenFileSystemManifestEntry& ManifestEntry = ZenFileSystemManifest->CreateManifestEntry(Info.Filename);
		FileEntry.Info					= Info;
		FileEntry.Info.ChunkId			= ManifestEntry.FileChunkId;
		FileEntry.ZenManifestServerPath = ManifestEntry.ServerPath;
		FileEntry.ZenManifestClientPath = ManifestEntry.ClientPath;
	}

	if (bProvidePerPackageResults)
	{
		PackageAdditionalFiles.FindOrAdd(Info.PackageName).Add(Info.Filename);
	}
}

void FZenStoreWriter::WriteLinkerAdditionalData(const FLinkerAdditionalDataInfo& Info, const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions)
{
	// LinkerAdditionalData is not yet implemented in this writer; it is only used for VirtualizedBulkData which is not used in cooked content
	checkNoEntry();
}

void FZenStoreWriter::WritePackageTrailer(const FPackageTrailerInfo& Info, const FIoBuffer& Data)
{
	// PackageTrailers are not yet implemented in this writer; it is only used for EditorBulkData which is not used in cooked content
	checkNoEntry();
}

void FZenStoreWriter::RegisterDeterminismHelper(UObject* SourceObject,
	const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
{
	if (Cooker)
	{
		Cooker->RegisterDeterminismHelper(this, SourceObject, DeterminismHelper);
	}
}

FString FZenStoreWriter::GetProjectStorePath() const
{
	return OutputPath / TEXT("ue.projectstore");
}

void FZenStoreWriter::InitializeConnection()
{
	if (bInitializedConnection)
	{
		return;
	}
	WriteProjectStoreFile();
	bInitializedConnection = true;
}

void FZenStoreWriter::WriteProjectStoreFile()
{
	// We need to write the ProjectStore file early, for use both during this->Initialize and by
	// FZenCookArtifactReader->InitializeConnection.
	FStringView ServiceHostName = TEXT("localhost");
	uint16 ServicePort = 8558;
	FString HostAuthJson;
	Zen::FZenProjectStoreWriter ProjectStoreWriter(*GetProjectStorePath());
	const Zen::FZenServiceInstance& ZenServiceInstance = HttpClient->GetZenServiceInstance();
	ServiceHostName = ZenServiceInstance.GetEndpoint().GetName();
	ServicePort = ZenServiceInstance.GetEndpoint().GetPort();
	Zen::GetLocalServiceHostAuth(HostAuthJson);
	ProjectStoreWriter.Write(ServiceHostName, ServicePort, HostAuthJson, IsLocalConnection, *ProjectId, *OplogId, TargetPlatformFName);
}

void FZenStoreWriter::Initialize(const FCookInfo& Info)
{
	CookMode = Info.CookMode;

	if (!bInitialized)
	{
		InitializeConnection();
		bool CleanBuild = Info.bFullBuild && !Info.bWorkerOnSharedSandbox;
		if (CleanBuild)
		{
			bool bOplogDeleted = HttpClient->TryDeleteOplog(ProjectId, OplogId);
			UE_CLOGF(!bOplogDeleted, LogZenStoreWriter, Fatal, "Failed to delete oplog on the ZenServer");

			UE_LOGF(LogZenStoreWriter, Display, "Deleting %ls...", *OutputPath);
			const bool bRequireExists = false;
			const bool bTree = true;
			IFileManager::Get().DeleteDirectory(*OutputPath, bRequireExists, bTree);
		}

		// If we deleted the directory above for a clean build, or the cooker deleted it, then rewrite
		// the ProjectStorePath for use by TryCreateOplog and any other systems that want to query the oplog
		// during cook.
		if (!IFileManager::Get().FileExists(*GetProjectStorePath()))
		{
			WriteProjectStoreFile();
		}

		bool bOplogEstablished = HttpClient->TryCreateOplog(ProjectId, OplogId, GetProjectStorePath());
		UE_CLOGF(!bOplogEstablished, LogZenStoreWriter, Fatal, "Failed to establish oplog on the ZenServer");

		if (!Info.bFullBuild)
		{
			UE_LOGF(LogZenStoreWriter, Display, "Fetching oplog...");

			TFuture<FIoStatus> FutureOplogStatus = HttpClient->GetOplog(false /* bTrimByReferencedSet*/)
				.Next([this](TIoStatusOr<FCbObject> OplogStatus)
				{
					if (!OplogStatus.IsOk())
					{
						return OplogStatus.Status();
					}

					FCbObject Oplog = OplogStatus.ConsumeValueOrDie();

					if (Oplog["entries"])
					{
						FWriteScopeLock _(EntriesLock);
						for (FCbField& OplogEntry : Oplog["entries"].AsArray())
						{
							FCbObject OplogObj = OplogEntry.AsObject();

							if (OplogObj["packagestoreentry"])
							{
								FPackageStoreEntryResource Entry = FPackageStoreEntryResource::FromCbObject(OplogObj["packagestoreentry"].AsObject());
								const FName PackageName = Entry.PackageName;

								const int32 Index = PackageStoreEntries.Num();

								PackageStoreEntries.Add(MoveTemp(Entry));
								FOplogCookInfo& CookInfo = CookedPackagesInfo.Add_GetRef({ PackageName });
								PackageNameToIndex.Add(PackageName, Index);

								for (FCbFieldView Field : OplogObj)
								{
									FUtf8StringView FieldName = Field.GetName();
									if (IsReservedOplogKey(FieldName))
									{
										continue;
									}
									if (Field.IsHash())
									{
										const UTF8CHAR* AttachmentId = UE::FZenStoreHttpClient::FindOrAddAttachmentId(FieldName);
										CookInfo.Attachments.Add({ AttachmentId, Field.AsHash() });
									}
								}
								CookInfo.Attachments.Shrink();
								check(Algo::IsSorted(CookInfo.Attachments,
									[](const FOplogCookInfo::FAttachment& A, const FOplogCookInfo::FAttachment& B)
									{
										return FUtf8StringView(A.Key).Compare(FUtf8StringView(B.Key), ESearchCase::IgnoreCase) < 0;
									}));
							}
						}
					}

					return FIoStatus::Ok;
				});

			UE_LOGF(LogZenStoreWriter, Display, "Fetching file manifest...");

			TSet<FString> CopyFromPreviousCookDenyListExtensions;
			CopyFromPreviousCookDenyListExtensions.Append({ TEXT("ushaderbytecode"),TEXT("assetinfo.json"), TEXT("stinfo"), TEXT("ushaderpipelines") });

			TIoStatusOr<FCbObject> FileStatus = HttpClient->GetFiles().Get();
			if (FileStatus.IsOk())
			{
				FCbObject FilesObj = FileStatus.ConsumeValueOrDie();
				for (FCbField& FileEntry : FilesObj["files"])
				{
					FCbObject FileObj = FileEntry.AsObject();
					FCbObjectId FileId = FileObj["id"].AsObjectId();
					FString ServerPath = FString(FileObj["serverpath"].AsString());
					FString ClientPath = FString(FileObj["clientpath"].AsString());

					if (!ServerPath.IsEmpty())
					{
						const FString AbsPath = ZenFileSystemManifest->ServerRootPath() / ServerPath;
						if (FPaths::FileExists(AbsPath))
						{
							FIoChunkId FileChunkId;
							FileChunkId.Set(FileId.GetView());

							// Do not allow the addition of some extensions that we know are rewritten every cook
							// and that we want to be able to remove from the cook when they are removed due to
							// asset or code changes. TODO: Remove this copy-through of the files from the previous
							// cook and use the ZenCookArtifactReader instead, and allow the cooker to delete files
							// from the ZenCookArtifactReader when it deletes them from local disk, see UE-351803
							// and UE-352816.
							FStringView Extension = FPathViews::GetExtension(ClientPath, UE::Paths::EFlags::AllowCompoundExtension);
							if (!CopyFromPreviousCookDenyListExtensions.ContainsByHash(
								GetTypeHash(Extension), Extension))
							{
								UE::TUniqueLock LockScope(ZenFileSystemManifestMutex);
								ZenFileSystemManifest->AddManifestEntry(FileChunkId, MoveTemp(ServerPath),
									MoveTemp(ClientPath));
							}
						}
					}
				}

				UE_LOGF(LogZenStoreWriter, Display, "Fetched '%d' file(s) from oplog '%ls/%ls'", ZenFileSystemManifest->NumEntries(), *ProjectId, *OplogId);
			}
			else
			{
				UE_LOGF(LogZenStoreWriter, Warning, "Failed to fetch file(s) from oplog '%ls/%ls'", *ProjectId, *OplogId);
			}

			if (FutureOplogStatus.Get().IsOk())
			{
				UE_LOGF(LogZenStoreWriter, Display, "Fetched '%d' packages(s) from oplog '%ls/%ls'", PackageStoreEntries.Num(), *ProjectId, *OplogId);
			}
			else
			{
				UE_LOGF(LogZenStoreWriter, Warning, "Failed to fetch oplog '%ls/%ls'", *ProjectId, *OplogId);
			}
		}
		bInitialized = true;
	}
	else
	{
		if (Info.bFullBuild)
		{
			RemoveCookedPackages();
		}
	}
}

void FZenStoreWriter::BeginCook(const FCookInfo& Info)
{
	if (Info.bWorkerOnSharedSandbox)
	{
		bProvidePerPackageResults = true;
	}
	AllPackageHashes.Empty();

	if (CookMode == ICookedPackageWriter::FCookInfo::CookOnTheFlyMode)
	{
		{
			UE::TUniqueLock LockScope(ZenFileSystemManifestMutex);
			ZenFileSystemManifest->Generate(MetadataDirectoryPath, Info.ReferencedPlugins);
		}
		TIoStatusOr<uint64> Status = HttpClient->AppendOp(CreateProjectMetaDataOpPackage("CookOnTheFly"));
		UE_CLOGF(!Status.IsOk(), LogZenStoreWriter, Error, "Failed to append OpLog. Reason: %ls", *Status.Status().ToString());
	}
}

void FZenStoreWriter::EndCook(const FCookInfo& Info)
{
	UE_LOGF(LogZenStoreWriter, Display, "Flushing...");
	
	FEventCountToken Token = CommitPendingEvent.PrepareWait();
	if (NumCommitPending.load())
	{
		CommitPendingEvent.Wait(Token);
	}

	if (!Info.bWorkerOnSharedSandbox)
	{
		{
			UE::TUniqueLock LockScope(ZenFileSystemManifestMutex);
			ZenFileSystemManifest->Generate(MetadataDirectoryPath, Info.ReferencedPlugins);
		}
		if (FCbPackage ReferencedSetPackage = CreateReferencedSetOpPackage())
		{
			TIoStatusOr<uint64> Status = HttpClient->AppendOp(MoveTemp(ReferencedSetPackage));
			UE_CLOGF(!Status.IsOk(), LogZenStoreWriter, Error, "Failed to append ReferencedSetOp. Reason: %ls", *Status.Status().ToString());
		}
		{
			TIoStatusOr<uint64> Status = HttpClient->EndBuildPass(CreateProjectMetaDataOpPackage("EndCook"));
			UE_CLOGF(!Status.IsOk(), LogZenStoreWriter, Error, "Failed to append OpLog and end the build pass. Reason: %ls", *Status.Status().ToString());
		}
	}
	UE_LOGF(LogZenStoreWriter, Display, "Output:\t%lld Public runtime script objects", PackageStoreOptimizer->GetTotalScriptObjectCount());
}

FZenStoreWriter::ZenHostInfo FZenStoreWriter::GetHostInfo() const
{
	const Zen::FZenServiceInstance& Instance = HttpClient->GetZenServiceInstance();

	FZenStoreWriter::ZenHostInfo Info;
	Info.ProjectId = ProjectId;
	Info.OplogId = OplogId;
	if (IsLocalConnection)
	{
		Info.HostName = "localhost";
	}
	else
	{
		Info.HostName = Instance.GetEndpoint().GetHostName();
	}
	Info.HostPort = Instance.GetEndpoint().GetPort();
	return Info;
}

void FZenStoreWriter::BeginPackage(const FBeginPackageInfo& Info)
{
	FPendingPackageState& State = AddPendingPackage(Info.PackageName);
	State.PackageName = Info.PackageName;
}

bool FZenStoreWriter::IsReservedOplogKey(FUtf8StringView Key)
{
	int32 Index = Algo::LowerBound(ReservedOplogKeys, Key,
		[](const UTF8CHAR* Existing, FUtf8StringView Key)
		{
			return FUtf8StringView(Existing).Compare(Key, ESearchCase::IgnoreCase) < 0;
		});
	return Index != ReservedOplogKeys.Num() &&
		FUtf8StringView(ReservedOplogKeys[Index]).Equals(Key, ESearchCase::IgnoreCase);
}

void FZenStoreWriter::CommitPackage(FCommitPackageInfo&& Info)
{
	if (Info.Status == ECommitStatus::Canceled || Info.Status == ECommitStatus::NotCommitted)
	{
		RemovePendingPackage(Info.PackageName);
		return;
	}

	UE::SavePackageUtilities::IncrementOutstandingAsyncWrites();

	// If we are computing hashes, we need to allocate where the hashes will go.
	// Access to this is protected by the above IncrementOutstandingAsyncWrites.
	if (EnumHasAnyFlags(Info.WriteOptions, EWriteOptions::ComputeHash))
	{
		FPendingPackageState& ExistingState = GetPendingPackage(Info.PackageName);
		ExistingState.PackageHashes = new FPackageHashes();
		if (bProvidePerPackageResults)
		{
			ExistingState.PackageHashesCompletionPromise.Reset(new TPromise<int>());
			ExistingState.PackageHashes->CompletionFuture = ExistingState.PackageHashesCompletionPromise->GetFuture();
		}

		if (Info.Status == ECommitStatus::Success)
		{
			// Only record hashes for successful saves. A single package can be saved unsuccessfully multiple times
			// during a cook if it keeps timing out.
			TRefCountPtr<FPackageHashes>& ExistingPackageHashes = AllPackageHashes.FindOrAdd(Info.PackageName);
			// This looks weird but we've found the _TRefCountPtr_, not the FPackageHashes. When newly assigned
			// it will be an empty pointer, which is what we want.
			if (ExistingPackageHashes.IsValid())
			{
				UE_LOGF(LogZenStoreWriter, Error, "FZenStoreWriter commiting the same package twice during a cook! (%ls)", *Info.PackageName.ToString());
			}
			ExistingPackageHashes = ExistingState.PackageHashes;
		}
	}

	TUniquePtr<FPendingPackageState> PackageState = RemovePendingPackage(Info.PackageName);
	if (EnumHasAnyFlags(Info.WriteOptions, IPackageWriter::EWriteOptions::Write | IPackageWriter::EWriteOptions::ComputeHash))
	{
		checkf(Info.Status != ECommitStatus::Success || !PackageState->PackageData.IsEmpty(),
			TEXT("CommitPackage called with CommitStatus::Success but without first calling WritePackageData"));
	}
	FZenCommitInfo ZenCommitInfo{ Forward<FCommitPackageInfo>(Info), MoveTemp(PackageState) };
	if (FPlatformProcess::SupportsMultithreading())
	{
		NumCommitPending++;
		UE::Tasks::Launch(
			TEXT("CommitPackageTask"),
			[this, ZenCommitInfo = MoveTemp(ZenCommitInfo)]() mutable
			{
				CommitPackageInternal(MoveTemp(ZenCommitInfo));
				if (--NumCommitPending == 0)
				{
					CommitPendingEvent.Notify();
				}
			}
		);
	}
	else
	{
		CommitPackageInternal(MoveTemp(ZenCommitInfo));
	}
}

void FZenStoreWriter::CommitPackageInternal(FZenCommitInfo&& ZenCommitInfo)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FZenStoreWriter::CommitPackage);
	FCommitPackageInfo& CommitInfo = ZenCommitInfo.CommitInfo;
	
	TUniquePtr<FPendingPackageState> PackageState = MoveTemp(ZenCommitInfo.PackageState);
	checkf(PackageState.IsValid(), TEXT("Trying to commit non-pending package '%s'"), *CommitInfo.PackageName.ToString());

	IPackageStoreWriter::FCommitEventArgs CommitEventArgs;

	CommitEventArgs.PlatformName = TargetPlatformFName;
	CommitEventArgs.PackageName = CommitInfo.PackageName;
	CommitEventArgs.EntryIndex = INDEX_NONE;

	const bool bWriteHashFlag = EnumHasAnyFlags(CommitInfo.WriteOptions, EWriteOptions::ComputeHash);
	const bool bWriteOp = EnumHasAnyFlags(CommitInfo.WriteOptions, EWriteOptions::Write);
	const bool bComputeHash = bWriteHashFlag && CommitInfo.Status == ECommitStatus::Success;
	const bool bComputeValidPackage = bWriteOp && CommitInfo.Status == ECommitStatus::Success;

	if (bWriteOp)
	{
		checkf(EnumHasAllFlags(CommitInfo.WriteOptions, EWriteOptions::Write), TEXT("Partial EWriteOptions::Write options are not yet implemented."));
		checkf(!EnumHasAnyFlags(CommitInfo.WriteOptions, EWriteOptions::SaveForDiff), TEXT("-diffonly -savefordiff is not yet implemented."));

		FPackageStoreEntryResource PackageStoreEntry;

		if (bComputeValidPackage)
		{
			checkf(!PackageState->PackageData.IsEmpty(), TEXT("CommitPackage called with bSucceeded but without first calling WritePackageData"));

			FPackageDataEntry* PkgData = nullptr;
			FPackageDataEntry* OptionalSegmentPkgData = nullptr;
			for (FPackageDataEntry& PackageDataEntry : PackageState->PackageData)
			{
				check(PackageDataEntry.Info.MultiOutputIndex <= 1);
				if (PackageDataEntry.Info.MultiOutputIndex == 0)
				{
					check(!PkgData);
					PkgData = &PackageDataEntry;
				}
				else if (PackageDataEntry.Info.MultiOutputIndex == 1)
				{
					check(!OptionalSegmentPkgData);
					OptionalSegmentPkgData = &PackageDataEntry;
				}
			}
			PackageStoreEntry = PackageStoreOptimizer->CreatePackageStoreEntry(
				PkgData->OptimizedPackage.Get(),
				OptionalSegmentPkgData ? OptionalSegmentPkgData->OptimizedPackage.Get() : nullptr);
		}
		else
		{
			const bool bHasCookError = CommitInfo.Status == ECommitStatus::Error;
			PackageStoreEntry = FPackageStoreEntryResource::CreateEmptyPackage(
				CommitInfo.PackageName, bHasCookError);
		}

		FMD5 PkgHashGen;
		FCbPackage OplogEntry;
		// Commit attachments
		const int32 NumAttachments = CommitInfo.Attachments.Num();
		TArray<FOplogCookInfo::FAttachment> CookInfoAttachments;
		TArray<FCbAttachment, TInlineAllocator<2>> CbAttachments;
		TArray<const FCommitAttachmentInfo*, TInlineAllocator<2>> SortedAttachments;

		if (NumAttachments)
		{
			SortedAttachments.Reserve(NumAttachments);
			for (const FCommitAttachmentInfo& Attachment : CommitInfo.Attachments)
			{
				SortedAttachments.Add(&Attachment);
			}

			SortedAttachments.Sort([](const FCommitAttachmentInfo& A, const FCommitAttachmentInfo& B)
			{
				return A.Key.ToString().Compare(B.Key.ToString(), ESearchCase::IgnoreCase) < 0;
			});

			CbAttachments.Reserve(NumAttachments);
			CookInfoAttachments.Reserve(NumAttachments);

			for (const FCommitAttachmentInfo* Attachment : SortedAttachments)
			{
				if (Attachment->FieldStorage == UE::Cook::Artifact::EFieldStorage::Attachment)
				{
					const FString KeyStr = Attachment->Key.ToString();
					auto KeyConversion = StringCast<UTF8CHAR>(*KeyStr);
					const FUtf8StringView KeyView(KeyConversion.Get(), KeyConversion.Length());
					check(!IsReservedOplogKey(KeyView));
					const FCbAttachment& CbAttachment = CbAttachments.Add_GetRef(CreateAttachment(Attachment->Value.GetBuffer().ToShared()));
					OplogEntry.AddAttachment(CbAttachment);

					CookInfoAttachments.Add(FOplogCookInfo::FAttachment
						{
							UE::FZenStoreHttpClient::FindOrAddAttachmentId(KeyView), CbAttachment.GetHash()
						});
				}
			}
		}

		// Create the oplog entry object
		FCbWriter OplogEntryDesc;
		OplogEntryDesc.BeginObject();
		FString PackageNameKey = CommitInfo.PackageName.ToString();
		PackageNameKey.ToLowerInline();
		OplogEntryDesc << "key" << PackageNameKey;
		OplogEntryDesc << "packagestoreentry" << PackageStoreEntry;

		auto AppendFileNameAndRegionsToOplog = [this, &OplogEntryDesc](const FString& LooseFilePath, const TArray<FFileRegion>& FileRegions)
		{
			FStringView RelativePathView;
			if (FPathViews::TryMakeChildPathRelativeTo(LooseFilePath, OutputPath, RelativePathView))
			{
				OplogEntryDesc << "filename" << RelativePathView;
			}
			if (!FileRegions.IsEmpty())
			{
				OplogEntryDesc.BeginArray("fileregions");
				for (const FFileRegion& FileRegion : FileRegions)
				{
					OplogEntryDesc << FileRegion;
				}
				OplogEntryDesc.EndArray();
			}
		};

		if (bComputeValidPackage && PackageState->PackageData.Num())
		{
			OplogEntryDesc.BeginArray("packagedata");

			for (FPackageDataEntry& PkgData : PackageState->PackageData)
			{
				FCompressedBuffer Payload = PkgData.CompressedPayload.GetResult();
				if (bComputeHash)
				{
					PackageState->PackageHashes->ChunkHashes.Add(PkgData.Info.ChunkId, Payload.GetRawHash());
				}

				FCbAttachment PkgDataAttachment = FCbAttachment(Payload);
				PkgHashGen.Update(PkgDataAttachment.GetHash().GetBytes(), sizeof(FIoHash::ByteArray));
				OplogEntry.AddAttachment(PkgDataAttachment);

				OplogEntryDesc.BeginObject();
				OplogEntryDesc << "id" << PkgData.ChunkId;
				OplogEntryDesc << "size" << Payload.GetCompressedSize();
				OplogEntryDesc << "rawsize" << Payload.GetRawSize();
				OplogEntryDesc << "data" << PkgDataAttachment;

				if (GZenStoreWriterExperimentalOutputPayloadInfo && PkgData.OptimizedPackage.IsValid())
				{
					const TArray<FBulkDataMapEntry>& BulkDataMap = PkgData.OptimizedPackage->GetBulkDataEntries();

					if (BulkDataMap.IsEmpty() == false)
					{
						// TODO: What if all payloads are inline?
						OplogEntryDesc.BeginArray("bulkdatapayloads");

						for (const FBulkDataMapEntry& MapEntry : BulkDataMap)
						{
							const EBulkDataFlags BulkDataFlags = static_cast<EBulkDataFlags>(MapEntry.Flags);
							if (FBulkData::HasFlags(BulkDataFlags, BULKDATA_PayloadAtEndOfFile) == false)
							{
								continue; // Skip inline
							}

							const FPackageId* PackageId = (const FPackageId*)PkgData.Info.ChunkId.GetData();	// TODO: Better to pass in this index but that can wait until larger interface refactors.
																												// but for now we know that the FPackageId is at the start of the FIoChunkId so this is
																												// ugly but safe.
							const EIoChunkType IoChunkType = GetIoChunkTypeFromFlags(BulkDataFlags);
							const FIoChunkId ChunkId = CreateBulkDataIoChunkId(PackageId->Value(), PkgData.Info.MultiOutputIndex, MapEntry.CookedIndex.GetValue(), IoChunkType);

							OplogEntryDesc.BeginObject();
							OplogEntryDesc << "chunkid" << ToObjectId(ChunkId);
							OplogEntryDesc << "offset" << MapEntry.SerialOffset;
							OplogEntryDesc << "duplicateoffset" << MapEntry.DuplicateSerialOffset;
							OplogEntryDesc << "size" << MapEntry.SerialSize;
							OplogEntryDesc << "flags" << MapEntry.Flags;
							OplogEntryDesc.EndObject();
						}

						OplogEntryDesc.EndArray();
					}
				}

				AppendFileNameAndRegionsToOplog(PkgData.Info.LooseFilePath, PkgData.FileRegions);
				OplogEntryDesc.EndObject();
			}

			OplogEntryDesc.EndArray();
		}

		if (bComputeValidPackage && PackageState->BulkData.Num())
		{
			OplogEntryDesc.BeginArray("bulkdata");

			for (FBulkDataEntry& Bulk : PackageState->BulkData)
			{
				FCompressedBuffer Payload = Bulk.CompressedPayload.GetResult();
				if (bComputeHash)
				{
					PackageState->PackageHashes->ChunkHashes.Add(Bulk.Info.ChunkId, Payload.GetRawHash());
				}

				FCbAttachment BulkAttachment(Payload);
				PkgHashGen.Update(BulkAttachment.GetHash().GetBytes(), sizeof(FIoHash::ByteArray));
				OplogEntry.AddAttachment(BulkAttachment);

				OplogEntryDesc.BeginObject();
				OplogEntryDesc << "id" << Bulk.ChunkId;
				OplogEntryDesc << "type" << LexToString(Bulk.Info.BulkDataType);
				OplogEntryDesc << "size" << Payload.GetCompressedSize();
				OplogEntryDesc << "rawsize" << Payload.GetRawSize();
				OplogEntryDesc << "data" << BulkAttachment;
				AppendFileNameAndRegionsToOplog(Bulk.Info.LooseFilePath, Bulk.FileRegions);
				OplogEntryDesc.EndObject();
			}

			OplogEntryDesc.EndArray();
		}

		if (bComputeValidPackage && PackageState->FileData.Num())
		{
			OplogEntryDesc.BeginArray("files");

			for (FFileDataEntry& File : PackageState->FileData)
			{
				if (bComputeHash)
				{
					PackageState->PackageHashes->ChunkHashes.Add(File.Info.ChunkId, File.CompressedPayload.GetResult().GetRawHash());
				}

				FCbAttachment FileDataAttachment(File.CompressedPayload.GetResult());
				PkgHashGen.Update(FileDataAttachment.GetHash().GetBytes(), sizeof(FIoHash::ByteArray));
				OplogEntry.AddAttachment(FileDataAttachment);

				OplogEntryDesc.BeginObject();
				OplogEntryDesc << "id" << ToObjectId(File.Info.ChunkId);
				// ZenServer treats the hash stored in "data" as mutually exlusive with the string stored in "serverpath".
				// We must write data as a zero hash (or exclude it entirely) if we want to be able to get the serverpath from ZenServer later.
				// This is relevant to incremental cooks which will obtain the filesystem manifest contents from ZenServer.
				OplogEntryDesc << "data" << FIoHash::Zero;
				OplogEntryDesc << "serverpath" << File.ZenManifestServerPath;
				OplogEntryDesc << "clientpath" << File.ZenManifestClientPath;
				OplogEntryDesc.EndObject();

				CommitEventArgs.AdditionalFiles.Add(FAdditionalFileInfo
				{ 
					CommitInfo.PackageName,
					File.ZenManifestClientPath,
					File.Info.ChunkId
				});
			}

			OplogEntryDesc.EndArray();
		}

		if (bComputeValidPackage)
		{
			for (const FCommitAttachmentInfo* Attachment : SortedAttachments)
			{
				if (Attachment->FieldStorage == UE::Cook::Artifact::EFieldStorage::Inline)
				{
					const FString KeyStr = Attachment->Key.ToString();
					auto KeyConversion = StringCast<UTF8CHAR>(*KeyStr);
					const FUtf8StringView KeyView(KeyConversion.Get(), KeyConversion.Length());

					OplogEntryDesc << KeyView << Attachment->Value;
				}
			}
		}

		if (bComputeHash)
		{
			PackageState->PackageHashes->PackageHash.Set(PkgHashGen);
		}

		for (int32 Index = 0; Index < CbAttachments.Num(); ++Index)
		{
			FCbAttachment& CbAttachment = CbAttachments[Index];
			FOplogCookInfo::FAttachment& CookInfoAttachment = CookInfoAttachments[Index];
			OplogEntryDesc << CookInfoAttachment.Key << CbAttachment;
		}
		OplogEntryDesc.EndObject();
		OplogEntry.SetObject(OplogEntryDesc.Save().AsObject());

		if (bComputeValidPackage && EntryCreatedEvent.IsBound())
		{
			IPackageStoreWriter::FEntryCreatedEventArgs EntryCreatedEventArgs
			{
				TargetPlatformFName,
				PackageStoreEntry
			};
			EntryCreatedEvent.Broadcast(EntryCreatedEventArgs);
		}

		{
			FWriteScopeLock _(EntriesLock);
			CommitEventArgs.EntryIndex = PackageNameToIndex.FindOrAdd(
				CommitInfo.PackageName, PackageStoreEntries.Num());
			if (CommitEventArgs.EntryIndex == PackageStoreEntries.Num())
			{
				PackageStoreEntries.Emplace();
				CookedPackagesInfo.Emplace(CommitInfo.PackageName);
			}
			PackageStoreEntries[CommitEventArgs.EntryIndex] = MoveTemp(PackageStoreEntry);

			FOplogCookInfo& CookInfo = CookedPackagesInfo[CommitEventArgs.EntryIndex];
			CookInfo.bUpToDate = true;
			CookInfo.Attachments = MoveTemp(CookInfoAttachments);
		}

		TIoStatusOr<uint64> Status = HttpClient->AppendOp(OplogEntry);
		if (!Status.IsOk())
		{
			UE_LOGF(LogZenStoreWriter, Warning, "Failed to commit oplog entry '%ls' to Zen, retrying: Reason: %ls", *CommitInfo.PackageName.ToString(), *Status.Status().ToString());
			Status = HttpClient->AppendOp(MoveTemp(OplogEntry));
			if (!Status.IsOk())
			{
				UE_LOGF(LogZenStoreWriter, Error, "Failed to commit oplog entry '%ls' to Zen: Reason: %ls", *CommitInfo.PackageName.ToString(), *Status.Status().ToString());
			}
			else
			{
				UE_LOGF(LogZenStoreWriter, Verbose, "Commit oplog entry '%ls' to Zen succeeded on retry", *CommitInfo.PackageName.ToString());
			}
		}
	}
	else if (bComputeHash)
	{
		checkf(!PackageState->PackageData.IsEmpty(), TEXT("CommitPackage called with bSucceeded but without first calling WritePackageData"));
		
		FMD5 PkgHashGen;
		
		for (FPackageDataEntry& PkgData : PackageState->PackageData)
		{
			FCompressedBuffer Payload = PkgData.CompressedPayload.GetResult();
			FIoHash IoHash = Payload.GetRawHash();
			PkgHashGen.Update(IoHash.GetBytes(), sizeof(FIoHash::ByteArray));
		}
		
		for (FBulkDataEntry& Bulk : PackageState->BulkData)
		{
			FCompressedBuffer Payload = Bulk.CompressedPayload.GetResult();
			FIoHash IoHash = Payload.GetRawHash();
			PackageState->PackageHashes->ChunkHashes.Add(Bulk.Info.ChunkId, IoHash);
			PkgHashGen.Update(IoHash.GetBytes(), sizeof(FIoHash::ByteArray));
		}
		
		for (FFileDataEntry& File : PackageState->FileData)
		{
			FCompressedBuffer Payload = File.CompressedPayload.GetResult();
			FIoHash IoHash = Payload.GetRawHash();
			PackageState->PackageHashes->ChunkHashes.Add(File.Info.ChunkId, IoHash);
			PkgHashGen.Update(IoHash.GetBytes(), sizeof(FIoHash::ByteArray));
		}
		
		PackageState->PackageHashes->PackageHash.Set(PkgHashGen);
	}

	if (bWriteOp)
	{
		BroadcastCommit(CommitEventArgs);
	}

	if (PackageState->PackageHashesCompletionPromise)
	{
		// Setting the CompletionFuture value may call arbitrary continuation code, so it
		// must be done outside of any lock.
		PackageState->PackageHashesCompletionPromise->EmplaceValue(0);
	}
	UE::SavePackageUtilities::DecrementOutstandingAsyncWrites();
}

void FZenStoreWriter::GetEntries(TFunction<void(TArrayView<const FPackageStoreEntryResource>, TArrayView<const FOplogCookInfo>)>&& Callback)
{
	FReadScopeLock _(EntriesLock);
	Callback(PackageStoreEntries, CookedPackagesInfo);
}

void FZenStoreWriter::PopulateOplog(const FAssetRegistryState& /* PreviousState*/, int32& OutNumPackagesInOplog)
{
	// We already populated the oplog during FZenStoreWriter::Initialize. Just return the number of packages.
	FReadScopeLock _(EntriesLock);
	OutNumPackagesInOplog = PackageNameToIndex.Num();
}

void FZenStoreWriter::UpdateLastReferenceDateAndPruneStaleOps(
	UE::Cook::Artifact::FUpdateOplogPackagesContext& OplogContext)
{
	// Not yet implemented. We need to store a LastReferenced timestamp in oplog entries, and this timestamp has to be
	// understood by the ZenServer so it can update the value for all ops that we pass to it in this function.
}

TArray<FName> FZenStoreWriter::GetOplogPackageNames()
{
	FReadScopeLock _(EntriesLock);
	TArray<FName> Result;
	PackageNameToIndex.GenerateKeyArray(Result);
	return Result;
}

FCbObject FZenStoreWriter::GetOplogAttachment(FName PackageName, FUtf8StringView AttachmentKey)
{
	FIoHash AttachmentHash;
	{
		FReadScopeLock _(EntriesLock);

		const int32* Idx = PackageNameToIndex.Find(PackageName);
		if (!Idx)
		{
			return FCbObject();
		}

		const UTF8CHAR* AttachmentId = UE::FZenStoreHttpClient::FindAttachmentId(AttachmentKey);
		if (!AttachmentId)
		{
			return FCbObject();
		}
		FUtf8StringView AttachmentIdView(AttachmentId);

		const FOplogCookInfo& CookInfo = CookedPackagesInfo[*Idx];
		int32 AttachmentIndex = Algo::LowerBound(CookInfo.Attachments, AttachmentIdView,
			[](const FOplogCookInfo::FAttachment& Existing, FUtf8StringView AttachmentIdView)
			{
				return FUtf8StringView(Existing.Key).Compare(AttachmentIdView, ESearchCase::IgnoreCase) < 0;
			});
		if (AttachmentIndex == CookInfo.Attachments.Num())
		{
			return FCbObject();
		}
		const FOplogCookInfo::FAttachment& Existing = CookInfo.Attachments[AttachmentIndex];
		if (!FUtf8StringView(Existing.Key).Equals(AttachmentIdView, ESearchCase::IgnoreCase))
		{
			return FCbObject();
		}
		AttachmentHash = Existing.Hash;
	}
	TIoStatusOr<FIoBuffer> BufferResult = HttpClient->ReadChunk(AttachmentHash);
	if (!BufferResult.IsOk())
	{
		return FCbObject();
	}
	FIoBuffer Buffer = BufferResult.ValueOrDie();
	if (Buffer.DataSize() == 0)
	{
		return FCbObject();
	}

	FSharedBuffer SharedBuffer = IoBufferToSharedBuffer(Buffer);
	return FCbObject(SharedBuffer);
}

void FZenStoreWriter::GetOplogAttachments(TArrayView<FName> PackageNames,
	TArrayView<FUtf8StringView> InAttachmentKeys,
	TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback)
{
	const int MaximumHashCount = PackageNames.Num() * InAttachmentKeys.Num();
	TArray<FIoHash> AttachmentHashes;
	AttachmentHashes.Reserve(MaximumHashCount);
	struct FAttachmentHashParam
	{
		FName PackageName;
		FUtf8StringView AttachmentKey;

		FAttachmentHashParam(const FName& InPackageName, FUtf8StringView InAttachmentKey)
		: PackageName(InPackageName), AttachmentKey(InAttachmentKey)
		{
		}
	};
	TMultiMap<FIoHash, FAttachmentHashParam> AttachmentHashParams;
	AttachmentHashParams.Reserve(MaximumHashCount);

	TArray<FAttachmentHashParam> InvalidAttachmentHashParams;
	InvalidAttachmentHashParams.Reserve(MaximumHashCount);

	TArray<FUtf8StringView, TInlineAllocator<2>> AttachmentIds;
	for (FUtf8StringView AttachmentKey : InAttachmentKeys)
	{
		const UTF8CHAR* Id = UE::FZenStoreHttpClient::FindOrAddAttachmentId(AttachmentKey);
		FUtf8StringView NewKey(Id);
		// The AttachmentId is supposed to be byte identical to the attachment looked up,
		// we look it up only so that we can have a single pointer value to use as Id.
		// Enforce the contract that the id is byte identical. If we ever change id to something
		// else (e.g. an abbreviation), then we will need to get a pointer to a persistent copy
		// of the key. We can not use the original AttachmentKey as our key because we store it
		// in an async function and we do not require that the caller maintain their copy of it
		// in memory after we return.
		check(NewKey.Equals(AttachmentKey, ESearchCase::CaseSensitive));
		AttachmentIds.Add(NewKey);
	}

	{
		FReadScopeLock _(EntriesLock);

		for (FName PackageName : PackageNames)
		{
			const int32* Idx = PackageNameToIndex.Find(PackageName);

			for (FUtf8StringView AttachmentKey : AttachmentIds)
			{
				const UTF8CHAR* AttachmentId = AttachmentKey.GetData();

				FIoHash AttachmentHash;
				ON_SCOPE_EXIT
				{
					if (AttachmentHash.IsZero())
					{
						InvalidAttachmentHashParams.Emplace(PackageName, AttachmentKey);
					}
					else
					{
						AttachmentHashes.Add(AttachmentHash);
						AttachmentHashParams.Emplace(AttachmentHash, FAttachmentHashParam{PackageName, AttachmentKey});
					}
				};

				if (!Idx || !AttachmentId)
				{
					continue;
				}

				FUtf8StringView AttachmentIdView(AttachmentId);

				const FOplogCookInfo& CookInfo = CookedPackagesInfo[*Idx];
				int32 AttachmentIndex = Algo::LowerBound(CookInfo.Attachments, AttachmentIdView,
					[](const FOplogCookInfo::FAttachment& Existing, FUtf8StringView AttachmentIdView)
					{
						return FUtf8StringView(Existing.Key).Compare(AttachmentIdView, ESearchCase::IgnoreCase) < 0;
					});
				if (AttachmentIndex == CookInfo.Attachments.Num())
				{
					continue;
				}
				const FOplogCookInfo::FAttachment& Existing = CookInfo.Attachments[AttachmentIndex];
				if (!FUtf8StringView(Existing.Key).Equals(AttachmentIdView, ESearchCase::IgnoreCase))
				{
					continue;
				}
				AttachmentHash = Existing.Hash;
			}
		}
	}

	// Invoke the callback for all invalid attachment hashes
	for (FAttachmentHashParam& InvalidAttachmentHashParam : InvalidAttachmentHashParams)
	{
		Callback(InvalidAttachmentHashParam.PackageName, InvalidAttachmentHashParam.AttachmentKey, FCbObject());
	}
	
	if (AttachmentHashes.IsEmpty())
	{
		return;
	}

	HttpClient->ReadChunksAsync(AttachmentHashes, [Callback = MoveTemp(Callback), AttachmentHashParams = MoveTemp(AttachmentHashParams)](const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)
	{
		for (auto It(AttachmentHashParams.CreateConstKeyIterator(RawHash)); It; ++It)
		{
			const FAttachmentHashParam& Param = It.Value();
			if (!Result.IsOk())
			{
				Callback(Param.PackageName, Param.AttachmentKey, FCbObject());
				continue;
			}

			FIoBuffer Buffer = Result.ConsumeValueOrDie();
			if (Buffer.DataSize() == 0)
			{
				Callback(Param.PackageName, Param.AttachmentKey, FCbObject());
				continue;
			}
			FSharedBuffer SharedBuffer = IoBufferToSharedBuffer(Buffer);
			Callback(Param.PackageName, Param.AttachmentKey, FCbObject(SharedBuffer));
		}
	});
}

void FZenStoreWriter::GetBaseGameOplogAttachments(TArrayView<FName> PackageNames,
	TArrayView<FUtf8StringView> InAttachmentKeys,
	TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback)
{
	if (ParentProjectId.IsEmpty())
	{
		for (FName PackageName : PackageNames)
		{
			for (FUtf8StringView AttachmentKey : InAttachmentKeys)
			{
				Callback(PackageName, AttachmentKey, FCbObject());
			}
		}
		return;
	}

	TArray<FUtf8StringView> AttachmentIds;
	AttachmentIds.Reserve(InAttachmentKeys.Num());
	for (FUtf8StringView AttachmentKey : InAttachmentKeys)
	{
		const UTF8CHAR* Id = UE::FZenStoreHttpClient::FindOrAddAttachmentId(AttachmentKey);
		FUtf8StringView NewKey(Id);
		// The AttachmentId is supposed to be byte identical to the attachment looked up,
		// we look it up only so that we can have a single pointer value to use as Id.
		// Enforce the contract that the id is byte identical. If we ever change id to something
		// else (e.g. an abbreviation), then we will need to get a pointer to a persistent copy
		// of the key. We can not use the original AttachmentKey as our key because we store it
		// in an async function and we do not require that the caller maintain their copy of it
		// in memory after we return.
		check(NewKey.Equals(AttachmentKey, ESearchCase::CaseSensitive));
		AttachmentIds.Add(NewKey);
	}
	TArray<FName> PackageNamesBuffer(PackageNames);
	TArray<FString> EntryKeys;
	EntryKeys.Reserve(PackageNames.Num());
	for (FName PackageName : PackageNames)
	{
		EntryKeys.Add(PackageName.ToString().ToLower());
	}

	HttpClient->GetOplogEntries(ParentProjectId, OplogId, MoveTemp(EntryKeys)).Next(
		[HttpClient = this->HttpClient.Get(), AttachmentIds = MoveTemp(AttachmentIds), Callback = MoveTemp(Callback),
		PackageNames = MoveTemp(PackageNamesBuffer)]
		(TArray<TIoStatusOr<FCbObject>> Entries) mutable
		{
			const int MaximumHashCount = PackageNames.Num() * AttachmentIds.Num();
			TArray<FIoHash> AttachmentHashes;
			AttachmentHashes.Reserve(MaximumHashCount);
			struct FAttachmentHashParam
			{
				FName PackageName;
				FUtf8StringView AttachmentKey;

				FAttachmentHashParam(const FName& InPackageName, FUtf8StringView InAttachmentKey)
					: PackageName(InPackageName), AttachmentKey(InAttachmentKey)
				{
				}
			};
			TMultiMap<FIoHash, FAttachmentHashParam> AttachmentHashParams;
			AttachmentHashParams.Reserve(MaximumHashCount);

			TArray<FAttachmentHashParam> InvalidAttachmentHashParams;
			InvalidAttachmentHashParams.Reserve(MaximumHashCount);

			TMap<FName, TMap<const UTF8CHAR*, FIoHash>> AttachmentsForPackage;
			for (TIoStatusOr<FCbObject>& EntryStatus : Entries)
			{
				if (!EntryStatus.IsOk())
				{
					continue;
				}

				FCbObject QueryResultObj = EntryStatus.ConsumeValueOrDie();
				FCbObjectView OplogObj = QueryResultObj["entry"].AsObjectView();
				if (OplogObj["packagestoreentry"])
				{
					FPackageStoreEntryResource Entry = FPackageStoreEntryResource::FromCbObject(OplogObj["packagestoreentry"].AsObjectView());
					const FName PackageName = Entry.PackageName;

					TMap<const UTF8CHAR*, FIoHash>& Attachments = AttachmentsForPackage.FindOrAdd(PackageName);

					for (FCbFieldView Field : OplogObj)
					{
						FUtf8StringView FieldName = Field.GetName();
						if (IsReservedOplogKey(FieldName))
						{
							continue;
						}
						if (Field.IsHash())
						{
							const UTF8CHAR* AttachmentId = UE::FZenStoreHttpClient::FindOrAddAttachmentId(FieldName);
							Attachments.Add(AttachmentId, Field.AsHash());
						}
					}
				}
			}

			for (FName PackageName : PackageNames)
			{
				TMap<const UTF8CHAR*, FIoHash>* Attachments = AttachmentsForPackage.Find(PackageName);

				for (FUtf8StringView AttachmentKey : AttachmentIds)
				{
					const UTF8CHAR* AttachmentId = AttachmentKey.GetData();
					FIoHash* AttachmentHash = (Attachments && AttachmentId) ? Attachments->Find(AttachmentId) : nullptr;
					if (AttachmentHash)
					{
						AttachmentHashes.Add(*AttachmentHash);
						AttachmentHashParams.Emplace(*AttachmentHash, FAttachmentHashParam{ PackageName, AttachmentKey });
					}
					else
					{
						InvalidAttachmentHashParams.Emplace(PackageName, AttachmentKey);
					}
				}
			}

			// Invoke the callback for all invalid attachment hashes
			for (FAttachmentHashParam& InvalidAttachmentHashParam : InvalidAttachmentHashParams)
			{
				Callback(InvalidAttachmentHashParam.PackageName, InvalidAttachmentHashParam.AttachmentKey, FCbObject());
			}

			if (AttachmentHashes.IsEmpty())
			{
				return;
			}

			HttpClient->ReadChunksAsync(AttachmentHashes,
				[Callback = MoveTemp(Callback), AttachmentHashParams = MoveTemp(AttachmentHashParams)]
				(const FIoHash& RawHash, TIoStatusOr<FIoBuffer> Result)
				{
					for (auto It(AttachmentHashParams.CreateConstKeyIterator(RawHash)); It; ++It)
					{
						const FAttachmentHashParam& Param = It.Value();
						if (!Result.IsOk())
						{
							Callback(Param.PackageName, Param.AttachmentKey, FCbObject());
							continue;
						}

						FIoBuffer Buffer = Result.ConsumeValueOrDie();
						if (Buffer.DataSize() == 0)
						{
							Callback(Param.PackageName, Param.AttachmentKey, FCbObject());
							continue;
						}
						FSharedBuffer SharedBuffer = IoBufferToSharedBuffer(Buffer);
						Callback(Param.PackageName, Param.AttachmentKey, FCbObject(SharedBuffer));
					}
				});
		});
}
IPackageWriter::ECommitStatus FZenStoreWriter::GetCommitStatus(FName PackageName)
{
	FReadScopeLock _(EntriesLock);

	const int32* Idx = PackageNameToIndex.Find(PackageName);
	if (!Idx)
	{
		return ECommitStatus::NotCommitted;
	}
	if (PackageStoreEntries[*Idx].HasPackageData())
	{
		return ECommitStatus::Success;
	}
	if (!PackageStoreEntries[*Idx].HasPackageData())
	{
		return ECommitStatus::NothingToCook;
	}
	return ECommitStatus::Error;
}

void FZenStoreWriter::RemoveCookedPackages(TArrayView<const FName> PackageNamesToRemove)
{
	FWriteScopeLock _(EntriesLock);

	TSet<int32> PackageIndicesToKeep;
	for (int32 Idx = 0, Num = PackageStoreEntries.Num(); Idx < Num; ++Idx)
	{
		PackageIndicesToKeep.Add(Idx);
	}
	
	for (const FName& PackageName : PackageNamesToRemove)
	{
		if (const int32* Idx = PackageNameToIndex.Find(PackageName))
		{
			PackageIndicesToKeep.Remove(*Idx);
		}
	}

	const int32 NumPackagesToKeep = PackageIndicesToKeep.Num();
	
	TArray<FPackageStoreEntryResource> PreviousPackageStoreEntries = MoveTemp(PackageStoreEntries);
	TArray<FOplogCookInfo> PreviousCookedPackageInfo = MoveTemp(CookedPackagesInfo);
	PackageNameToIndex.Empty();

	if (NumPackagesToKeep > 0)
	{
		PackageStoreEntries.Reserve(NumPackagesToKeep);
		CookedPackagesInfo.Reserve(NumPackagesToKeep);
		PackageNameToIndex.Reserve(NumPackagesToKeep);

		int32 EntryIndex = 0;
		for (int32 Idx : PackageIndicesToKeep)
		{
			const FName PackageName = PreviousCookedPackageInfo[Idx].PackageName;

			PackageStoreEntries.Add(MoveTemp(PreviousPackageStoreEntries[Idx]));
			CookedPackagesInfo.Add(MoveTemp(PreviousCookedPackageInfo[Idx]));
			PackageNameToIndex.Add(PackageName, EntryIndex++);
		}
	}
}

void FZenStoreWriter::RemoveCookedPackages()
{
	FWriteScopeLock _(EntriesLock);

	PackageStoreEntries.Empty();
	CookedPackagesInfo.Empty();
	PackageNameToIndex.Empty();
}

void FZenStoreWriter::UpdatePackageModifiedStatus(FUpdatePackageModifiedStatusContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FZenStoreWriter::UpdatePackageModifiedStatus);

	if (!Context.bIncrementallyUnmodified)
	{
		return;
	}

	IPackageStoreWriter::FMarkUpToDateEventArgs MarkUpToDateEventArgs;

	{
		FWriteScopeLock _(EntriesLock);
		int32* Index = PackageNameToIndex.Find(Context.PackageName);
		if (!Index)
		{
			if (!FPackageName::IsScriptPackage(WriteToString<128>(Context.PackageName)))
			{
				UE_LOGF(LogZenStoreWriter, Verbose, "UpdatePackageModifiedStatus called with package %ls that is not in the oplog.",
					*Context.PackageName.ToString());
			}
			return;
		}

		MarkUpToDateEventArgs.PackageIndexes.Add(*Index);
		CookedPackagesInfo[*Index].bUpToDate = true;
	}
	if (MarkUpToDateEventArgs.PackageIndexes.Num())
	{
		BroadcastMarkUpToDate(MarkUpToDateEventArgs);
	}
}

bool FZenStoreWriter::GetPreviousCookedBytes(const FPackageInfo& Info, FPreviousCookedBytesData& OutData)
{
	if (!Info.ChunkId.IsValid())
	{
		return false;
	}

	FIoReadOptions ReadOptions;
	TIoStatusOr<FIoBuffer> Status = HttpClient->ReadChunk(Info.ChunkId, ReadOptions.GetOffset(), ReadOptions.GetSize());
	if (!Status.IsOk())
	{
		return false;
	}

	FIoBuffer Buffer = Status.ConsumeValueOrDie();
	OutData.HeaderSize = reinterpret_cast<const FZenPackageSummary*>(Buffer.Data())->HeaderSize;
	OutData.Size = Buffer.GetSize();
	OutData.StartOffset = 0;
	Buffer.EnsureOwned();
	OutData.Data.Reset(Buffer.Release().ConsumeValueOrDie());

	return true;
}

void FZenStoreWriter::CompleteExportsArchiveForDiff(FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive)
{
	check(Info.ChunkId.IsValid());
	FPendingPackageState& ExistingState = GetPendingPackage(Info.PackageName);

	uint64 OptimizedHeaderSize = 0;
	TUniquePtr<FPackageStorePackage> PackageStorePackage;
	FIoBuffer PackageBuffer;
	{
		FIoBuffer CookedHeaderBuffer(FIoBuffer::Wrap, ExportsArchive.GetData(), Info.HeaderSize);
		FIoBuffer CookedExportsBuffer(FIoBuffer::Wrap, ExportsArchive.GetData() + Info.HeaderSize, ExportsArchive.TotalSize() - Info.HeaderSize);
		PackageStorePackage.Reset(PackageStoreOptimizer->CreatePackageFromCookedHeader(Info.PackageName, CookedHeaderBuffer));
		OptimizedHeaderSize = PackageStorePackage->GetHeaderSize();
		PackageBuffer = PackageStoreOptimizer->CreatePackageBuffer(PackageStorePackage.Get(), CookedExportsBuffer);
	}

	ExistingState.OriginalHeaderSize = Info.HeaderSize;
	ExportsArchive.Seek(0);
	FMemory::Free(ExportsArchive.ReleaseOwnership());
	ExportsArchive.Reserve(PackageBuffer.DataSize());
	ExportsArchive.Serialize(PackageBuffer.GetData(), PackageBuffer.DataSize());
	Info.HeaderSize = OptimizedHeaderSize;
	ExistingState.PreOptimizedPackage = MoveTemp(PackageStorePackage);
}

EPackageWriterResult FZenStoreWriter::BeginCacheForCookedPlatformData(
	FBeginCacheForCookedPlatformDataInfo& Info)
{
	check(Cooker); // Fallback for non-cooker case not yet implemented
	return Cooker->CookerBeginCacheForCookedPlatformData(Info);
}

TFuture<FCbObject> FZenStoreWriter::WriteMPCookMessageForPackage(FName PackageName)
{
	TArray<FString> AdditionalFiles;
	PackageAdditionalFiles.RemoveAndCopyValue(PackageName, AdditionalFiles);

	TRefCountPtr<FPackageHashes> PackageHashes;
	AllPackageHashes.RemoveAndCopyValue(PackageName, PackageHashes);

	auto ComposeMessage =
	[AdditionalFiles=MoveTemp(AdditionalFiles)](FPackageHashes* PackageHashes)
	{
		FCbWriter Writer;
		Writer.BeginObject();
		if (!AdditionalFiles.IsEmpty())
		{
			Writer << "AdditionalFiles" << AdditionalFiles;
		}
		if (PackageHashes)
		{
			Writer << "PackageHash" << PackageHashes->PackageHash;
			Writer << "ChunkHashes" << PackageHashes->ChunkHashes.Array();
		}
		Writer.EndObject();
		return Writer.Save().AsObject();
	};

	if (PackageHashes && PackageHashes->CompletionFuture.IsValid())
	{
		TUniquePtr<TPromise<FCbObject>> Promise(new TPromise<FCbObject>());
		TFuture<FCbObject> ResultFuture = Promise->GetFuture();
		PackageHashes->CompletionFuture.Next(
			[PackageHashes, Promise = MoveTemp(Promise), ComposeMessage = MoveTemp(ComposeMessage)](int)
			{
				Promise->SetValue(ComposeMessage(PackageHashes.GetReference()));
			});
		return ResultFuture;
	}
	else
	{
		TPromise<FCbObject> Promise;
		Promise.SetValue(ComposeMessage(PackageHashes.GetReference()));
		return Promise.GetFuture();
	}
}

bool FZenStoreWriter::TryReadMPCookMessageForPackage(FName PackageName, FCbObjectView Message)
{
	TArray<FString> AdditionalFiles;
	if (LoadFromCompactBinary(Message["AdditionalFiles"], AdditionalFiles))
	{
		UE::TUniqueLock LockScope(ZenFileSystemManifestMutex);
		for (const FString& Filename : AdditionalFiles)
		{
			ZenFileSystemManifest->CreateManifestEntry(Filename);
		}
	}

	bool bOk = true;
	TRefCountPtr<FPackageHashes> ThisPackageHashes(new FPackageHashes());
	if (LoadFromCompactBinary(Message["PackageHash"], ThisPackageHashes->PackageHash))
	{
		TArray<TPair<FIoChunkId, FIoHash>> LocalChunkHashes;
		bOk = LoadFromCompactBinary(Message["ChunkHashes"], LocalChunkHashes) & bOk;
		if (bOk)
		{
			for (TPair<FIoChunkId, FIoHash>& Pair : LocalChunkHashes)
			{
				ThisPackageHashes->ChunkHashes.Add(Pair.Key, Pair.Value);
			}
			bool bAlreadyExisted = false;
			TRefCountPtr<FPackageHashes>& ExistingPackageHashes = AllPackageHashes.FindOrAdd(PackageName);
			bAlreadyExisted = ExistingPackageHashes.IsValid();
			ExistingPackageHashes = ThisPackageHashes;
			if (bAlreadyExisted)
			{
				UE_LOGF(LogSavePackage, Error, "FZenStoreWriter encountered the same package twice in a cook! (%ls)",
					*PackageName.ToString());
			}
		}
	}

	return bOk;
}

void FZenStoreWriter::AppendOp(FName KeyName, FCbObject Payload)
{
	const FString KeyStr = KeyName.ToString();
	auto KeyConversion = StringCast<UTF8CHAR>(*KeyStr);
	const FUtf8StringView KeyView(KeyConversion.Get(), KeyConversion.Length());

	FCbPackage Pkg;
	FCbWriter PackageObj;

	PackageObj.BeginObject();
	PackageObj << "key" << KeyView;

	const FCbAttachment CbAttachment = CreateAttachment(Payload.GetBuffer().ToShared());
	Pkg.AddAttachment(CbAttachment);
	PackageObj << "value" << CbAttachment;
	PackageObj.EndObject();

	FCbObject Obj = PackageObj.Save().AsObject();

	Pkg.SetObject(Obj);

	TIoStatusOr<uint64> Status = HttpClient->AppendOp(MoveTemp(Pkg));
	UE_CLOGF(!Status.IsOk(), LogZenStoreWriter, Error, "Failed to append Op '%ls'. Reason: %ls", *KeyStr, *Status.Status().ToString());
}

FZenStoreWriter::FPendingPackageState& FZenStoreWriter::AddPendingPackage(const FName& PackageName)
{
	FScopeLock _(&PackagesCriticalSection);
	checkf(!PendingPackages.Contains(PackageName), TEXT("Trying to add package that is already pending"));
	TUniquePtr<FPendingPackageState>& Package = PendingPackages.Add(PackageName, MakeUnique<FPendingPackageState>());
	check(Package.IsValid());
	return *Package;
}

FCbPackage FZenStoreWriter::CreateReferencedSetOpPackage()
{
#if WITH_EDITOR
	UE::TUniqueLock LockScope(ZenFileSystemManifestMutex);
	TOptional<FZenFileSystemManifestEntry> ReferencedSet = ZenFileSystemManifest->GetReferencedSet();
	if (!ReferencedSet)
	{
		return FCbPackage();
	}
	FCbPackage Pkg;
	FCbWriter PackageObj;

	PackageObj.BeginObject();
	PackageObj << "key" << UE::Cook::GetReferencedSetOpName();
	PackageObj.BeginArray("files");
	WriteManifestEntryToPackageWriter(Pkg, PackageObj, *ReferencedSet);
	PackageObj.EndArray();
	PackageObj.EndObject();
	FCbObject Obj = PackageObj.Save().AsObject();

	Pkg.SetObject(Obj);
	return Pkg;
#else // else !WITH_EDITOR
	return FCbPackage();
#endif
}

FCbPackage FZenStoreWriter::CreateProjectMetaDataOpPackage(const ANSICHAR* MetadataOplogKeyName)
{
	FCbPackage Pkg;
	FCbWriter PackageObj;

	PackageObj.BeginObject();
	PackageObj << "key" << MetadataOplogKeyName;
	CreateProjectMetaData(Pkg, PackageObj);
	PackageObj.EndObject();

	Pkg.SetObject(PackageObj.Save().AsObject());
	return Pkg;
}

void FZenStoreWriter::CreateProjectMetaData(FCbPackage& Pkg, FCbWriter& PackageObj)
{
	// File Manifest
	{
		UE::TUniqueLock LockScope(ZenFileSystemManifestMutex);
		if (ZenFileSystemManifest->NumEntries() > 0)
		{
			TArrayView<FZenFileSystemManifestEntry const> Entries = ZenFileSystemManifest->ManifestEntries();

			PackageObj.BeginArray("files");
			for (const FZenFileSystemManifestEntry& NewEntry : Entries)
			{
				WriteManifestEntryToPackageWriter(Pkg, PackageObj, NewEntry);
			}
			PackageObj.EndArray();
		}

		FString ManifestPath = FPaths::Combine(MetadataDirectoryPath, TEXT("zenfs.manifest"));
		UE_LOGF(LogZenStoreWriter, Display, "Saving Zen filesystem manifest '%ls'", *ManifestPath);
		ZenFileSystemManifest->Save(*ManifestPath);
	}

	// Metadata section
	{
		PackageObj.BeginArray("meta");

		// Summarize Script Objects
		FIoBuffer ScriptObjectsBuffer = PackageStoreOptimizer->CreateScriptObjectsBuffer();
		FCbObjectId ScriptOid = ToObjectId(CreateIoChunkId(0, 0, EIoChunkType::ScriptObjects));

		FCbAttachment ScriptAttachment = CreateAttachment(ScriptObjectsBuffer); 
		Pkg.AddAttachment(ScriptAttachment);

		PackageObj.BeginObject();
		PackageObj << "id" << ScriptOid;
		PackageObj << "name" << "ScriptObjects";
		PackageObj << "data" << ScriptAttachment;
		PackageObj.EndObject();

		PackageObj.EndArray();	// End of Meta array
	}
}

void FZenStoreWriter::WriteManifestEntryToPackageWriter(FCbPackage& Pkg, FCbWriter& PackageObj,
	const FZenFileSystemManifestEntry& Entry) const
{
	FCbObjectId FileOid = ToObjectId(Entry.FileChunkId);

	if (IsLocalConnection)
	{
		PackageObj.BeginObject();
		PackageObj << "id" << FileOid;
		PackageObj << "data" << FIoHash::Zero;
		PackageObj << "serverpath" << Entry.ServerPath;
		PackageObj << "clientpath" << Entry.ClientPath;
		PackageObj.EndObject();
	}
	else
	{
		FString AbsPath = Entry.ServerPath;
		// The path can be a full path if the file was on another drive than engine's working directory,
		// if that's the  case ServerPath will be a fullpath, hence we don't prepend server root. 
		if (FPaths::IsRelative(Entry.ServerPath))
		{
			AbsPath = ZenFileSystemManifest->ServerRootPath() / AbsPath;
		}

		TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*AbsPath, 0));
		if (Reader)
		{
			int64 TotalSize = Reader->TotalSize();
			if (TotalSize > 0)
			{
				FIoBuffer FileBuffer(TotalSize);
				Reader->Serialize(FileBuffer.GetData(), TotalSize);
				bool Success = Reader->Close();
				FCbAttachment FileAttachment = CreateAttachment(MoveTemp(FileBuffer));

				PackageObj.BeginObject();
				PackageObj << "id" << FileOid;
				PackageObj << "data" << FileAttachment;
				PackageObj << "serverpath" << Entry.ServerPath;
				PackageObj << "clientpath" << Entry.ClientPath;
				PackageObj.EndObject();

				Pkg.AddAttachment(FileAttachment);
			}
		}
	}
}

void FZenStoreWriter::BroadcastCommit(IPackageStoreWriter::FCommitEventArgs& EventArgs)
{
	FScopeLock CommitEventLock(&CommitEventCriticalSection);
	
	if (CommitEvent.IsBound())
	{
		FReadScopeLock _(EntriesLock);
		EventArgs.Entries = PackageStoreEntries;
		CommitEvent.Broadcast(EventArgs);
	}
}

void FZenStoreWriter::BroadcastMarkUpToDate(IPackageStoreWriter::FMarkUpToDateEventArgs& EventArgs)
{
	FScopeLock CommitEventLock(&CommitEventCriticalSection);

	if (MarkUpToDateEvent.IsBound())
	{
		FReadScopeLock _(EntriesLock);
		EventArgs.PlatformName = TargetPlatformFName;
		EventArgs.Entries = PackageStoreEntries;
		EventArgs.CookInfos = CookedPackagesInfo;
		MarkUpToDateEvent.Broadcast(EventArgs);
	}
}

FCbAttachment FZenStoreWriter::CreateAttachment(FSharedBuffer AttachmentData) const
{
	check(AttachmentData.GetSize() > 0);
	FCompressedBuffer CompressedBuffer = FCompressedBuffer::Compress(AttachmentData, Compressor, CompressionLevel);
	check(!CompressedBuffer.IsNull());
	return FCbAttachment(CompressedBuffer);
}

FCbAttachment FZenStoreWriter::CreateAttachment(FIoBuffer AttachmentData) const
{
	return CreateAttachment(IoBufferToSharedBuffer(AttachmentData));
}
