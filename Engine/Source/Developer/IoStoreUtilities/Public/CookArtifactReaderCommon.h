// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Cooker/CookArtifactReader.h"
#include "GenericPlatform/GenericPlatformFile.h"

class FArchive;

class FCookArtifactReaderCommon
	: public UE::Cook::ICookArtifactReader
{
public:
	virtual ~FCookArtifactReaderCommon() = default;

	// Utilities
	IOSTOREUTILITIES_API FArchive* CreateFileReader(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	IOSTOREUTILITIES_API void FindFiles(TArray<FString>& Result, const TCHAR* Filename, bool Files, bool Directories) override;
	IOSTOREUTILITIES_API void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr) override;
};
