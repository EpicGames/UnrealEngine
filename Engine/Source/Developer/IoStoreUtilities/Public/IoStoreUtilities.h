// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/IEngineCrypto.h"
#include "Templates/Function.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "CoreMinimal.h"
#endif

struct FGuid;
struct FKeyChain;
class FIoChunkId;

#define UE_API IOSTOREUTILITIES_API

UE_API int32 CreateIoStoreContainerFiles(const TCHAR* CmdLine);

UE_API bool DumpIoStoreContainerInfo(const TCHAR* InContainerFilename, const FKeyChain& InKeyChain);

UE_API bool LegacyListIoStoreContainer(
	const TCHAR* InContainerFilename,
	int64 InSizeFilter,
	const FString& InCSVFilename,
	const FKeyChain& InKeyChain);

UE_API bool ListIoStoreContainer(const TCHAR* CmdLine);
UE_API bool ListIoStoreContainerBulkData(const TCHAR* CmdLine);
UE_API bool ListIoStoreContainerMetadataFile(const TCHAR* CmdLine);

UE_API bool DiffIoStoreContainer(const TCHAR* CmdLine);
UE_API bool LegacyDiffIoStoreContainers(
	const TCHAR* InContainerFilename1,
	const TCHAR* InContainerFilename2,
	bool bInLogUniques1,
	bool bInLogUniques2,
	const FKeyChain& InKeyChain1,
	const FKeyChain* InKeyChain2 = nullptr);

UE_API bool ExtractFilesFromIoStoreContainer(
	const TCHAR* InContainerFilename,
	const TCHAR* InDestPath,
	const FKeyChain& InKeyChain,
	const FString* InFilter,
	TMap<FString, uint64>* OutOrderMap,
	TArray<FGuid>* OutUsedEncryptionKeys,
	bool* bOutIsSigned);

UE_API bool ProcessFilesFromIoStoreContainer(
	const TCHAR* InContainerFilename,
	const TCHAR* InDestPath,
	const FKeyChain& InKeyChain,
	const FString* InFilter,
	TFunction<bool (const FString&, const FString&, const FIoChunkId&, const uint8*, uint64)> FileProcessFunc,
	TMap<FString, uint64>* OutOrderMap,
	TArray<FGuid>* OutUsedEncryptionKeys,
	bool* bOutIsSigned,
	int32 MaxConcurrentReaders);

UE_API bool SignIoStoreContainer(const TCHAR* InContainerFilename, const FRSAKeyHandle InSigningKey);

#undef UE_API
