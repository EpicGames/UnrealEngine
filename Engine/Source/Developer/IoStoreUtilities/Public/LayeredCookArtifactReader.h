// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CookArtifactReaderCommon.h"
#include "Templates/SharedPointer.h"

class FLayeredCookArtifactReader
	: public FCookArtifactReaderCommon
{
public:
	FLayeredCookArtifactReader() = default;

	void Initialize(bool bCleanBuild) override
	{
		for (TSharedRef<UE::Cook::ICookArtifactReader>& Layer : Layers)
		{
			Layer->Initialize(bCleanBuild);
		}
	}

	IOSTOREUTILITIES_API void AddLayer(TSharedRef<UE::Cook::ICookArtifactReader> InLayer);
	IOSTOREUTILITIES_API bool RemoveLayer(TSharedRef<UE::Cook::ICookArtifactReader> InLayer);
	IOSTOREUTILITIES_API void EmptyLayers();

	IOSTOREUTILITIES_API bool FileExists(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API int64 FileSize(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API IFileHandle* OpenRead(const TCHAR* Filename) override;

	IOSTOREUTILITIES_API bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
private:
	TArray<TSharedRef<UE::Cook::ICookArtifactReader>> Layers;
};
