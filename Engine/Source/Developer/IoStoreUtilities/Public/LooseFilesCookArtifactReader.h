// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CookArtifactReaderCommon.h"

class FLooseFilesCookArtifactReader
	: public FCookArtifactReaderCommon
{
public:
	FLooseFilesCookArtifactReader() = default;

	IOSTOREUTILITIES_API bool FileExists(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API int64 FileSize(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API IFileHandle* OpenRead(const TCHAR* Filename) override;

	IOSTOREUTILITIES_API bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
};
