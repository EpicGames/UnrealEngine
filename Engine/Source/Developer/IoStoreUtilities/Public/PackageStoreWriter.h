// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/BasePackageWriter.h"
#include "Serialization/PackageStore.h"

class FZenStoreWriter;

class IPackageStoreWriter : public FBaseCookedPackageWriter
{
public:
	/** Identify as a member of this interface from the ICookedPackageWriter api. */
	virtual IPackageStoreWriter* AsPackageStoreWriter() override
	{
		return this;
	}

	/** Downcast function for IPackageStoreWriter that is of class FZenStoreWriter. */
	virtual FZenStoreWriter* AsZenStoreWriter()
	{
		return nullptr;
	}

	struct FOplogCookInfo
	{
		struct FAttachment
		{
			const UTF8CHAR* Key;
			FIoHash Hash;
		};

		FName PackageName;
		TArray<FAttachment> Attachments;
		bool bUpToDate = false;
	};

	/**
	 * Returns all cooked package store entries.
	 */
	virtual void GetEntries(TFunction<void(TArrayView<const FPackageStoreEntryResource>, TArrayView<const FOplogCookInfo>)>&&) = 0;

	struct FEntryCreatedEventArgs
	{
		FName PlatformName;
		const FPackageStoreEntryResource& Entry;
	};

	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FEntryCreatedEvent, const FEntryCreatedEventArgs&);
	virtual FEntryCreatedEvent& OnEntryCreated() = 0;

	/**
	 * Package commit event arguments
	 */
	struct FCommitEventArgs
	{
		FName PlatformName;
		FName PackageName;
		int32 EntryIndex = INDEX_NONE;
		TArrayView<const FPackageStoreEntryResource> Entries;
		TArrayView<const FOplogCookInfo> CookInfos;
		TArray<FAdditionalFileInfo> AdditionalFiles;
	};

	/**
	 * Broadcasted after a package has been committed, i.e cooked.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FCommitEvent, const FCommitEventArgs&);
	virtual FCommitEvent& OnCommit() = 0;

	struct FMarkUpToDateEventArgs
	{
		FName PlatformName;
		TArray<int32> PackageIndexes;
		TArrayView<const FPackageStoreEntryResource> Entries;
		TArrayView<const FOplogCookInfo> CookInfos;
		TArray<FAdditionalFileInfo> AdditionalFiles;
	};
	/**
	 * Broadcasted after a set of packages have been found to be up to date.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FMarkUpToDateEvent, const FMarkUpToDateEventArgs&);
	virtual FMarkUpToDateEvent& OnMarkUpToDate() = 0;
};