// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CookArtifactReaderCommon.h"
#include "Templates/UniquePtr.h"

class IStorageServerPlatformFile;

namespace UE::Zen { class FScopeZenService; }

class FZenCookArtifactReader
	: public FCookArtifactReaderCommon
{
public:
	IOSTOREUTILITIES_API FZenCookArtifactReader(const FString& InputPath, 
												const FString& MetadataDirectoryPath, 
												const ITargetPlatform* TargetPlatform);
	IOSTOREUTILITIES_API virtual ~FZenCookArtifactReader();
	IOSTOREUTILITIES_API void InitializeConnection();
	IOSTOREUTILITIES_API void Initialize(bool bCleanBuild) override;

	IOSTOREUTILITIES_API bool FileExists(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API int64 FileSize(const TCHAR* Filename) override;
	IOSTOREUTILITIES_API IFileHandle* OpenRead(const TCHAR* Filename) override;

	IOSTOREUTILITIES_API bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
private:
#if !UE_BUILD_SHIPPING
	bool MakeStorageServerPath(const TCHAR* Filename, FString& OutFilename) const;

	TUniquePtr<UE::Zen::FScopeZenService> ScopeZenService;
	FString ZenRootPath;
	TUniquePtr<IStorageServerPlatformFile> StorageServerPlatformFile;
#endif // !UE_BUILD_SHIPPING
};
