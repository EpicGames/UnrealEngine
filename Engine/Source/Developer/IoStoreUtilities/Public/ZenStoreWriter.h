// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Async/Future.h"
#include "Async/EventCount.h"
#include "Compression/CompressedBuffer.h"
#include "Compression/OodleDataCompression.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "IO/IoDispatcher.h"
#include "Serialization/PackageStore.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "PackageStoreWriter.h"
#include "Serialization/AsyncLoading2.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/PackageWriter.h"
#include "Tasks/Task.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class FAssetRegistryState;
class FCompressedBuffer;
class FIoBuffer;
class FLargeMemoryWriter;
class FSharedBuffer;
class ITargetPlatform;
struct FFileRegion;
class FCbAttachment;
class FCbPackage;
class FCbWriter;
class FPackageStoreOptimizer;
class FPackageStorePackage;
class FZenFileSystemManifest;
namespace UE { class FZenStoreHttpClient; }
namespace UE::Cook { class ICookArtifactReader; }
struct FZenFileSystemManifestEntry;
template <typename FuncType> class TFunction;

/** 
 * A PackageStoreWriter that saves cooked packages for use by IoStore, and stores them in the Zen storage service.
 */
class FZenStoreWriter
	: public IPackageStoreWriter
{
public:
	IOSTOREUTILITIES_API FZenStoreWriter(	const FString& OutputPath, 
											const FString& MetadataDirectoryPath, 
											const ITargetPlatform* TargetPlatform,
											TSharedRef<UE::Cook::ICookArtifactReader> CookArtifactReader);

	IOSTOREUTILITIES_API ~FZenStoreWriter();

	static IOSTOREUTILITIES_API bool CanUse();

	/** Identify as a implmenter of this class from the IPackageStoreWriter api. */
	virtual FZenStoreWriter* AsZenStoreWriter() override
	{
		return this;
	};

	virtual void SetCooker(UE::PackageWriter::Private::ICookerInterface* CookerInterface) override;

	typedef TUniqueFunction<EPackageWriterResult(FBeginCacheForCookedPlatformDataInfo& Info)> 
		FDeprecatedBeginCacheCallback;
	typedef TUniqueFunction<void(UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)>
		FDeprecatedRegisterDeterminismHelperCallback;
	UE_DEPRECATED(5.7, "Use SetCooker instead")
	void SetBeginCacheCallback(FDeprecatedBeginCacheCallback&&)
	{
	}
	UE_DEPRECATED(5.7, "Use SetCooker instead")
	void SetRegisterDeterminismHelperCallback(FDeprecatedRegisterDeterminismHelperCallback&&)
	{
	}

	struct ZenHostInfo
	{
		FString ProjectId;
		FString OplogId;
		FString HostName;
		uint16 HostPort;
	};

	virtual FCapabilities GetCapabilities() const override
	{
		FCapabilities Result;
		Result.bDeterminismDebug = Cooker ? Cooker->IsDeterminismDebug() : false;
		return Result;
	}

	virtual FCookCapabilities GetCookCapabilities() const override
	{
		FCookCapabilities Result;
		Result.bDiffModeSupported = true;
		Result.bOplogAttachments = true;
		Result.bIgnorePathLengthLimits = true;
		Result.HeaderFormat = EPackageHeaderFormat::ZenPackageSummary;
		return Result;
	}

	IOSTOREUTILITIES_API ZenHostInfo GetHostInfo() const;

	IOSTOREUTILITIES_API virtual void BeginPackage(const FBeginPackageInfo& Info) override;
	IOSTOREUTILITIES_API virtual void CommitPackage(FCommitPackageInfo&& Info) override;

	IOSTOREUTILITIES_API virtual void WritePackageData(const FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive, const TArray<FFileRegion>& FileRegions) override;
	IOSTOREUTILITIES_API virtual void WriteAdditionalFile(const FAdditionalFileInfo& Info, const FIoBuffer& FileData) override;
	IOSTOREUTILITIES_API virtual void WriteLinkerAdditionalData(const FLinkerAdditionalDataInfo& Info, const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions) override;
	IOSTOREUTILITIES_API virtual void WritePackageTrailer(const FPackageTrailerInfo& Info, const FIoBuffer& Data) override;


	IOSTOREUTILITIES_API virtual void WriteBulkData(const FBulkDataInfo& Info, const FIoBuffer& BulkData, const TArray<FFileRegion>& FileRegions) override;
	IOSTOREUTILITIES_API virtual void RegisterDeterminismHelper(UObject* SourceObject,
		const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper) override;

	IOSTOREUTILITIES_API void InitializeConnection();
	IOSTOREUTILITIES_API virtual void Initialize(const FCookInfo& Info) override;
	IOSTOREUTILITIES_API virtual void BeginCook(const FCookInfo& Info) override;
	IOSTOREUTILITIES_API virtual void EndCook(const FCookInfo& Info) override;

	IOSTOREUTILITIES_API virtual void GetEntries(TFunction<void(TArrayView<const FPackageStoreEntryResource>, TArrayView<const FOplogCookInfo>)>&& Callback) override;

	DECLARE_DERIVED_EVENT(FZenStoreWriter, IPackageStoreWriter::FEntryCreatedEvent, FEntryCreatedEvent);
	virtual FEntryCreatedEvent& OnEntryCreated() override
	{
		return EntryCreatedEvent;
	}
	DECLARE_DERIVED_EVENT(FZenStoreWriter, IPackageStoreWriter::FCommitEvent, FCommitEvent);
	virtual FCommitEvent& OnCommit() override
	{
		return CommitEvent;
	}
	DECLARE_DERIVED_EVENT(FZenStoreWriter, IPackageStoreWriter::FMarkUpToDateEvent, FMarkUpToDateEvent);
	virtual FMarkUpToDateEvent& OnMarkUpToDate() override
	{
		return MarkUpToDateEvent;
	}

	IOSTOREUTILITIES_API void WriteIoStorePackageData(const FPackageInfo& Info, const FIoBuffer& PackageData, const FPackageStoreEntryResource& PackageStoreEntry, const TArray<FFileRegion>& FileRegions);

	IOSTOREUTILITIES_API virtual void PopulateOplog(const FAssetRegistryState& PreviousState,
		int32& OutNumPackagesInOplog) override;
	IOSTOREUTILITIES_API virtual void UpdateLastReferenceDateAndPruneStaleOps(
		UE::Cook::Artifact::FUpdateOplogPackagesContext& OplogContext) override;
	IOSTOREUTILITIES_API virtual TArray<FName> GetOplogPackageNames() override;
	IOSTOREUTILITIES_API virtual FCbObject GetOplogAttachment(FName PackageName, FUtf8StringView AttachmentKey) override;
	IOSTOREUTILITIES_API virtual void GetOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) override;
	IOSTOREUTILITIES_API virtual void GetBaseGameOplogAttachments(TArrayView<FName> PackageNames,
		TArrayView<FUtf8StringView> AttachmentKeys,
		TUniqueFunction<void(FName PackageName, FUtf8StringView AttachmentKey, FCbObject&& Attachment)>&& Callback) override;
	IOSTOREUTILITIES_API virtual ECommitStatus GetCommitStatus(FName PackageName) override;

	IOSTOREUTILITIES_API virtual void RemoveCookedPackages(TArrayView<const FName> PackageNamesToRemove) override;
	IOSTOREUTILITIES_API virtual void RemoveCookedPackages() override;
	IOSTOREUTILITIES_API virtual void UpdatePackageModifiedStatus(FUpdatePackageModifiedStatusContext& Context) override;
	IOSTOREUTILITIES_API bool GetPreviousCookedBytes(const FPackageInfo& Info, FPreviousCookedBytesData& OutData) override;
	IOSTOREUTILITIES_API void CompleteExportsArchiveForDiff(FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive) override;
	IOSTOREUTILITIES_API virtual EPackageWriterResult BeginCacheForCookedPlatformData(FBeginCacheForCookedPlatformDataInfo& Info) override;
	IOSTOREUTILITIES_API virtual TFuture<FCbObject> WriteMPCookMessageForPackage(FName PackageName) override;
	IOSTOREUTILITIES_API virtual bool TryReadMPCookMessageForPackage(FName PackageName, FCbObjectView Message) override;
	IOSTOREUTILITIES_API virtual void AppendOp(FName KeyName, FCbObject Payload) override;

	virtual TMap<FName, TRefCountPtr<FPackageHashes>>& GetPackageHashes() override
	{
		return AllPackageHashes;
	}

private:
	struct FBulkDataEntry
	{
		UE::Tasks::TTask<FCompressedBuffer> CompressedPayload;
		FBulkDataInfo Info;
		FCbObjectId ChunkId;
		TArray<FFileRegion> FileRegions;
		bool IsValid = false;
	};

	struct FPackageDataEntry
	{
		~FPackageDataEntry();

		UE::Tasks::TTask<FCompressedBuffer> CompressedPayload;
		FPackageInfo Info;
		FCbObjectId ChunkId;
		TUniquePtr<FPackageStorePackage> OptimizedPackage;
		TArray<FFileRegion> FileRegions;
		bool IsValid = false;
	};

	struct FFileDataEntry
	{
		UE::Tasks::TTask<FCompressedBuffer> CompressedPayload;
		FAdditionalFileInfo Info;
		FString ZenManifestServerPath;
		FString ZenManifestClientPath;
	};

	struct FPendingPackageState
	{
		~FPendingPackageState();

		FName PackageName;
		TArray<FPackageDataEntry> PackageData;
		TArray<FBulkDataEntry> BulkData;
		TArray<FFileDataEntry> FileData;
		TRefCountPtr<FPackageHashes> PackageHashes;
		TUniquePtr<TPromise<int>> PackageHashesCompletionPromise;

		/** Solely for use in DiffOnly mode */
		uint64 OriginalHeaderSize = 0;
		TUniquePtr<FPackageStorePackage> PreOptimizedPackage;
	};

	FPendingPackageState& GetPendingPackage(const FName& PackageName)
	{
		FScopeLock _(&PackagesCriticalSection);
		TUniquePtr<FPendingPackageState>& Package = PendingPackages.FindChecked(PackageName);
		checkf(Package.IsValid(), TEXT("Trying to retrieve non-pending package '%s'"), *PackageName.ToString());
		return *Package;
	}
	
	FPendingPackageState& AddPendingPackage(const FName& PackageName);
	
	TUniquePtr<FPendingPackageState> RemovePendingPackage(const FName& PackageName)
	{
		FScopeLock _(&PackagesCriticalSection);
		return PendingPackages.FindAndRemoveChecked(PackageName);
	}

	FCbPackage CreateReferencedSetOpPackage();
	FCbPackage CreateProjectMetaDataOpPackage(const ANSICHAR* MetadataOplogKeyName);
	void CreateProjectMetaData(FCbPackage& Pkg, FCbWriter& PackageObj);
	void WriteManifestEntryToPackageWriter(FCbPackage& Pkg, FCbWriter& PackageObj,
		const FZenFileSystemManifestEntry& Entry) const;
	void BroadcastCommit(IPackageStoreWriter::FCommitEventArgs& EventArgs);
	void BroadcastMarkUpToDate(IPackageStoreWriter::FMarkUpToDateEventArgs& EventArgs);
	struct FZenCommitInfo;

	void CommitPackageInternal(FZenCommitInfo&& CommitInfo);
	FCbAttachment CreateAttachment(FSharedBuffer Buffer) const;
	FCbAttachment CreateAttachment(FIoBuffer Buffer) const;
	FString GetProjectStorePath() const;
	void WriteProjectStoreFile();

	FCriticalSection								PackagesCriticalSection;
	TMap<FName, TUniquePtr<FPendingPackageState>>	PendingPackages;
	TUniquePtr<UE::FZenStoreHttpClient>	HttpClient;
	bool IsLocalConnection = true;

	TSharedRef<UE::Cook::ICookArtifactReader>		CookArtifactReader;
	const ITargetPlatform&				TargetPlatform;
	const FName							TargetPlatformFName;
	FString								ProjectId;
	FString								ParentProjectId;
	FString								OplogId;
	FString								OutputPath;
	FString								MetadataDirectoryPath;
	TMap<FName, TRefCountPtr<FPackageHashes>> AllPackageHashes;

	TUniquePtr<FPackageStoreOptimizer>	PackageStoreOptimizer;
	
	FRWLock								EntriesLock;
	TArray<FPackageStoreEntryResource>	PackageStoreEntries;
	TArray<FOplogCookInfo>				CookedPackagesInfo;
	TMap<FName, int32>					PackageNameToIndex;

	UE::FMutex				ZenFileSystemManifestMutex;
	TUniquePtr<FZenFileSystemManifest>	ZenFileSystemManifest;
	TMap<FName, TArray<FString>>		PackageAdditionalFiles;

	UE::PackageWriter::Private::ICookerInterface* Cooker = nullptr;

	std::atomic<int32>					NumCommitPending;
	UE::FEventCount						CommitPendingEvent;
	FEntryCreatedEvent					EntryCreatedEvent;
	FCriticalSection					CommitEventCriticalSection;
	FCommitEvent						CommitEvent;
	FMarkUpToDateEvent					MarkUpToDateEvent;

	ICookedPackageWriter::FCookInfo::ECookMode CookMode;

	FOodleDataCompression::ECompressor			Compressor;				
	FOodleDataCompression::ECompressionLevel	CompressionLevel;	
	
	bool								bInitialized;
	bool								bInitializedConnection;
	bool								bProvidePerPackageResults;

	static void StaticInit();
	static bool IsReservedOplogKey(FUtf8StringView Key);
	static TArray<const UTF8CHAR*>		ReservedOplogKeys;
};
